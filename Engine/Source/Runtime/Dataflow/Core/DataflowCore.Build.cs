// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataflowCore : ModuleRules
	{
        public DataflowCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"Core",
					"CoreUObject",
					"Chaos",
					"DeveloperSettings",
					"InputCore",
					"ImageCore",
					"SlateCore",
					"ApplicationCore"
				}
			);
		}
	}
}
