// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "Dataflow/DataflowConvertNodes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowAnyType)

namespace UE::Dataflow
{
	void RegisterAnyTypes()
	{
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowAnyType);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowAllTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowArrayTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowUStructTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowNumericTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowVectorTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowStringTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowBoolTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowTransformTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowStringConvertibleTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowUObjectConvertibleTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowSelectionTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowRotationTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowMeshSelectionTypes);

		// array types 
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowVectorArrayTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowNumericArrayTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowStringArrayTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowBoolArrayTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowTransformArrayTypes);

		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowNumericTypes, FDataflowNumericTypes, FConvertNumericTypesDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowVectorTypes, FDataflowVectorTypes, FConvertVectorTypesDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowStringTypes, FDataflowStringTypes, FConvertStringTypesDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowBoolTypes, FDataflowBoolTypes, FConvertBoolTypesDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowTransformTypes, FDataflowTransformTypes, FConvertTransformTypesDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowRotationTypes, FDataflowRotationTypes, FConvertRotationDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowVectorArrayTypes, FDataflowVectorArrayTypes, FConvertVectorArrayTypesDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowNumericArrayTypes, FDataflowNumericArrayTypes, FConvertNumericArrayTypesDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowStringArrayTypes, FDataflowStringArrayTypes, FConvertStringArrayTypesDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowBoolArrayTypes, FDataflowBoolArrayTypes, FConvertBoolArrayTypesDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowTransformArrayTypes, FDataflowTransformArrayTypes, FConvertTransformArrayTypesDataflowNode);
	}

	bool AreTypesCompatible(FName TypeA, FName TypeB)
	{
		return FAnyTypesRegistry::AreTypesCompatibleStatic(TypeA, TypeB);
	}
}
