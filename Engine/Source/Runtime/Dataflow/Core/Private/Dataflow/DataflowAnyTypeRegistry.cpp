// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowAnyTypeRegistry.h"

namespace UE::Dataflow
{
	bool FAnyTypesRegistry::AreTypesCompatibleStatic(FName TypeA, FName TypeB)
	{
		return GetInstance().AreTypesCompatible(TypeA, TypeB);
	}
	
	FName FAnyTypesRegistry::GetStorageTypeStatic(FName Type)
	{
		return GetInstance().GetStorageType(Type);
	}

	void FAnyTypesRegistry::RegisterAutoConvertNodeTypeStatic(FName FromType, FName ToType, FName AutoConvertNode, FTypeFilterFunction TypeFilterFunction)
	{
		GetInstance().RegisterAutoConvertNodeType(FromType, ToType, AutoConvertNode, TypeFilterFunction);
	}

	FName FAnyTypesRegistry::GetAutoConvertNodeTypeStatic(FName FromType, FName ToType)
	{
		return GetInstance().GetAutoConvertNodeType(FromType, ToType);
	}

	bool FAnyTypesRegistry::IsAnyTypeStatic(FName Type)
	{
		return GetInstance().IsAnyType(Type);
	}

	FAnyTypesRegistry& FAnyTypesRegistry::GetInstance()
	{
		static FAnyTypesRegistry Instance;
		return Instance;
	}

	bool FAnyTypesRegistry::AreTypesCompatible(FName TypeA, FName TypeB) const
	{
		if (TypeA == TypeB)
		{
			return true;
		}
		if (const FTypeInfo* TypeInfoA = TypeInfosByName.Find(TypeA))
		{
			if (TypeInfoA->SupportTypeFunction && (*TypeInfoA->SupportTypeFunction)(TypeB))
			{
				return true;
			}
		}
		if (const FTypeInfo* TypeInfoB = TypeInfosByName.Find(TypeB))
		{
			if (TypeInfoB->SupportTypeFunction && (*TypeInfoB->SupportTypeFunction)(TypeA))
			{
				return true;
			}
		}
		return false;
	}

	FName FAnyTypesRegistry::GetStorageType(FName Type) const
	{
		if (const FTypeInfo* TypeInfo = TypeInfosByName.Find(Type))
		{
			return TypeInfo->StorageType;
		}
		return NAME_None;
	}

	void FAnyTypesRegistry::RegisterAutoConvertNodeType(FName FromType, FName ToType, FName AutoConvertNode, FTypeFilterFunction TypeFilterFunction)
	{
		AutoConvertNodes.Add(
			FAutoConvertNodeInfo
			{
				.FromType = FromType,
				.ToType = ToType,
				.AutoConvertNode = AutoConvertNode,
				.TypeFilterFunction = TypeFilterFunction,
			});
	}

	FName FAnyTypesRegistry::GetAutoConvertNodeType(FName FromType, FName ToType) const
	{
		// check the node that matches both the from and to types
		for (const FAutoConvertNodeInfo& Info : AutoConvertNodes)
		{
			if (!Info.TypeFilterFunction.IsSet() || Info.TypeFilterFunction(FromType, ToType))
			{
				if (AreTypesCompatible(FromType, Info.FromType) && AreTypesCompatible(ToType, Info.ToType))
				{
					return Info.AutoConvertNode;
				}
			}
		}
		return NAME_None;
	}

	bool FAnyTypesRegistry::IsAnyType(FName Type) const
	{
		return TypeInfosByName.Contains(Type);
	}
};
