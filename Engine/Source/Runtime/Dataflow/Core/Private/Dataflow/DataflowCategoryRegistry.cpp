// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCategoryRegistry.h"
#include "Misc/LazySingleton.h"

namespace UE::Dataflow
{
	FCategoryRegistry& FCategoryRegistry::Get()
	{
		return TLazySingleton<FCategoryRegistry>::Get();
	}

	void FCategoryRegistry::TearDown()
	{
		return TLazySingleton<FCategoryRegistry>::TearDown();
	}

	void FCategoryRegistry::RegisterCategoryForAssetType(const FCategoryName Category, const FAssetType AssetType)
	{
		AssetTypesByCategory.FindOrAdd(Category).AddUnique(AssetType);
	}

	bool FCategoryRegistry::IsCategoryForAssetType(const FCategoryName Category, const FAssetType AssetType) const
	{
		if (const TArray<FAssetType>* AssetTypes = AssetTypesByCategory.Find(Category))
		{
			return AssetTypes->Contains(AssetType);
		}
		// category not found so it's a common one 
		return true;
	}
}