// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCompiledGraph.h"

namespace UE::Dataflow
{
	FCompiledGraph::FCompiledGraph(TSharedRef<const FGraph> InSourceGraph)
		: SourceGraph(InSourceGraph)
	{}

	TSharedRef<const FGraph> FCompiledGraph::GetSourceGraph() const
	{
		return SourceGraph;
	}

	int32 FCompiledGraph::StartTask()
	{
		return Tasks.Emplace(FTask{ .CommandIndex = Commands.Num() });
	}

	void FCompiledGraph::AddNode(FGuid NodeId, TConstArrayView<FGuid> UpstreamNodeIds, bool bIsTerminal)
	{
		if (ensure(Tasks.Num() > 0))
		{
			ensure(!NodeIdToIndex.Contains(NodeId));
			const int32 CurrentTaskIndex = Tasks.Num() - 1;
			Tasks[CurrentTaskIndex].NumCommands++;

			// Resolve upstream guids to indices. Topological build order guarantees they're already present;
			// any guid that doesn't resolve is dropped (defensive — would indicate compiler bug).
			const int32 UpstreamEdgeStart = UpstreamEdgesFlat.Num();
			int32 UpstreamEdgeCount = 0;
			for (const FGuid& UpstreamNodeId : UpstreamNodeIds)
			{
				if (const int32* UpstreamIndex = NodeIdToIndex.Find(UpstreamNodeId))
				{
					UpstreamEdgesFlat.Add(*UpstreamIndex);
					++UpstreamEdgeCount;
				}
				else
				{
					ensureMsgf(false, TEXT("FCompiledGraph::AddNode: upstream node not yet in graph (topological order violated?)"));
				}
			}

			const int32 NodeIndex = Nodes.Emplace(FNode{
				.NodeId = NodeId,
				.TaskIndex = CurrentTaskIndex,
				.UpstreamEdgeStart = UpstreamEdgeStart,
				.UpstreamEdgeCount = UpstreamEdgeCount,
			});
			NodeIdToIndex.Add(NodeId, NodeIndex);
			if (bIsTerminal)
			{
				TerminalNodeIndices.Add(NodeIndex);
			}
			Commands.Emplace(FCommand{ .Type = ECommandType::AddNode, .Parameter = NodeIndex });
		}
	}
	void FCompiledGraph::AddDependency(int32 TaskId)
	{
		if (ensure(Tasks.Num() > 0) && ensure(Tasks.IsValidIndex(TaskId)))
		{
			const int32 CurrentTaskIndex = Tasks.Num() - 1;
			Tasks[CurrentTaskIndex].NumCommands++;
			Commands.Emplace(FCommand{ .Type = ECommandType::AddDependency, .Parameter = TaskId });
		}
	}

	void FCompiledGraph::EndTask()
	{
		const int32 CurrentTaskIndex = Tasks.Num() - 1;
		if (ensure(Tasks.IsValidIndex(CurrentTaskIndex)))
		{
			Tasks[CurrentTaskIndex].NumCommands++;
			Commands.Emplace(FCommand{ .Type = ECommandType::AddDependency, .Parameter = CurrentTaskIndex });
		}
	}

	int32 FCompiledGraph::GetNumNodes() const
	{
		return Nodes.Num();
	}

	FGuid FCompiledGraph::GetNextNodeId(int32 NodeIndex) const
	{
		if (NodeIndex >= INDEX_NONE)
		{
			const int32 NextNodeIndex = NodeIndex + 1;
			if (Nodes.IsValidIndex(NextNodeIndex))
			{
				return Nodes[NextNodeIndex].NodeId;
			}
		}
		return FGuid();
	}

	int32 FCompiledGraph::FindNodeIndex(const FGuid& NodeId) const
	{
		const int32* FoundIndex = NodeIdToIndex.Find(NodeId);
		return FoundIndex ? *FoundIndex : INDEX_NONE;
	}

	FGuid FCompiledGraph::GetNodeId(int32 NodeIndex) const
	{
		return Nodes.IsValidIndex(NodeIndex) ? Nodes[NodeIndex].NodeId : FGuid();
	}

	TConstArrayView<int32> FCompiledGraph::GetUpstreamNodeIndices(int32 NodeIndex) const
	{
		if (!Nodes.IsValidIndex(NodeIndex))
		{
			return {};
		}
		const FNode& Node = Nodes[NodeIndex];
		return MakeArrayView(UpstreamEdgesFlat.GetData() + Node.UpstreamEdgeStart, Node.UpstreamEdgeCount);
	}

	TConstArrayView<int32> FCompiledGraph::GetTerminalNodeIndices() const
	{
		return TerminalNodeIndices;
	}

	int32 FCompiledGraph::GetNumTasks() const
	{
		return Tasks.Num();
	}

	void FCompiledGraph::GetTasksNodesAndDependencies(int32 TaskId, TArray<FGuid>& OutNodeIds, TArray<int32>& OutParentTaskIds) const
	{
		OutNodeIds.Reset();
		OutParentTaskIds.Reset();
		if (Tasks.IsValidIndex(TaskId))
		{
			for (int32 Offset = 0; Offset < Tasks[TaskId].NumCommands; ++Offset)
			{
				const int32 CommandIndex = Tasks[TaskId].CommandIndex + Offset;
				if (Commands.IsValidIndex(CommandIndex))
				{
					if (Commands[CommandIndex].Type == ECommandType::AddNode)
					{
						const int32 NodeIndex = Commands[CommandIndex].Parameter;
						if (Nodes.IsValidIndex(NodeIndex))
						{
							OutNodeIds.Add(Nodes[NodeIndex].NodeId);
						}
					}
					if (Commands[CommandIndex].Type == ECommandType::AddDependency)
					{
						OutParentTaskIds.Add(Commands[CommandIndex].Parameter);
					}
				}
			}
		}
	}
}
