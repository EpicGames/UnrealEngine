// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowConnection.h"

#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowConnection)

const UE::Dataflow::FConnectionKey UE::Dataflow::FConnectionKey::Invalid = { (uint32)INDEX_NONE, INDEX_NONE, (uint32)INDEX_NONE };
const UE::Dataflow::FPin UE::Dataflow::FPin::InvalidPin = { UE::Dataflow::FPin::EDirection::NONE, NAME_None, NAME_None };

FDataflowConnection::FDataflowConnection(UE::Dataflow::FPin::EDirection InDirection, const UE::Dataflow::FConnectionParameters& Params)
	: OwningNode(Params.Owner)
	, Property(Params.Property)
	, Guid(Params.Guid)
	, Offset(Params.Offset)
	, Direction(InDirection)
	, OriginalType(Params.Type)
	, Name(Params.Name)
{
	ensure(!OriginalType.ToString().Contains(TEXT(" ")));
	ensure(!Type.ToString().Contains(TEXT(" ")));
	SetTypeInternal(Params.Type);
	InitFromType();

}

void FDataflowConnection::InitFromType()
{	
	bIsAnyType = FDataflowConnection::IsAnyType(Type);
	bHasConcreteType = !bIsAnyType;
#if WITH_EDITOR
	if (Property && Property->GetClass()->IsChildOf(FStructProperty::StaticClass()))
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			const bool bInheritsFromAnyType = StructProperty->Struct->IsChildOf<FDataflowAnyType>();
			ensure(bIsAnyType == bInheritsFromAnyType);
		}
	}
	else
	{
		ensure(bIsAnyType == false);
	}
#endif
}

FName FDataflowConnection::GetTypeNameFromProperty(const FProperty* Property)
{
	if (Property)
	{
		FString ExtendedType;
		const FString CPPType = Property->GetCPPType(&ExtendedType);
		ExtendedType.RemoveSpacesInline();
		return FName(CPPType + ExtendedType);
	}
	return NAME_None;
}


bool FDataflowConnection::IsOwningNodeEnabled() const
{
	return (OwningNode && OwningNode->IsActive());
}

FGuid FDataflowConnection::GetOwningNodeGuid() const
{
	return OwningNode ? OwningNode->GetGuid() : FGuid();
}

UE::Dataflow::FTimestamp FDataflowConnection::GetOwningNodeTimestamp() const
{
	return OwningNode ? OwningNode->GetTimestamp() : UE::Dataflow::FTimestamp::Invalid;
}

uint32 FDataflowConnection::GetOwningNodeValueHash() const
{
	return OwningNode ? OwningNode->GetValueHash() : 0;
}

bool FDataflowConnection::IsAnyType(const FName& InType)
{
	return UE::Dataflow::FAnyTypesRegistry::IsAnyTypeStatic(InType);
}

void FDataflowConnection::SetTypeInternal(FName NewType)
{
	if (Type != NewType)
	{
		FString TypeAsString{ NewType.ToString() };
		if (TypeAsString.Contains(TEXT(" ")))
		{
			TypeAsString.RemoveSpacesInline();
			Type = FName(TypeAsString);
		}
		else
		{
			Type = NewType;
		}
	}
	ensure(!Type.ToString().Contains(TEXT(" ")));
}

void FDataflowConnection::SetAsAnyType(bool bAnyType, const FName& ConcreteType)
{
	bIsAnyType = bAnyType;
	if (bIsAnyType)
	{
		SetTypeInternal(ConcreteType);
		bHasConcreteType = !IsAnyType(ConcreteType);
	}
}

bool FDataflowConnection::SupportsType(FName InType) const
{
	// Incoming anytypes are not supported ( they need to be concrete types )
	if (IsAnyType(InType))
	{
		return false;
	}
	// resort to policy only if the concrete type is not defined ( case of anytype connection )
	if (bIsAnyType)
	{
		return TypePolicy ? TypePolicy->SupportsType(InType) : true;
	}
	// todo : in the future we could also check for pointer compatibility
	return (InType == GetType());
}

void FDataflowConnection::GetSupportedTypes(TArray<FName>& OutSupportedTypes) const
{
	if (bIsAnyType && TypePolicy)
	{
		TypePolicy->GetSupportedTypes(OutSupportedTypes);
	}
}

void FDataflowConnection::ForceTypeDependencyGroup(FName InTypeDependencyGroup)
{
	if (IsAnyType())
	{
		TypeDependencyGroup = InTypeDependencyGroup;
	}
}

FDataflowConnection& FDataflowConnection::SetTypeDependencyGroup(FName DependencyGroupName)
{
	if (IsAnyType() && !HasConcreteType())
	{
		TypeDependencyGroup = DependencyGroupName;
	}
	return *this;
}

bool FDataflowConnection::IsExtendedType(FName InType) const
{
	return (InType.ToString().StartsWith(Type.ToString() + "<"));
}

bool FDataflowConnection::IsSafeToTryChangingType() const
{
	if (!IsAnyType())
	{
		return false;
	}
	if (bLockType)
	{
		return false;
	}
	if (IsConnected())
	{
		return false;
	}
	if (IsAnytypeDependencyConnected())
	{
		return false;
	}
	return true;
}

bool FDataflowConnection::ResetToOriginalType()
{
	if (!IsAnyType())
	{
		// non any type simply have Type == OriginalType
		return true;
	}

	if (IsSafeToTryChangingType())
	{
		SetTypeInternal(OriginalType);
		bHasConcreteType = false;
		return true;
	}

	return false;
}

void FDataflowConnection::Rename(FName NewName)
{
	Name = NewName;
}

bool FDataflowConnection::IsAnytypeDependencyConnected() const
{
	if (OwningNode && !TypeDependencyGroup.IsNone())
	{
		return OwningNode->IsAnytypeDependencyConnected(TypeDependencyGroup);
	}
	return false;
}

bool FDataflowConnection::SetConcreteType(FName InType)
{
	// Can only change from AnyType to a concrete type
	if (Type != InType)
	{
		// special case when fixing types from Array to Array<
		const bool bExtendedType = IsExtendedType(InType);
		if (bExtendedType)
		{
			SetTypeInternal(InType);
			bHasConcreteType = true;
			return true;
		}
		// standard case: make sure we are safe to change and that this type is supported
		if (ensure(IsSafeToTryChangingType()))
		{
			if (ensure(SupportsType(InType)))
			{
				SetTypeInternal(InType);
				bHasConcreteType = true;
				return true;
			}
		}
	}
	return false;
}

void FDataflowConnection::SetTypePolicy(IDataflowTypePolicy* InTypePolicy)
{
	// for now only allow setting it once
	if (ensure(TypePolicy == nullptr))
	{
		TypePolicy = InTypePolicy;
	}
}

void FDataflowConnection::ForceSimpleType(FName InType)
{
	check(Type.ToString().StartsWith(InType.ToString()));
	SetTypeInternal(InType);
	bHasConcreteType = true;
}

void FDataflowConnection::FixAndPropagateType()
{
	check(Property);
	const FName FixedType = GetTypeNameFromProperty(Property);
	FixAndPropagateType(FixedType);
}

FString FDataflowConnection::GetPropertyTooltip() const
{
#if WITH_EDITORONLY_DATA
	check(Property);
	return Property->GetToolTipText().ToString();
#else
	return {};
#endif // WITH_EDITORONLY_DATA
}

FString FDataflowConnection::GetPropertyTypeNameTooltip() const
{
#if WITH_EDITORONLY_DATA
	FString TypeNameStr = Type.ToString();
	if (bIsAnyType && ensure(Property))
	{
		if (!HasConcreteType())
		{
			TypeNameStr = TEXT("Wildcard");
		}
		if (Property->GetClass()->IsChildOf(FStructProperty::StaticClass()))
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (StructProperty->Struct && StructProperty->Struct->IsChildOf<FDataflowAnyType>())
				{
					TypeNameStr += TEXT("\n");
					TypeNameStr += StructProperty->Struct->GetToolTipText().ToString();
				}
			}
		}
	}
	return TypeNameStr;
#else
	return {};
#endif
}