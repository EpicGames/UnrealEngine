// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowContextAssetStore.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"

namespace UE::Dataflow
{
	UObject* FContextAssetStore::AddAsset(const FString& PersistentAssetPath, const UClass* AssetClass)
	{
		if (AssetClass)
		{
			FStaticConstructObjectParameters Params(AssetClass);
			Params.Outer = GetTransientPackage();
			Params.SetFlags = RF_Public;
			if (TObjectPtr<UObject> NewObject = StaticConstructObject_Internal(Params))
			{
				AssetData.Emplace(
					FAssetData
					{
						.PersistentPath = PersistentAssetPath,
						.TransientPath = NewObject->GetPathName(),
						.TransientAsset = NewObject
					});
				return NewObject;
			}
		}
		return nullptr;
	}

	UObject* FContextAssetStore::CommitAsset(const FString& TransientAssetPath)
	{
		if (const FAssetData* FoundAssetData = FindByTransientPath(TransientAssetPath))
		{
			return CommitAsset(*FoundAssetData);
		}
		return nullptr;
	}

	UObject* FContextAssetStore::CommitAsset(const UObject* TransientAsset)
	{
		if (const FAssetData* FoundAssetData = FindByTransientAsset(TransientAsset))
		{
			return CommitAsset(*FoundAssetData);
		}
		return nullptr;
	}

	UObject* FContextAssetStore::CommitAsset(const FAssetData& InAssetData)
	{
		if (InAssetData.TransientAsset)
		{
			FString UniqueAssetPath{ InAssetData.PersistentPath };

			// make sure we have a unique name 
			if (FindPackage(nullptr, *UniqueAssetPath))
			{
				UniqueAssetPath = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName(UniqueAssetPath)).ToString();
			}
			// create and save the corresponding package
			if (UPackage* const NewAssetPackage = CreatePackage(*UniqueAssetPath))
			{
				const FString AssetName = FPaths::GetBaseFilename(UniqueAssetPath);
				if (UObject* const ObjectToCommit = DuplicateObject_Internal(InAssetData.TransientAsset->GetClass(), InAssetData.TransientAsset, NewAssetPackage, FName(AssetName)))
				{
					ObjectToCommit->SetFlags(RF_Standalone);
					ObjectToCommit->MarkPackageDirty();
					FAssetRegistryModule::AssetCreated(ObjectToCommit);
					return ObjectToCommit;
				}
			}
		}
		return nullptr;
	}

	void FContextAssetStore::ClearAssets()
	{
		AssetData.Empty();
	}

	const FContextAssetStore::FAssetData* FContextAssetStore::FindByTransientPath(const FString& TransientAssetPath) const
	{
		return AssetData.FindByPredicate([&TransientAssetPath](const FAssetData& Data)
			{
				return (Data.TransientPath == TransientAssetPath);
			});
	}

	const FContextAssetStore::FAssetData* FContextAssetStore::FindByTransientAsset(const UObject* TransientAsset) const
	{
		return AssetData.FindByPredicate([&TransientAsset](const FAssetData& Data)
			{
				return (Data.TransientAsset == TransientAsset);
			});
	}

	void FContextAssetStore::AddReferencedObjects(FReferenceCollector& Collector) 
	{ 
		for (FAssetData& Data : AssetData)
		{
			Collector.AddReferencedObject(Data.TransientAsset);
		}
	}
}