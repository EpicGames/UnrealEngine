// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowContextCache.h"
#include "UObject/StructOnScope.h"

namespace UE::Dataflow
{
	//////////////////////////////////////////////////////////////////////////////////////////////////////

	const void* FContextCacheElementReference::GetUntypedData(const IContextCacheStore& Context, const FProperty* InProperty) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = Context.FindCacheElement(DataKey))
		{
			return (*Cache)->GetUntypedData(Context, InProperty);
		}
		return nullptr;
	}

	bool FContextCacheElementReference::IsArray(const IContextCacheStore& Context) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = Context.FindCacheElement(DataKey))
		{
			return (*Cache)->IsArray(Context);
		}
		return false;
	}

	int32 FContextCacheElementReference::GetNumArrayElements(const IContextCacheStore& Context) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = Context.FindCacheElement(DataKey))
		{
			return (*Cache)->GetNumArrayElements(Context);
		}
		return 0;
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementReference::CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = Context.FindCacheElement(DataKey))
		{
			return (*Cache)->CreateFromArrayElement(Context, Index, InProperty, InNodeGuid, InNodeHash, InTimestamp);
		}
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementReference::CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = Context.FindCacheElement(DataKey))
		{
			return (*Cache)->CreateArrayFromElement(Context, InProperty, InNodeGuid, InNodeHash, InTimestamp);
		}
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementReference::CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = Context.FindCacheElement(DataKey))
		{
			return (*Cache)->CreateArrayFromElementAndAppend(DereferencedElements, Context, InProperty, InNodeGuid, InNodeHash, InTimestamp);
		}
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementReference::AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = Context.FindCacheElement(DataKey))
		{
			return (*Cache)->AppendArrayElements(DereferencedElements, Context, InProperty, InNodeGuid, InNodeHash, InTimestamp);
		}
		return {};
	}

	FConstStructView FContextCacheElementReference::GetConstStructView(const IContextCacheStore& Context) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = Context.FindCacheElement(DataKey))
		{
			return (*Cache)->GetConstStructView(Context);
		}
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementReference::Clone(const IContextCacheStore& Context) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* ReferencedCacheEntry = Context.FindCacheElement(DataKey))
		{
			if (ReferencedCacheEntry && ReferencedCacheEntry->IsValid())
			{
				return (*ReferencedCacheEntry)->Clone(Context);
			}
		}
		return MakeUnique<FContextCacheElementNull>(GetNodeGuid(), GetProperty(), GetNodeHash(), GetTimestamp());
	}

	const FContextCacheElementBase* FContextCacheElementReference::GetReferencedElement(const IContextCacheStore& Context) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* ReferencedCacheEntry = Context.FindCacheElement(DataKey))
		{
			if (ReferencedCacheEntry->IsValid())
			{
				if ((*ReferencedCacheEntry)->GetType() == EType::CacheElementReference)
				{
					// Recursive call
					return static_cast<const FContextCacheElementReference*>(ReferencedCacheEntry->Get())->GetReferencedElement(Context);
				}
				return ReferencedCacheEntry->Get();
			}
		}
		return nullptr;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	const void* FContextCacheElementNull::GetUntypedData(const IContextCacheStore& Context, const FProperty* PropertyIn) const
	{
		return nullptr;
	}

	bool FContextCacheElementNull::IsArray(const IContextCacheStore& Context) const
	{
		return false; // we have no way to tell at this point , null entry means that we don't have a value and we let the requester of teh value to get a default value 
	}

	int32 FContextCacheElementNull::GetNumArrayElements(const IContextCacheStore& Context) const
	{
		return 0;
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementNull::CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementNull::CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementNull::CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementNull::AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementNull::Clone(const IContextCacheStore& Context) const
	{
		return MakeUnique<FContextCacheElementNull>(GetNodeGuid(), GetProperty(), GetNodeHash(), GetTimestamp());
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	FContextCacheElementUStruct::FContextCacheElementUStruct(FGuid InNodeGuid, const FProperty* InProperty, const FConstStructView& StructView, uint32 InNodeHash, FTimestamp Timestamp)
		: FContextCacheElementBase(EType::CacheElementUStruct, InNodeGuid, InProperty, InNodeHash, Timestamp)
		, InstancedStruct(StructView)
	{}

	FContextCacheElementUStruct::FContextCacheElementUStruct(const FContextCacheElementUStruct& Other)
		: FContextCacheElementBase(EType::CacheElementUStruct, Other.GetNodeGuid(), Other.GetProperty(), Other.GetNodeHash(), Other.GetTimestamp())
		, InstancedStruct(Other.InstancedStruct)
	{}

	const void* FContextCacheElementUStruct::GetUntypedData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/) const
	{
		return InstancedStruct.GetMemory();
	}

	bool FContextCacheElementUStruct::IsArray(const IContextCacheStore& Context) const
	{
		return false;
	}

	int32 FContextCacheElementUStruct::GetNumArrayElements(const IContextCacheStore& Context) const
	{
		return 0;
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStruct::CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStruct::CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		return MakeUnique<FContextCacheElementUStructArray>(InNodeGuid, InProperty, FConstStructView(InstancedStruct), InNodeHash, InTimestamp);
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStruct::CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		TArray<FConstStructView> ElementViews{ FConstStructView(InstancedStruct) };
		ElementViews.Reserve(DereferencedElements.Num() + 1);
		for (int32 Index = 0; Index < DereferencedElements.Num(); ++Index)
		{
			check(DereferencedElements[Index]);
			FConstStructView ElemView = DereferencedElements[Index]->GetConstStructView(Context);
			if (ElemView.IsValid() && ElemView.GetScriptStruct() == InstancedStruct.GetScriptStruct())
			{
				ElementViews.Add(ElemView);
			}
		}
		return MakeUnique<FContextCacheElementUStructArray>(InNodeGuid, InProperty, ElementViews, InNodeHash, InTimestamp);
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStruct::AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		return {};
	}

	FConstStructView FContextCacheElementUStruct::GetConstStructView(const IContextCacheStore& Context) const
	{
		return FConstStructView(InstancedStruct);
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStruct::Clone(const IContextCacheStore& Context) const
	{
		return MakeUnique<FContextCacheElementUStruct>(*this);
	}

	void FContextCacheElementUStruct::AddReferencedObjects(FReferenceCollector& Collector)
	{
		InstancedStruct.AddStructReferencedObjects(Collector);
	}

	FString FContextCacheElementUStruct::GetReferencerName() const
	{
		return TEXT("FContextCacheElementUStruct");
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	const void* FContextCacheElementUStructArray::GetUntypedData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/) const
	{
		return InstancedStructArray.GetMemory();
	}

	bool FContextCacheElementUStructArray::IsArray(const IContextCacheStore& Context) const
	{
		return true;
	}

	int32 FContextCacheElementUStructArray::GetNumArrayElements(const IContextCacheStore& Context) const
	{
		return InstancedStructArray.Num();
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStructArray::CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		if (InstancedStructArray.IsValidIndex(Index))
		{
			const FConstStructView ElementStructView = InstancedStructArray.GetScriptViewAt(Index);
			return MakeUnique<FContextCacheElementUStruct>(GetNodeGuid(), GetProperty(), ElementStructView, GetNodeHash(), GetTimestamp());
		}
		return {};
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStructArray::CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		return {}; // can't make arrays of arrays
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStructArray::CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		return {}; // can't make arrays of arrays
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStructArray::AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		TUniquePtr<FContextCacheElementBase> Result;
		if (InstancedStructArray.GetScriptStruct())
		{
			Result = Clone(Context);
			check(Result.IsValid());
		}
		FContextCacheElementUStructArray* TypedResult = static_cast<FContextCacheElementUStructArray*>(Result.Get());

		TArray<const FInstancedStructArray*> OtherArrays;
		OtherArrays.Reserve(DereferencedElements.Num());
		for (const FContextCacheElementBase* Element : DereferencedElements)
		{
			check(Element);
			check(Element->GetType() == EType::CacheElementUStructArray);
			const FContextCacheElementUStructArray* TypedElement = static_cast<const FContextCacheElementUStructArray*>(Element);
			if (TypedElement->InstancedStructArray.GetScriptStruct())
			{
				if (!TypedResult)
				{
					Result = TypedElement->Clone(Context);
					TypedResult = static_cast<FContextCacheElementUStructArray*>(Result.Get());
					continue;
				}

				check(TypedElement->InstancedStructArray.GetScriptStruct() == TypedResult->InstancedStructArray.GetScriptStruct());
				OtherArrays.Add(&TypedElement->InstancedStructArray);
			}
		}

		if (TypedResult)
		{
			TypedResult->InstancedStructArray.AppendElements(OtherArrays);
		}
		return Result;
	}

	TUniquePtr<FContextCacheElementBase> FContextCacheElementUStructArray::Clone(const IContextCacheStore& Context) const
	{
		return MakeUnique<FContextCacheElementUStructArray>(*this);
	}

	void FContextCacheElementUStructArray::AddReferencedObjects(FReferenceCollector& Collector)
	{
		InstancedStructArray.AddReferencedObjects(Collector);
	}

	FString FContextCacheElementUStructArray::GetReferencerName() const
	{
		return TEXT("FContextCacheElementUStructArray");
	}

	FConstStructArrayView FContextCacheElementUStructArray::GetStructArrayView() const
	{
		return InstancedStructArray.GetScriptStruct() ?
			FConstStructArrayView(*InstancedStructArray.GetScriptStruct(), InstancedStructArray.GetData(), InstancedStructArray.Num()) :
			FConstStructArrayView();
	}

	FContextCacheElementUStructArray::FInstancedStructArray::FInstancedStructArray(const UScriptStruct* const InScriptStruct)
		: ScriptStruct(InScriptStruct)
	{}

	FContextCacheElementUStructArray::FInstancedStructArray::FInstancedStructArray(const FConstStructView& StructView)
		: ScriptStruct(StructView.GetScriptStruct())
	{
		InitFromRawData(StructView.GetMemory(), 1);
	}

	FContextCacheElementUStructArray::FInstancedStructArray::FInstancedStructArray(const TArray<FConstStructView>& StructViews)
		: ScriptStruct(StructViews.Num() ? StructViews[0].GetScriptStruct() : nullptr)
	{
		if (ScriptStruct && StructViews.Num())
		{
			ArrayNum = StructViews.Num();
			SetNumUnsafeInternal(ArrayNum);
			ArrayMax = ArrayNum;
			GetAllocatorInstance().ResizeAllocation(0, ArrayNum, GetStructureSize(), ScriptStruct->GetMinAlignment());
			ScriptStruct->InitializeStruct(GetData(), StructViews.Num());
			for (int32 Index = 0; Index < StructViews.Num(); ++Index)
			{
				check(StructViews[Index].GetScriptStruct() == ScriptStruct);
				ScriptStruct->CopyScriptStruct(GetData() + (GetStructureSize() * Index), StructViews[Index].GetMemory());
			}			
		}
	}

	FContextCacheElementUStructArray::FInstancedStructArray::FInstancedStructArray(const FConstStructArrayView& StructArrayView)
		: ScriptStruct(StructArrayView.GetScriptStruct())
	{
		InitFromRawData(StructArrayView.GetData(), StructArrayView.Num());
	}
	
	FContextCacheElementUStructArray::FInstancedStructArray::~FInstancedStructArray()
	{
		if (ScriptStruct)
		{
			ScriptStruct->DestroyStruct(GetData(), Num());
			GetAllocatorInstance().ResizeAllocation(ArrayMax, 0, GetStructureSize(), ScriptStruct->GetMinAlignment());
		}
	}

	const int32 FContextCacheElementUStructArray::FInstancedStructArray::GetStructureSize() const
	{
		return FMath::Max(1, ScriptStruct? ScriptStruct->GetStructureSize(): 0);
	}

	const UScriptStruct* FContextCacheElementUStructArray::FInstancedStructArray::GetScriptStruct() const
	{
		return ScriptStruct;
	}

	FConstStructView FContextCacheElementUStructArray::FInstancedStructArray::GetScriptViewAt(int32 Index) const
	{
		if (IsValidIndex(Index))
		{
			return FConstStructView(ScriptStruct, GetData() + (GetStructureSize() * Index));
		}
		return FConstStructView();
	}

	const void* FContextCacheElementUStructArray::FInstancedStructArray::GetMemory() const
	{
		return GetData();
	}

	void FContextCacheElementUStructArray::FInstancedStructArray::InitFromRawData(const void* Data, const int32 Num)
	{
		if (ScriptStruct)  // Null if the array has 0 element
		{
			ArrayNum = Num;
			// Note: SetNumUnsafeInternal cannot expand the array, but we use it here to make sure e.g. slack tracking is called (if enabled)
			SetNumUnsafeInternal(Num);
			ArrayMax = Num;
			GetAllocatorInstance().ResizeAllocation(0, Num, GetStructureSize(), ScriptStruct->GetMinAlignment());
			ScriptStruct->InitializeStruct(GetData(), Num);
			ScriptStruct->CopyScriptStruct(GetData(), Data, Num);
		}
	}

	void FContextCacheElementUStructArray::FInstancedStructArray::AddReferencedObjects(FReferenceCollector& Collector)
	{
		for(int32 Index = 0; IsValidIndex(Index); ++Index)
		{
			FStructOnScope ScopeStruct(ScriptStruct, GetData() + (GetStructureSize() * Index));
			ScopeStruct.AddReferencedObjects(Collector);
		}
	}

	void FContextCacheElementUStructArray::FInstancedStructArray::AddElements(const TArray<FConstStructView>& StructViews)
	{
		if (ScriptStruct && StructViews.Num())
		{
			const int32 OldArrayNum = ArrayNum;
			ArrayNum += StructViews.Num();
			SetNumUnsafeInternal(ArrayNum);
			ArrayMax = ArrayNum;
			GetAllocatorInstance().ResizeAllocation(OldArrayNum, ArrayNum, GetStructureSize(), ScriptStruct->GetMinAlignment());
			ScriptStruct->InitializeStruct(GetData() + (GetStructureSize() * OldArrayNum), StructViews.Num());
			for (int32 Index = 0; Index < StructViews.Num(); ++Index)
			{
				check(StructViews[Index].GetScriptStruct() == ScriptStruct);
				ScriptStruct->CopyScriptStruct(GetData() + (GetStructureSize() * (OldArrayNum + Index)), StructViews[Index].GetMemory());
			}
		}
	}

	void FContextCacheElementUStructArray::FInstancedStructArray::AppendElements(const TArray<const FInstancedStructArray*>& OtherArrays)
	{
		int32 NumNewElements = 0;
		for (const FInstancedStructArray* OtherArray : OtherArrays)
		{
			check(OtherArray);
			NumNewElements += OtherArray->Num();
		}

		if (ScriptStruct && NumNewElements > 0)
		{
			const int32 OldArrayNum = ArrayNum;
			ArrayNum += NumNewElements;
			SetNumUnsafeInternal(ArrayNum);
			ArrayMax = ArrayNum;
			GetAllocatorInstance().ResizeAllocation(OldArrayNum, ArrayNum, GetStructureSize(), ScriptStruct->GetMinAlignment());
			ScriptStruct->InitializeStruct(GetData() + (GetStructureSize() * OldArrayNum), NumNewElements);
			int32 ElementOffset = OldArrayNum;
			for(int32 Index = 0; Index < OtherArrays.Num(); ++ Index)
			{
				const FInstancedStructArray* OtherArray = OtherArrays[Index];
				check(OtherArray);
				check(OtherArray->GetScriptStruct() == ScriptStruct);
				check(ElementOffset + OtherArray->Num() <= ArrayNum);
				ScriptStruct->CopyScriptStruct(GetData() + (GetStructureSize() * ElementOffset), OtherArray->GetMemory(), OtherArray->Num());
				ElementOffset += OtherArray->Num();
			}
		}
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////

	FContextValue::FContextValue(IContextCacheStore& InContext, FContextCacheKey InCacheKey)
		: Context(InContext)
		, CacheKey(InCacheKey)
	{}

	FContextValue::FContextValue(IContextCacheStore& InContext, TUniquePtr<FContextCacheElementBase>&& InCacheElement)
		: Context(InContext)
		, CacheKey(0)
		, CacheElement(InCacheElement.Release())
	{}

	int32 FContextValue::Num() const
	{
		if (const FContextCacheElementBase* CacheEntry = GetCacheEntryPrivate())
		{
			return CacheEntry->GetNumArrayElements(Context);
		}
		return 0;
	}

	bool FContextValue::IsArray() const
	{
		if (const FContextCacheElementBase* CacheEntry = GetCacheEntryPrivate())
		{
			return CacheEntry->IsArray(Context);
		}
		return false;
	}

	FContextValue FContextValue::GetAt(int32 Index) const
	{
		if (const FContextCacheElementBase* CacheEntry = GetCacheEntryPrivate())
		{
			TUniquePtr<FContextCacheElementBase> NewCacheElement = CacheEntry->CreateFromArrayElement(Context, Index, CacheEntry->GetProperty(), CacheEntry->GetNodeGuid(), CacheEntry->GetNodeHash(), CacheEntry->GetTimestamp());
			return FContextValue(Context, MoveTemp(NewCacheElement));
		}
		return FContextValue(Context, 0);
	}

	FContextValue FContextValue::ToArray() const
	{
		if (const FContextCacheElementBase* CacheEntry = GetCacheEntryPrivate())
		{
			// arrays return themselves
			if (CacheEntry->IsArray(Context))
			{
				if (CacheElement)
				{
					return FContextValue(Context, CacheEntry->Clone(Context));
				}
				// simply return the reference to the original cached array 
				return FContextValue(Context, CacheKey);
			}

			// actually convert to an array 
			TUniquePtr<FContextCacheElementBase> NewCacheElement = CacheEntry->CreateArrayFromElement(Context, CacheEntry->GetProperty(), CacheEntry->GetNodeGuid(), CacheEntry->GetNodeHash(), CacheEntry->GetTimestamp());
			return FContextValue(Context, MoveTemp(NewCacheElement));
		}
		return FContextValue(Context, 0);
	}


	FContextValue FContextValue::MakeArray(IContextCacheStore& InContext, const TArray<const FContextValue>& Elements)
	{
		if (Elements.IsEmpty())
		{
			return FContextValue(InContext, 0);
		}

		// Find the first valid element's cache entry and collect remaining valid cache entries
		const FContextCacheElementBase* FirstCacheEntry = nullptr;
		TArray<const FContextCacheElementBase*> ElementEntries;
		ElementEntries.Reserve(Elements.Num());
		for (int32 Index = 0; Index < Elements.Num(); ++Index)
		{
			if (const FContextCacheElementBase* CacheEntry = Elements[Index].GetCacheEntryPrivate())
			{
				if (const FContextCacheElementBase* DeferencedCacheEntry = (CacheEntry->GetType() == FContextCacheElementBase::CacheElementReference) ?
					static_cast<const FContextCacheElementReference*>(CacheEntry)->GetReferencedElement(InContext) :
					CacheEntry)
				{
					if (!DeferencedCacheEntry->IsArray(InContext))
					{
						if (FirstCacheEntry == nullptr)
						{
							FirstCacheEntry = DeferencedCacheEntry;
							continue;
						}
						ElementEntries.Emplace(DeferencedCacheEntry);
					}
				}
			}
		}

		if (!FirstCacheEntry)
		{
			// Failed to find a valid cache entry
			return FContextValue(InContext, 0);
		}

		TUniquePtr<FContextCacheElementBase> NewCacheElement;
		if (ElementEntries.IsEmpty())
		{
			// Only one cache entry found. Equivalent to ToArray 
			NewCacheElement = FirstCacheEntry->CreateArrayFromElement(InContext, FirstCacheEntry->GetProperty(), FirstCacheEntry->GetNodeGuid(), FirstCacheEntry->GetNodeHash(), FirstCacheEntry->GetTimestamp());
		}
		else
		{
			NewCacheElement = FirstCacheEntry->CreateArrayFromElementAndAppend(ElementEntries, InContext, FirstCacheEntry->GetProperty(), FirstCacheEntry->GetNodeGuid(), FirstCacheEntry->GetNodeHash(), FirstCacheEntry->GetTimestamp());
		}
		return FContextValue(InContext, MoveTemp(NewCacheElement));
	}

	FContextValue FContextValue::AppendArrays(IContextCacheStore& InContext, const TArray<const FContextValue>& ArrayElements)
	{
		if (ArrayElements.IsEmpty())
		{
			return FContextValue(InContext, 0);
		}

		// Find the first valid element's cache entry and collect remaining valid cache entries
		const FContextCacheElementBase* FirstCacheEntry = nullptr;
		int32 FirstCacheEntryIndex = INDEX_NONE;
		TArray<const FContextCacheElementBase*> ElementEntries;
		ElementEntries.Reserve(ArrayElements.Num());
		for (int32 Index = 0; Index < ArrayElements.Num(); ++Index)
		{
			if (const FContextCacheElementBase* CacheEntry = ArrayElements[Index].GetCacheEntryPrivate())
			{
				if (const FContextCacheElementBase* DeferencedCacheEntry = (CacheEntry->GetType() == FContextCacheElementBase::CacheElementReference) ?
					static_cast<const FContextCacheElementReference*>(CacheEntry)->GetReferencedElement(InContext) :
					CacheEntry)
				{
					if (DeferencedCacheEntry->IsArray(InContext))
					{
						if (FirstCacheEntry == nullptr)
						{
							FirstCacheEntry = DeferencedCacheEntry;
							FirstCacheEntryIndex = Index;
							continue;
						}
						ElementEntries.Emplace(DeferencedCacheEntry);
					}
				}
			}
		}

		if (!FirstCacheEntry)
		{
			// Failed to find a valid cache entry
			return FContextValue(InContext, 0);
		}

		TUniquePtr<FContextCacheElementBase> NewCacheElement;
		if (ElementEntries.IsEmpty())
		{
			// Only one cache entry found. Equivalent to ToArray 
			return ArrayElements[FirstCacheEntryIndex].ToArray();
		}
		else
		{
			NewCacheElement = FirstCacheEntry->AppendArrayElements(ElementEntries, InContext, FirstCacheEntry->GetProperty(), FirstCacheEntry->GetNodeGuid(), FirstCacheEntry->GetNodeHash(), FirstCacheEntry->GetTimestamp());
		}
		return FContextValue(InContext, MoveTemp(NewCacheElement));
	}

	FConstStructView FContextValue::GetConstStructView() const
	{
		if (const FContextCacheElementBase* CacheEntry = GetCacheEntryPrivate())
		{
			return CacheEntry->GetConstStructView(Context);
		}
		return {};
	}

	FInstancedStruct FContextValue::GetInstancedStruct() const
	{
		FInstancedStruct OutInstancedStruct;
		FConstStructView StructView = GetConstStructView();
		if (StructView.IsValid())
		{
			OutInstancedStruct.InitializeAs(StructView.GetScriptStruct(), StructView.GetMemory());
		}
		return OutInstancedStruct;
	}


	const FContextCacheElementBase* FContextValue::GetCacheEntry() const
	{
		return GetCacheEntryPrivate();
	}

	const FContextCacheElementBase* FContextValue::GetCacheEntryPrivate() const
	{
		if (CacheElement)
		{
			return CacheElement.Get();
		}
		if (const TUniquePtr<FContextCacheElementBase>* CacheEntry = Context.FindCacheElement(CacheKey))
		{
			return CacheEntry->Get();
		}
		return nullptr;
	}
};

FArchive& operator<<(FArchive& Ar, UE::Dataflow::FTimestamp& ValueIn)
{
	Ar << ValueIn.Value;
	Ar << ValueIn.Invalid;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UE::Dataflow::FContextCache& ValueIn)
{
	ValueIn.Serialize(Ar);
	return Ar;
}




