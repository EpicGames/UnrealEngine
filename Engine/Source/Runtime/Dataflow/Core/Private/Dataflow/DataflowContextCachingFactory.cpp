// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowContextCachingFactory.h"

#include "Dataflow/DataflowNodeParameters.h"
#include "Logging/LogMacros.h"
#include "Misc/MessageDialog.h"

DEFINE_LOG_CATEGORY_STATIC(LogDataflowContextCachingFactory, Warning, All);


namespace UE::Dataflow
{
	FContextCachingFactory* FContextCachingFactory::Instance = nullptr;

	void FContextCachingFactory::RegisterSerializeFunction(const FName& Type, FSerializeFunction InSerializeFunc)
	{
		if (CachingMap.Contains(Type))
		{
			UE_LOGF(LogDataflowContextCachingFactory, Warning,
				"Warning : Dataflow output caching registration conflicts with "
					"existing type(%ls)", *Type.ToString());
		}
		else
		{
			CachingMap.Add(Type, InSerializeFunc);
		}
	}


	FContextCacheElementBase* FContextCachingFactory::Serialize(FArchive& Ar, FContextCacheData&& Element)
	{
		FContextCacheElementBase* RetVal = nullptr;
		if (CachingMap.Contains(Element.Type))
		{
			RetVal = CachingMap[Element.Type](Ar, Element.Data);
			if (Ar.IsSaving())
			{
				check(RetVal == nullptr);
			}
			else if( Ar.IsLoading())
			{
				check(RetVal != nullptr);
			}
		}
		else
		{
			UE_LOGF(LogDataflowContextCachingFactory, Warning,
				"Warning : Dataflow missing context chaching callback type(%ls)", *Element.Type.ToString());
		}
		return RetVal;
	}
}

