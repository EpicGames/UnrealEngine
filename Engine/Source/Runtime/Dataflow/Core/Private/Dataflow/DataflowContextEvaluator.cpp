// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowContextEvaluator.h"
#include "Dataflow/DataflowCompiledGraph.h"
#include "Dataflow/DataflowDemandPullSlice.h"
#include "Dataflow/DataflowGraph.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowSystem.h"
#include "HAL/IConsoleManager.h"

#define LOCTEXT_NAMESPACE "DataflowEvaluator"

DEFINE_LOG_CATEGORY_STATIC(LogDataflowEvaluator, Log, All);

static TAutoConsoleVariable<int32> CVarUseDemandPullDirect(
	TEXT("p.Dataflow.UseDemandPullDirect"),
	1,
	TEXT("When non-zero, FDataflowDirectEvaluator uses the demand-pull (RTL slice-aware) path. ")
	TEXT("When zero, it falls back to the legacy left-to-right walk. Default: 1."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarUseDemandPullTick(
	TEXT("p.Dataflow.UseDemandPullTick"),
	1,
	TEXT("When non-zero, FDataflowTickEvaluator uses the demand-pull (RTL slice-aware) path. ")
	TEXT("When zero, it falls back to the legacy left-to-right walk. Default: 1."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarUseDemandPullTaskGraph(
	TEXT("p.Dataflow.UseDemandPullTaskGraph"),
	1,
	TEXT("When non-zero, FDataflowTaskGraphEvaluator filters per-task node lists by the demand-pull slice ")
	TEXT("and skips launching tasks whose entire content is already cached. When zero, every compiled task ")
	TEXT("is launched as before. Default: 1."),
	ECVF_Default);

namespace UE::Dataflow
{
	void FContextEvaluator::ScheduleNodeEvaluation(const FDataflowNode& Node, FOnPostEvaluationFunction OnPostEvaluation)
	{
		FEvaluationEntry Entry
		{
			.Id = FNodeId{ Node.GetGuid() },
			.WeakNode = Node.AsWeak(),
			.OnPostEvaluation = OnPostEvaluation,
		};
		ScheduleEvaluation(Entry);
		Process();
	}

	void FContextEvaluator::ScheduleOutputEvaluation(const FDataflowOutput& Output, FOnPostEvaluationFunction OnPostEvaluation)
	{
		if (const FDataflowNode* Node = Output.GetOwningNode())
		{
			ScheduleNodeEvaluation(*Node, OnPostEvaluation);
		}
	}

	void FContextEvaluator::ScheduleEvaluation(const FEvaluationEntry& Entry)
	{
		if (IsScheduledOrRunning(Entry.Id))
		{
			UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::ScheduleEvaluation : skipped [%ls]", *Entry.ToString());
			return;
		}
		UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::ScheduleEvaluation : [%ls]", *Entry.ToString());
		PendingEvaluationEntries.Add(Entry.Id, Entry);

		// add upstream nodes
		if (TSharedPtr<const FDataflowNode> Node = Entry.WeakNode.Pin())
		{
			TArray<const FDataflowNode*> InvalidUpstreamNodes;
			FindInvalidUpstreamNodes(*Node, InvalidUpstreamNodes);
			for (const FDataflowNode* UpstreamNode: InvalidUpstreamNodes)
			{
				UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::ScheduleEvaluation :  [%ls] -- Invalid Upstream Node [%ls]", *Entry.ToString(), *UpstreamNode->GetName().ToString());
			}
			for (const FDataflowNode* UpstreamNode : InvalidUpstreamNodes)
			{
				FEvaluationEntry UpstreamEntry
				{
					.Id = FNodeId{ UpstreamNode->GetGuid() },
					.WeakNode = UpstreamNode->AsWeak(),
					.OnPostEvaluation = {},
				};
				ScheduleEvaluation(UpstreamEntry);
			}
		}
	}

	bool FContextEvaluator::IsScheduledOrRunning(const FNodeId& Id) const
	{
		return (RunningTasks.Contains(Id) || PendingEvaluationEntries.Contains(Id));
	}

	void FContextEvaluator::Cancel()
	{
		PendingEvaluationEntries.Reset();
		CompletedTasks.Reset();
	}

	void FContextEvaluator::FindInvalidUpstreamNodes(const FDataflowNode& Node, TArray<const FDataflowNode*>& OutInvalidUpstreamNodes)
	{
		for (const FDataflowInput* Input : Node.GetInputs())
		{
			if (Input)
			{
				if (const FDataflowOutput* UpstreamOutput = Input->GetConnection())
				{
					UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::FindInvalidUpstreamOutputs :  [%ls] input[%ls] -> output [%ls]",
						*Node.GetName().ToString(),
						*Input->GetName().ToString(),
						*UpstreamOutput->GetName().ToString()
					);

					if (!UpstreamOutput->HasValidData(OwningContext))
					{
						if (const FDataflowNode* UpstreamNode = UpstreamOutput->GetOwningNode())
						{
							OutInvalidUpstreamNodes.AddUnique(UpstreamNode);
						}
					}
				}
			}
		}
	}

	bool FContextEvaluator::ShouldRunOnGameThread(const FDataflowNode& Node)
	{
		for (const FDataflowInput* Input : Node.GetInputs())
		{
			if (Input)
			{
				const FString InputTypeName = Input->GetType().ToString();
				// skeletal mesh and static mesh do support asynchronous loading and do not allow for accessing 
				// their property from elsewhere than the gamethread
				if (InputTypeName.Contains("UStaticMesh") || InputTypeName.Contains("USkeletalMesh"))
				{
					return true;
				}
			}
		}
		return Node.EvaluateOnGameThreadOnly();
	}

	bool FContextEvaluator::TryScheduleTask(const FEvaluationEntry& Entry)
	{
		if (TSharedPtr<const FDataflowNode> Node = Entry.WeakNode.Pin())
		{
			// todo(ccaillaud) : can optimize by findingthe first one
			TArray<const FDataflowNode*> InvalidUpstreamNodes;
			FindInvalidUpstreamNodes(*Node, InvalidUpstreamNodes);
			if (InvalidUpstreamNodes.IsEmpty())
			{
				ScheduleTask(Entry);
				return true;
			}
		}
		return false;
	}

	void FContextEvaluator::GetStats(int32& OutNumPendingTasks, int32& OutNumRunningTasks, int32& OutNumCompletedTasks) const
	{
		// ugly but we need to make sure the stats are up to date 
		const_cast<FContextEvaluator*>(this)->ClearCompletedTasks();

		OutNumPendingTasks = PendingEvaluationEntries.Num();
		OutNumRunningTasks = RunningTasks.Num();
		OutNumCompletedTasks = CompletedTasks.Num();
	}

	void FContextEvaluator::Process()
	{
		int32 NumScheduleTasks = 0;
		for (auto Iter = PendingEvaluationEntries.CreateIterator(); Iter; ++Iter)
		{
			if (TryScheduleTask(Iter->Value))
			{
				NumScheduleTasks++;
				Iter.RemoveCurrent();
			}
		}

		if (NumScheduleTasks == 0)
		{
			UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::Process : No Task Scheduled NumPendingTasks=[%d]", PendingEvaluationEntries.Num());
			for (const TPair<FNodeId, FEvaluationEntry>& Entry : PendingEvaluationEntries)
			{
				UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::Process : \t -[%ls]", *Entry.Value.ToString());
			}
		}

		ClearCompletedTasks();
	}

	void FContextEvaluator::ClearCompletedTasks()
	{
		for (auto Iter = RunningTasks.CreateIterator(); Iter; ++Iter)
		{
			const FGraphEventRef& Task = (*Iter).Value;
			const FNodeId TaskId = (*Iter).Key;
			if (Task->IsCompleted())
			{
				CompletedTasks.Add(TaskId);
				Iter.RemoveCurrent();
			}
		}
	}

	void FContextEvaluator::ScheduleTask(const FEvaluationEntry& Entry)
	{
		if (TSharedPtr<const FDataflowNode> Node = Entry.WeakNode.Pin())
		{
			FGraphEventArray ExistingTasks;
			if (FGraphEventRef* ExistingTask = RunningTasks.Find(Entry.Id))
			{ 
				ExistingTasks.Add(*ExistingTask);
			}

			const bool bUseGameThread = ShouldRunOnGameThread(*Node);

			UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::ScheduleTask : [%ls] GameThread=[%d] previousTasks=[%d]",
				*Entry.ToString(),
				(int32)bUseGameThread,
				(int32)ExistingTasks.Num()
			);

			FContext* ContextPtr = &OwningContext;
			FGraphEventRef NewTask = FFunctionGraphTask::CreateAndDispatchWhenReady(
				[ContextPtr, Entry]
				{
					if (ContextPtr)
					{
						if (TSharedPtr<const FDataflowNode> Node = Entry.WeakNode.Pin())
						{
							Node->SetAsyncEvaluating(true);
							Node->EvaluateNode(*ContextPtr);
							Node->SetAsyncEvaluating(false);
							UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::EndTask : [%ls]", *Entry.ToString());
						}
					}
				},
				TStatId(),
				&ExistingTasks, /* prerequisites - make sure we wait on the previous one if any */
				bUseGameThread ? ENamedThreads::GameThread : ENamedThreads::AnyThread
			);

			auto OnFinishEvaluating = 
				[Evaluator = this, ContextPtr, OnPostEvaluation = Entry.OnPostEvaluation]()
				{
					Evaluator->Process();
					if (OnPostEvaluation.IsSet() && ContextPtr)
					{
						OnPostEvaluation(*ContextPtr);
					}
				};

			// handle post evaluation and run it on the game thread 
			NewTask = FFunctionGraphTask::CreateAndDispatchWhenReady(
				OnFinishEvaluating,
				TStatId(),
				NewTask, /* prerequisites */
				ENamedThreads::GameThread
			);

			RunningTasks.Add(Entry.Id, NewTask);
		}
	}

	FString FContextEvaluator::FEvaluationEntry::ToString() const
	{
		static FName UnknownName("-Unknown-");
		FName NodeName = UnknownName;
		if (TSharedPtr<const FDataflowNode> Node = WeakNode.Pin())
		{
			NodeName = Node->GetName();
		}
		return NodeName.ToString();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	float FThreadSafeProgressTracker::GetProgress() const
	{
		FScopeLock Lock(&ProgressLock);
		return Progress;
	}

	FText FThreadSafeProgressTracker::GetProgressMessage() const
	{
		FScopeLock Lock(&ProgressLock);
		return FText::Format(LOCTEXT("DirectEvaluator_ProgressMsg_Format", "Evaluating {0}"), FText::FromName(NodeName));
	}

	void FThreadSafeProgressTracker::SetProgress(float InCurrent, float InTotal, FName InNodeName)
	{
		FScopeLock Lock(&ProgressLock);
		Progress = (InTotal > 0)
			? FMath::Clamp(InCurrent / InTotal, 0.0f, 1.0f)
			: 0.0f;
		NodeName = InNodeName;
	}

	void FThreadSafeProgressTracker::SetProgress(float InProgress, FName InNodeName)
	{
		FScopeLock Lock(&ProgressLock);
		Progress = FMath::Clamp(InProgress, 0.0f, 1.0f);
		NodeName = InNodeName;
	}

	void FThreadSafeProgressTracker::AddProgress(float InProgressDelta, FName InNodeName)
	{
		FScopeLock Lock(&ProgressLock);
		Progress = FMath::Clamp(Progress + InProgressDelta, 0.0f, 1.0f);
		NodeName = InNodeName;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	FDataflowDirectEvaluator::FDataflowDirectEvaluator(FContext& InOwningContext)
		: OwningContext(InOwningContext)
	{
	}

	void FDataflowDirectEvaluator::Start(const FDataflowEvaluatorParameters& InParams)
	{
		bIsCanceled = false;
		bIsRunning = true;

		UE_LOGF(LogDataflowEvaluator, Verbose, "FDataflowDirectEvaluator::Start -- START");

		TSharedRef<const FGraph> DataflowGraph = InParams.CompiledGraph->GetSourceGraph();

		// Decide which nodes to evaluate.
		// - DemandPull path : cache-aware slice from the target (or every terminal when no target is set).
		// - Legacy path     : every compiled node in topological order; per-output cache checks inside
		//                     Node->EvaluateNode still no-op already-valid outputs but the per-node lookup
		//                     and progress update cost remains.
		TArray<int32> NodeIndicesToEvaluate;
		if (CVarUseDemandPullDirect.GetValueOnGameThread() != 0)
		{
			FDemandPullSlice Slice;
			const TConstArrayView<FGuid> TargetView = InParams.NodeId.IsValid()
				? MakeArrayView(&InParams.NodeId, 1)
				: TConstArrayView<FGuid>();
			ComputeDemandPullSlice(*InParams.CompiledGraph, OwningContext, TargetView, Slice);
			NodeIndicesToEvaluate = MoveTemp(Slice.OrderedNodeIndices);
		}
		else
		{
			const int32 NumNodes = InParams.CompiledGraph->GetNumNodes();
			NodeIndicesToEvaluate.Reserve(NumNodes);
			for (int32 NodeIndex = 0; NodeIndex < NumNodes; ++NodeIndex)
			{
				NodeIndicesToEvaluate.Add(NodeIndex);
			}
		}

		const int32 Total = NodeIndicesToEvaluate.Num();
		for (int32 i = 0; i < Total && !bIsCanceled; ++i)
		{
			const FGuid NodeId = InParams.CompiledGraph->GetNodeId(NodeIndicesToEvaluate[i]);
			if (TSharedPtr<const FDataflowNode> Node = DataflowGraph->FindBaseNode(NodeId))
			{
				ProgressTracker.SetProgress((float)(i + 1), (float)Total, Node->GetName());

				const double StartEvalTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

				if (Node->EvaluateNode(OwningContext))
				{
					if (Node->IsTerminal() && InParams.OnTerminalNodeEvaluated.IsSet())
					{
						InParams.OnTerminalNodeEvaluated(OwningContext, *Node);
					}
				}

				const double EndEvalTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

				UE_LOGF(LogDataflowEvaluator, Verbose, "FDataflowDirectEvaluator::Start --     Node [%ls] : [%.3f ms]", *Node->GetName().ToString(), (float)(EndEvalTimeMs - StartEvalTimeMs));
			}
		}

		bIsRunning = false;

		// No more nodes to process
		if (!bIsCanceled && InParams.OnPostEvaluation.IsSet())
		{
			InParams.OnPostEvaluation(OwningContext);
		}

		UE_LOGF(LogDataflowEvaluator, Verbose, "FDataflowDirectEvaluator::Start -- END");
	}

	void FDataflowDirectEvaluator::Cancel()
	{
		bIsCanceled = true;
	}

	bool FDataflowDirectEvaluator::IsRunning() const
	{
		return bIsRunning;
	}

	float FDataflowDirectEvaluator::GetProgress() const
	{
		return ProgressTracker.GetProgress();
	}

	FText FDataflowDirectEvaluator::GetProgressMessage() const
	{
		return ProgressTracker.GetProgressMessage();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	FDataflowTickEvaluator::FDataflowTickEvaluator(FContext& InOwningContext)
		: OwningContext(InOwningContext)
	{
	}

	FDataflowTickEvaluator::~FDataflowTickEvaluator()
	{
		FTSTicker::GetCoreTicker().RemoveTicker(TickDelegateHandle);
		TickDelegateHandle.Reset();
	}

	void FDataflowTickEvaluator::Reset()
	{
		check(IsInGameThread());

		CompiledGraph.Reset();
		DataflowGraph.Reset();
		NodeIndicesToEvaluate.Reset();
		NextNodeCursor = 0;
		NodeId = FGuid();
		PostEvaluationFunction.Reset();
		TerminalNodeEvaluatedFunction.Reset();
		bIsRunning = false;

		FTSTicker::GetCoreTicker().RemoveTicker(TickDelegateHandle);
		TickDelegateHandle.Reset();
	}

	void FDataflowTickEvaluator::Start(const FDataflowEvaluatorParameters& InParams)
	{
		Reset();

		DataflowGraph = InParams.CompiledGraph->GetSourceGraph();
		CompiledGraph = InParams.CompiledGraph;
		NodeId = InParams.NodeId;
		PostEvaluationFunction = InParams.OnPostEvaluation;
		TerminalNodeEvaluatedFunction = InParams.OnTerminalNodeEvaluated;
		NextNodeCursor = 0;

		// Compute the list of node indices to walk once, up front. See FDataflowDirectEvaluator::Start
		// for the rationale of the two code paths.
		if (CVarUseDemandPullTick.GetValueOnGameThread() != 0)
		{
			FDemandPullSlice Slice;
			const TConstArrayView<FGuid> TargetView = NodeId.IsValid()
				? MakeArrayView(&NodeId, 1)
				: TConstArrayView<FGuid>();
			ComputeDemandPullSlice(*CompiledGraph, OwningContext, TargetView, Slice);
			NodeIndicesToEvaluate = MoveTemp(Slice.OrderedNodeIndices);
		}
		else
		{
			const int32 NumNodes = CompiledGraph->GetNumNodes();
			NodeIndicesToEvaluate.Reset(NumNodes);
			for (int32 Index = 0; Index < NumNodes; ++Index)
			{
				NodeIndicesToEvaluate.Add(Index);
			}
		}

		TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FDataflowTickEvaluator::Tick));

		bIsCanceled = false;
		bIsRunning = true;
	}

	void FDataflowTickEvaluator::Cancel()
	{
		bIsCanceled = true;
	}

	bool FDataflowTickEvaluator::IsRunning() const
	{
		return bIsRunning;
	}

	float FDataflowTickEvaluator::GetProgress() const
	{
		return ProgressTracker.GetProgress();
	}

	FText FDataflowTickEvaluator::GetProgressMessage() const
	{
		return ProgressTracker.GetProgressMessage();
	}

	void FDataflowTickEvaluator::OnEvaluationCompleted()
	{
		// Save the function because reset reset it 
		// and we need to call reset first before calling the call to the post evaluation function 
		FOnPostEvaluationFunction FunctionToCall = PostEvaluationFunction;
		Reset();
		if (FunctionToCall.IsSet())
		{
			FunctionToCall(OwningContext);
		}		
	}

	void FDataflowTickEvaluator::OnEvaluationCanceled()
	{
		Reset();
	}

	bool FDataflowTickEvaluator::Tick(float DeltaTime)
	{
		constexpr double TickTimeBudget = 5.0;

		UE_LOGF(LogDataflowEvaluator, Verbose, "FDataflowTickEvaluator::Tick -- START");

		const double StartTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
		double CurrentTimeMs = StartTimeMs;

		const int32 Total = NodeIndicesToEvaluate.Num();
		while (!bIsCanceled && CompiledGraph && DataflowGraph)
		{
			if (NextNodeCursor >= Total)
			{
				OnEvaluationCompleted();
				break;
			}

			const FGuid NextNodeId = CompiledGraph->GetNodeId(NodeIndicesToEvaluate[NextNodeCursor]);
			if (TSharedPtr<const FDataflowNode> Node = DataflowGraph->FindBaseNode(NextNodeId))
			{
				ProgressTracker.SetProgress((float)(NextNodeCursor + 1), (float)Total, Node->GetName());

				const double StartEvalTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

				if (Node->EvaluateNode(OwningContext))
				{
					if (Node->IsTerminal() && TerminalNodeEvaluatedFunction.IsSet())
					{
						TerminalNodeEvaluatedFunction(OwningContext, *Node);
					}
				}

				const double EndEvalTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

				UE_LOGF(LogDataflowEvaluator, Verbose, "FDataflowTickEvaluator::Tick --     Node [%ls] : [%.3f ms]", *Node->GetName().ToString(), (float)(EndEvalTimeMs - StartEvalTimeMs));
			}
			NextNodeCursor++;

			// timeout check
			CurrentTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
			if ((CurrentTimeMs - StartTimeMs) > TickTimeBudget)
			{
				break;
			}
		}

		UE_LOGF(LogDataflowEvaluator, Verbose, "FDataflowTickEvaluator::Tick -- END : [%.3f ms]", (float)(CurrentTimeMs - StartTimeMs));

		if (bIsCanceled)
		{
			OnEvaluationCanceled();
			return false;
		}

		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////

	FDataflowTaskGraphEvaluator::FDataflowTaskGraphEvaluator(FContext& InOwningContext)
		: OwningContext(InOwningContext)
		, CancellationToken(MakeShared<UE::Tasks::FCancellationToken>())
		, ProgressTracker(MakeShared<FThreadSafeProgressTracker>())
	{
	}

	void FDataflowTaskGraphEvaluator::Start(const FDataflowEvaluatorParameters& InParams)
	{
		// build the taskgraph from the compiled graph
		const int32 NumTasks = InParams.CompiledGraph->GetNumTasks();

		CancellationToken = MakeShared<UE::Tasks::FCancellationToken>();
		TArray<FTask> AllTasks;
		AllTasks.SetNum(NumTasks);

		TSharedRef<const FGraph> DataflowGraph = InParams.CompiledGraph->GetSourceGraph();

		// Phase 1 (game-thread, sync): compute the demand-pull slice up front.
		// The mask drives a per-node filter inside the task-building loop below; the slice itself
		// never needs to survive into the launched task bodies (option (A) from the design review).
		const bool bUseDemandPull = (CVarUseDemandPullTaskGraph.GetValueOnGameThread() != 0);
		FDemandPullSlice Slice;
		if (bUseDemandPull)
		{
			const TConstArrayView<FGuid> TargetView = InParams.NodeId.IsValid()
				? MakeArrayView(&InParams.NodeId, 1)
				: TConstArrayView<FGuid>();
			ComputeDemandPullSlice(*InParams.CompiledGraph, OwningContext, TargetView, Slice);
		}

		// First pass: per compiled task, build the filtered NodeWeakPtrs list and remember its raw
		// ParentTaskIds for the launch pass below. Tasks whose filtered node list is empty are
		// skipped entirely (AllTasks[T] stays default-invalid). The existing prereq filter naturally
		// drops unlaunched parents because FTask::IsValid() returns false on default-constructed entries.
		struct FTaskBuild
		{
			int32 TaskIndex = INDEX_NONE;
			TArray<TWeakPtr<const FDataflowNode>> NodeWeakPtrs;
			TArray<int32> ParentTaskIds;
		};
		TArray<FTaskBuild> TasksToLaunch;
		TasksToLaunch.Reserve(NumTasks);

		TArray<FGuid> NodeIds;
		TArray<int32> ParentTaskIds;
		for (int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			InParams.CompiledGraph->GetTasksNodesAndDependencies(TaskIndex, NodeIds, ParentTaskIds);

			TArray<TWeakPtr<const FDataflowNode>> NodeWeakPtrs;
			NodeWeakPtrs.Reserve(NodeIds.Num());
			for (const FGuid& NodeId : NodeIds)
			{
				if (bUseDemandPull)
				{
					const int32 NodeIdx = InParams.CompiledGraph->FindNodeIndex(NodeId);
					if (NodeIdx == INDEX_NONE || !Slice.SliceMask[NodeIdx])
					{
						continue;
					}
				}
				if (TSharedPtr<const FDataflowNode> Node = DataflowGraph->FindBaseNode(NodeId))
				{
					NodeWeakPtrs.Add(Node);
				}
			}

			if (NodeWeakPtrs.IsEmpty())
			{
				continue;
			}

			TasksToLaunch.Add(FTaskBuild{
				.TaskIndex = TaskIndex,
				.NodeWeakPtrs = MoveTemp(NodeWeakPtrs),
				.ParentTaskIds = ParentTaskIds,
			});
		}

		// PerTaskProgress is computed against the count we'll actually launch so the progress bar
		// reaches 1.0 even when most of the graph is cached.
		const int32 NumTasksToLaunch = TasksToLaunch.Num();
		const float PerTaskProgress = (NumTasksToLaunch > 0) ? (1.f / (float)NumTasksToLaunch) : 0.f;

		UE_LOGF(LogDataflowEvaluator, Verbose,
			"FDataflowTaskGraphEvaluator::Start -- launching [%d / %d] tasks (DemandPull=%d)",
			NumTasksToLaunch, NumTasks, bUseDemandPull ? 1 : 0);

		// Second pass: actually launch.
		TArray<FTask> PreRequisites;
		for (FTaskBuild& Work : TasksToLaunch)
		{
			PreRequisites.Reset();
			for (int32 ParentTaskId : Work.ParentTaskIds)
			{
				if (AllTasks.IsValidIndex(ParentTaskId) && AllTasks[ParentTaskId].IsValid())
				{
					PreRequisites.Add(AllTasks[ParentTaskId]);
				}
			}
			// TODO : maybe we should have the context as a shared Ptr
			auto TaskBody =
				[NodeWeakPtrs = MoveTemp(Work.NodeWeakPtrs), CancellationToken = CancellationToken,
				PerTaskProgress, ProgressTracker = ProgressTracker, &OwningContext = OwningContext, OnTerminalNodeEvaluated = InParams.OnTerminalNodeEvaluated]()
				{
					UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::Task -- START");

					const int32 NumNodes = NodeWeakPtrs.Num();
					const float PerNodeProgress = (NumNodes > 0) ? (PerTaskProgress / (float)NumNodes) : 0;
					for (TWeakPtr<const FDataflowNode> NodeWeakPtr : NodeWeakPtrs)
					{
						if (TSharedPtr<const FDataflowNode> NodePtr = NodeWeakPtr.Pin())
						{
							UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::Task -- EVALNODE [%ls]", *NodePtr->GetName().ToString());
							ProgressTracker->AddProgress(PerNodeProgress, NodePtr->GetName());

							if (NodePtr->EvaluateNode(OwningContext))
							{
								// todo : may need to ensure this is running on the game thread once we can execute on aritrary threads
								if (NodePtr->IsTerminal() && OnTerminalNodeEvaluated.IsSet())
								{
									OnTerminalNodeEvaluated(OwningContext, *NodePtr);
								}
							}
						}
						if (CancellationToken->IsCanceled())
						{
							UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::Task -- CANCELLED");
							break;
						}
					}
					UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::Task -- END");
				};

			AllTasks[Work.TaskIndex] = Dataflow::FSystem::Get().TaskGraph.LaunchGameThreadTask(
				TEXT("DataflowEvaluationTask"), MoveTemp(TaskBody), PreRequisites, CancellationToken);
		}

		// Defensive: the completion task should only wait on actually-launched prereqs. The previous
		// code passed the full AllTasks (with potentially-invalid entries); whether that worked was
		// luck of taskgraph internals. With demand-pull slicing producing real invalid entries, filter.
		TArray<FTask> CompletionPrereqs;
		CompletionPrereqs.Reserve(NumTasksToLaunch);
		for (const FTask& Task : AllTasks)
		{
			if (Task.IsValid())
			{
				CompletionPrereqs.Add(Task);
			}
		}

		auto CompletionTaskBody = [OnPostEvaluation = InParams.OnPostEvaluation, &OwningContext = OwningContext]()
			{
				UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::PostEvalTask -- START");
				if (OnPostEvaluation.IsSet())
				{
					UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::PostEvalTask -- CALL");
					OnPostEvaluation(OwningContext);
				}
				UE_LOGF(LogDataflowEvaluator, Verbose, "FContextEvaluator::PostEvalTask -- END");
			};
		CompletionTask = MakeShared<FTask>(Dataflow::FSystem::Get().TaskGraph.LaunchGameThreadTask(
			TEXT("DataflowCompletionTask"), MoveTemp(CompletionTaskBody), CompletionPrereqs));
	}

	void FDataflowTaskGraphEvaluator::Cancel()
	{
		CancellationToken->Cancel();
	}

	bool FDataflowTaskGraphEvaluator::IsRunning() const
	{
		return CompletionTask && !CompletionTask->IsCompleted();
	}

	float FDataflowTaskGraphEvaluator::GetProgress() const
	{
		return ProgressTracker->GetProgress();
	}

	FText FDataflowTaskGraphEvaluator::GetProgressMessage() const
	{
		return ProgressTracker->GetProgressMessage();
	}
};

#undef LOCTEXT_NAMESPACE
