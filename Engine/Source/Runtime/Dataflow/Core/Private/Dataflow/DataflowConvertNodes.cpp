// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowConvertNodes.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"

#include "MathUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowConvertNodes)

namespace UE::Dataflow
{
	void RegisterDataflowConvertNodes()
	{
		// Convert
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertNumericTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertVectorTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertStringTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertBoolTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertTransformTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertStringConvertibleTypesDataflowNode);
		// Needs to be fixed
//		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertUObjectConvertibleTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertSelectionTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertSelectionTypesToIndexArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertIndexToSelectionTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertIndexArrayToSelectionTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertVectorArrayTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertNumericArrayTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertStringArrayTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertBoolArrayTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertTransformArrayTypesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertRotationDataflowNode);

		// Deprecated
	}
}

//-----------------------------------------------------------------------------------------------

FConvertNumericTypesDataflowNode::FConvertNumericTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertNumericTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const double InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertVectorTypesDataflowNode::FConvertVectorTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertVectorTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const FVector4 InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertStringTypesDataflowNode::FConvertStringTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertStringTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const FString InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertBoolTypesDataflowNode::FConvertBoolTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertBoolTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const bool InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertTransformTypesDataflowNode::FConvertTransformTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertTransformTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const FTransform InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertStringConvertibleTypesDataflowNode::FConvertStringConvertibleTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertStringConvertibleTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const FString InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertUObjectConvertibleTypesDataflowNode::FConvertUObjectConvertibleTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertUObjectConvertibleTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const TObjectPtr<UObject> InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertSelectionTypesDataflowNode::FConvertSelectionTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Out);
}

void FConvertSelectionTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FDataflowSelection InValue = GetValue(Context, &In);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

		const FDataflowInput* SelectionInput = FindInput(&In);
		if (SelectionInput)
		{
			if (SelectionInput->IsType<FDataflowTransformSelection>())
			{
				if (Output && Output->IsType<FDataflowFaceSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertTransformSelectionToFaceSelection(InValue.AsArray());

					FDataflowFaceSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowVertexSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertTransformSelectionToVertexSelection(InValue.AsArray());

					FDataflowVertexSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowGeometrySelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertTransformSelectionToGeometrySelection(InValue.AsArray());

					FDataflowGeometrySelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowCurveSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertTransformSelectionToCurveSelection(InValue.AsArray());

					FDataflowCurveSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
			}
			else if (SelectionInput->IsType<FDataflowFaceSelection>())
			{
				if (Output && Output->IsType<FDataflowTransformSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertFaceSelectionToTransformSelection(InValue.AsArray(), bAllElementsMustBeSelected);

					FDataflowTransformSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowVertexSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertFaceSelectionToVertexSelection(InValue.AsArray());

					FDataflowVertexSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowGeometrySelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertFaceSelectionToGeometrySelection(InValue.AsArray(), bAllElementsMustBeSelected);

					FDataflowGeometrySelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowCurveSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertFaceSelectionToCurveSelection(InValue.AsArray(), bAllElementsMustBeSelected);

					FDataflowCurveSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
			}
			else if (SelectionInput->IsType<FDataflowVertexSelection>())
			{
				if (Output && Output->IsType<FDataflowTransformSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertVertexSelectionToTransformSelection(InValue.AsArray(), bAllElementsMustBeSelected);

					FDataflowTransformSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowFaceSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertVertexSelectionToFaceSelection(InValue.AsArray(), bAllElementsMustBeSelected);

					FDataflowFaceSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowGeometrySelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertVertexSelectionToGeometrySelection(InValue.AsArray(), bAllElementsMustBeSelected);

					FDataflowGeometrySelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowCurveSelection>())
				{
					TBitArray<> SelectedCurves;
					TransformSelectionFacade.ConvertVertexSelectionToCurveSelection(InValue.GetBitArray(), bAllElementsMustBeSelected, SelectedCurves);

					FDataflowCurveSelection NewSelection;
					NewSelection.SetFromBitArray(MoveTemp(SelectedCurves));

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
			}
			else if (SelectionInput->IsType<FDataflowGeometrySelection>())
			{
				if (Output && Output->IsType<FDataflowTransformSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertGeometrySelectionToTransformSelection(InValue.AsArray());

					FDataflowTransformSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowFaceSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertGeometrySelectionToFaceSelection(InValue.AsArray());

					FDataflowFaceSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowVertexSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertGeometrySelectionToVertexSelection(InValue.AsArray());

					FDataflowVertexSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowCurveSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertGeometrySelectionToCurveSelection(InValue.AsArray());

					FDataflowCurveSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
			}
			else if (SelectionInput->IsType<FDataflowCurveSelection>())
			{
				if (Output && Output->IsType<FDataflowTransformSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertCurveSelectionToTransformSelection(InValue.AsArray(), bAllElementsMustBeSelected);

					FDataflowTransformSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowFaceSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertCurveSelectionToFaceSelection(InValue.AsArray());

					FDataflowFaceSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowVertexSelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertCurveSelectionToVertexSelection(InValue.AsArray());

					FDataflowVertexSelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
				else if (Output && Output->IsType<FDataflowGeometrySelection>())
				{
					const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertCurveSelectionToGeometrySelection(InValue.AsArray(), bAllElementsMustBeSelected);

					FDataflowGeometrySelection NewSelection;
					NewSelection.InitFromArray(InCollection, SelectionArr);

					SetValue(Context, MoveTemp(NewSelection), &Out);
				}
			}
		}
	}
	else if (Output->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

//-----------------------------------------------------------------------------------------------

namespace UE::Private::DataflowConvertNodesLocal
{
	template<typename DataflowSelectionType>
	bool SetMatchingSelectionHelper(const TArray<int32>& SelectionArr,
		const FManagedArrayCollection& InCollection, UE::Dataflow::FContext& Context,
		const FDataflowOutput* Output, const FDataflowSelectionTypes* OutMemberPtr, const FDataflowNode* ThisNode)
	{
		if (Output && Output->IsType<DataflowSelectionType>())
		{
			DataflowSelectionType NewSelection;
			NewSelection.InitFromArray(InCollection, SelectionArr);
			ThisNode->SetValue(Context, MoveTemp(NewSelection), OutMemberPtr);
			return true;
		}
		return false;
	}

	bool SetMatchingSelection(const TArray<int32>& SelectionArr,
		const FManagedArrayCollection& InCollection, UE::Dataflow::FContext& Context,
		const FDataflowOutput* Output, const FDataflowSelectionTypes* OutMemberPtr, const FDataflowNode* ThisNode)
	{
		return SetMatchingSelectionHelper<FDataflowTransformSelection>(SelectionArr, InCollection, Context, Output, OutMemberPtr, ThisNode)
			|| SetMatchingSelectionHelper<FDataflowGeometrySelection>(SelectionArr, InCollection, Context, Output, OutMemberPtr, ThisNode)
			|| SetMatchingSelectionHelper<FDataflowFaceSelection>(SelectionArr, InCollection, Context, Output, OutMemberPtr, ThisNode)
			|| SetMatchingSelectionHelper<FDataflowVertexSelection>(SelectionArr, InCollection, Context, Output, OutMemberPtr, ThisNode) 
			|| SetMatchingSelectionHelper<FDataflowCurveSelection>(SelectionArr, InCollection, Context, Output, OutMemberPtr, ThisNode);
	}
}

FConvertIndexArrayToSelectionTypesDataflowNode::FConvertIndexArrayToSelectionTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Out);
}


void FConvertIndexArrayToSelectionTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const bool bHasValidCollection = IsConnected(&Collection);
		if (!bHasValidCollection)
		{
			Context.Warning(TEXT("Indices cannot be converted to a valid selection without a connected Collection"), this);
		}
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		UE::Private::DataflowConvertNodesLocal::SetMatchingSelection(
			bHasValidCollection ? GetValue(Context, &In) : TArray<int32>(), InCollection, Context, Output, &Out, this);
	}
	else if (Output->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

FConvertIndexToSelectionTypesDataflowNode::FConvertIndexToSelectionTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Out);
}


void FConvertIndexToSelectionTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		TArray<int32> InValueArr;
		const bool bHasValidCollection = IsConnected(&Collection);
		if (bHasValidCollection)
		{
			int32 InValue = GetValue(Context, &In);
			InValueArr.Add(InValue);
		}
		else
		{
			Context.Warning(TEXT("Index cannot be converted to a valid selection without a connected Collection"), this);
		}
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		UE::Private::DataflowConvertNodesLocal::SetMatchingSelection(InValueArr, InCollection, Context, Output, &Out, this);
	}
	else if (Output->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


//-----------------------------------------------------------------------------------------------

FConvertSelectionTypesToIndexArrayDataflowNode::FConvertSelectionTypesToIndexArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertSelectionTypesToIndexArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const FDataflowSelection InValue = GetValue(Context, &In);
		TArray<int32> NewSelection;
		InValue.AsArray(NewSelection);
		SetValue(Context, MoveTemp(NewSelection), &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertVectorArrayTypesDataflowNode::FConvertVectorArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertVectorArrayTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const TArray<FVector4> InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertNumericArrayTypesDataflowNode::FConvertNumericArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertNumericArrayTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const TArray<double> InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertStringArrayTypesDataflowNode::FConvertStringArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertStringArrayTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const TArray<FString> InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertBoolArrayTypesDataflowNode::FConvertBoolArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertBoolArrayTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const TArray<bool> InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertTransformArrayTypesDataflowNode::FConvertTransformArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertTransformArrayTypesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const TArray<FTransform> InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------

FConvertRotationDataflowNode::FConvertRotationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&In);
	RegisterOutputConnection(&Out);
}

void FConvertRotationDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
	if (Output->IsA(&Out))
	{
		const FRotator InValue = GetValue(Context, &In);
		SetValue(Context, InValue, &Out);
	}
}

//-----------------------------------------------------------------------------------------------




