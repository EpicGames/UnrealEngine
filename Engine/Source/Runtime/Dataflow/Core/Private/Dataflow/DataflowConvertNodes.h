// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "ChaosLog.h"
//#include "CoreMinimal.h"
//#include "Dataflow/DataflowConnection.h"
//#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"

#include "DataflowConvertNodes.generated.h"

//--------------------------------------------------------------------------
//
// Convert nodes
//
//--------------------------------------------------------------------------

#define DATAFLOW_CONVERT_NODES_CATEGORY "Convert"

/**
* Convert Numeric types
* (double, float, int64, uint64, int32, uint32, int16, uint16, int8, uint8)
*/
USTRUCT()
struct FConvertNumericTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertNumericTypesDataflowNode, "ConvertNumericTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowNumericTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertNumericTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Vector types
* (2D, 3D and 4D vector, single and double precision)
*/
USTRUCT()
struct FConvertVectorTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertVectorTypesDataflowNode, "ConvertVectorTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowVectorTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertVectorTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert String types 
* (FString or FName or FText)
*/
USTRUCT()
struct FConvertStringTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertStringTypesDataflowNode, "ConvertStringTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowStringTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowStringTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertStringTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Bool types
*/
USTRUCT()
struct FConvertBoolTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertBoolTypesDataflowNode, "ConvertBoolTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowBoolTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowBoolTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertBoolTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Transform types
*/
USTRUCT()
struct FConvertTransformTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertTransformTypesDataflowNode, "ConvertTransformTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowTransformTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowTransformTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertTransformTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert String convertible types
* (String types, Numeric types, Vector types and Booleans)
*/
USTRUCT()
struct FConvertStringConvertibleTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertStringConvertibleTypesDataflowNode, "ConvertStringConvertibleTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowStringConvertibleTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowStringConvertibleTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertStringConvertibleTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert UObject types
*/
USTRUCT()
struct FConvertUObjectConvertibleTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertUObjectConvertibleTypesDataflowNode, "ConvertUObjectConvertibleTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowUObjectConvertibleTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowUObjectConvertibleTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertUObjectConvertibleTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Selection types
*/
USTRUCT()
struct FConvertSelectionTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertSelectionTypesDataflowNode, "ConvertSelectionTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowSelectionTypes In;

	/** If true then for converting vertex/face selection to transform/geometry selection all vertex/face must be selected for selecting the associated transform/geometry 
	or going from vertex to face selection all vertices must be selected to select the face 
	*/
	UPROPERTY(EditAnywhere, Category = "Selection")
	bool bAllElementsMustBeSelected = false;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertSelectionTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Selection to Index Array
*/
USTRUCT()
struct FConvertSelectionTypesToIndexArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertSelectionTypesToIndexArrayDataflowNode, "ConvertSelectionToIndexArray", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:

	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowSelectionTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertSelectionTypesToIndexArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Index to Selection
*/
USTRUCT()
struct FConvertIndexToSelectionTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertIndexToSelectionTypesDataflowNode, "ConvertIndexToSelection", DATAFLOW_CONVERT_NODES_CATEGORY, "")

public:
	FConvertIndexToSelectionTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
private:

	/** Collection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	int32 In = 0;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};

/**
* Convert Index Array to Selection
*/
USTRUCT()
struct FConvertIndexArrayToSelectionTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertIndexArrayToSelectionTypesDataflowNode, "ConvertIndexArrayToSelection", DATAFLOW_CONVERT_NODES_CATEGORY, "")
	
public:
	FConvertIndexArrayToSelectionTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** Collection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<int32> In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};


/**
* Convert Vector array types
*/
USTRUCT()
struct FConvertVectorArrayTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertVectorArrayTypesDataflowNode, "ConvertVectorArrayTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowVectorArrayTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorArrayTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertVectorArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Numeric array types
*/
USTRUCT()
struct FConvertNumericArrayTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertNumericArrayTypesDataflowNode, "ConvertNumericArrayTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowNumericArrayTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericArrayTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertNumericArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert String array types
*/
USTRUCT()
struct FConvertStringArrayTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertStringArrayTypesDataflowNode, "ConvertStringArrayTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowStringArrayTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowStringArrayTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertStringArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Bool array types
*/
USTRUCT()
struct FConvertBoolArrayTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertBoolArrayTypesDataflowNode, "ConvertBoolArrayTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowBoolArrayTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowBoolArrayTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertBoolArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Transform array types
*/
USTRUCT()
struct FConvertTransformArrayTypesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertTransformArrayTypesDataflowNode, "ConvertTransformArrayTypes", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowTransformArrayTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowTransformArrayTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertTransformArrayTypesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Convert Rotation
* (FQuat, FRotator, FVector)
*/
USTRUCT()
struct FConvertRotationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertRotationDataflowNode, "ConvertRotation", DATAFLOW_CONVERT_NODES_CATEGORY, "")

private:
	/** Input value */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowRotationTypes In;

	/** Output value */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowRotationTypes Out;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

public:
	FConvertRotationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void RegisterDataflowConvertNodes();
}

