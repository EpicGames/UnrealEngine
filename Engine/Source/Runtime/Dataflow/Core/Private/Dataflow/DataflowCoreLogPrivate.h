// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

// You should place include statements to your module's private header files here.  You only need to
// add includes for headers that are used in most of your module's source files though.


DECLARE_STATS_GROUP(TEXT("Dataflow"), STATGROUP_Dataflow, STATCAT_Advanced);
