// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCoreModule.h"

#include "CoreMinimal.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "DataflowCore"

void IDataflowCoreModule::StartupModule()
{/*Never Called ?*/}

void IDataflowCoreModule::ShutdownModule()
{/*Never Called ?*/ }

IMPLEMENT_MODULE(IDataflowCoreModule, DataflowCore)

#undef LOCTEXT_NAMESPACE
