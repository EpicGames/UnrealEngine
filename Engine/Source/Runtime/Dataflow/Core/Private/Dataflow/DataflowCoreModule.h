// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


/**
 * The public interface to this module
 */
class IDataflowCoreModule : public IModuleInterface
{

public:

	// IModuleInterface interface

	DATAFLOWCORE_API virtual void StartupModule() override;

	DATAFLOWCORE_API virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

};
