// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCoreNodeAndPinTypeColors.h"
#include "Dataflow/DataflowNode.h"
#include "Math/Color.h" 
#include "Dataflow/DataflowNodeColorsRegistry.h"

namespace UE::Dataflow
{
	void RegisterDataflowCoreColors()
	{
		static const FLinearColor CDefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);
		static const float CDefaultWireThickness = 1.5f;

		// Node colors
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Math", FLinearColor(0.453125f, 0.453125f, 0.453125f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Image", FLinearColor(0.f, 0.4f, 0.8f), CDefaultNodeBodyTintColor);

		// PinType colors
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("bool", FLinearColor(0.300000f, 0.0f, 0.0f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("uint8", FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("uint16", FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("uint32", FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("uint64", FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("int8", FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("int16", FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("int32", FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("int64", FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("float", FLinearColor(0.357667f, 1.0f, 0.060000f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("double", FLinearColor(0.039216f, 0.666667f, 0.0f, 1.0f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FName", FLinearColor(0.607717f, 0.224984f, 1.0f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FString", FLinearColor(1.0f, 0.0f, 0.660537f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FText", FLinearColor(0.8f, 0.2f, 0.4f, 1.0f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FVector2D", FLinearColor(1.0f, 0.591255f, 0.016512f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FVector", FLinearColor(1.0f, 0.591255f, 0.016512f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FVector4", FLinearColor(1.0f, 0.591255f, 0.016512f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FVector2f", FLinearColor(1.0f, 0.591255f, 0.016512f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FVector3f", FLinearColor(1.0f, 0.591255f, 0.016512f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FVector4f", FLinearColor(1.0f, 0.591255f, 0.016512f, 1.0f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FQuat", FLinearColor(1.f, 0.591255f, 0.016512f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FQuat4f", FLinearColor(1.f, 0.591255f, 0.016512f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FLinearColor", FLinearColor(1.f, 0.591255f, 0.016512f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FIntPoint", FLinearColor(1.f, 0.591255f, 0.016512f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FIntVector3", FLinearColor(1.f, 0.591255f, 0.016512f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FIntVector4", FLinearColor(1.f, 0.591255f, 0.016512f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FRotator", FLinearColor(0.353393f, 0.454175f, 1.0f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FTransform", FLinearColor(1.0f, 0.172585f, 0.0f, 1.0f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowTransformSelection", FLinearColor(0.83077f, 0.361307f, 0.003035f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowVertexSelection", FLinearColor(0.83077f, 0.361307f, 0.003035f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowFaceSelection", FLinearColor(0.83077f, 0.361307f, 0.003035f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowGeometrySelection", FLinearColor(0.83077f, 0.361307f, 0.003035f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowCurvesSelection", FLinearColor(0.83077f, 0.361307f, 0.003035f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowMaterialSelection", FLinearColor(0.3f, 0.3f, 0.3f, 1.0f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowAnyType", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowAllTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowNumericTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowVectorTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowStringTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowBoolTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowTransformTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowStringConvertibleTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowUObjectConvertibleTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowSelectionTypes", FLinearColor(0.83077f, 0.361307f, 0.003035f, 1.0f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowVectorArrayTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowNumericArrayTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowStringArrayTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowBoolArrayTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowTransformArrayTypes", FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f), CDefaultWireThickness);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowImage", FLinearColor(0.382609f, 0.047687f, 0.047687f, 1.0f), CDefaultWireThickness);
	}
}
