// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCoreNodes.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowNodeFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowCoreNodes)

namespace UE::Dataflow
{
	void RegisterCoreNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowReRouteNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowBranchNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSelectNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowPrintNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowForceDependencyNode);
	}
}

FDataflowReRouteNode::FDataflowReRouteNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid)
	: Super(Param, InGuid)
{
	static const FName MainTypeGroup("Main");

	RegisterInputConnection(&Value)
		.SetTypeDependencyGroup(MainTypeGroup);
	RegisterOutputConnection(&Value)
		.SetPassthroughInput(&Value)
		.SetTypeDependencyGroup(MainTypeGroup);
}

void FDataflowReRouteNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ForwardInput(Context, &Value, &Value);
}

FDataflowBranchNode::FDataflowBranchNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid)
	: Super(Param, InGuid)
{
	static const FName MainTypeGroup("Main");

	RegisterInputConnection(&TrueValue)
		.SetTypeDependencyGroup(MainTypeGroup);
	RegisterInputConnection(&FalseValue)
		.SetTypeDependencyGroup(MainTypeGroup);
	RegisterInputConnection(&bCondition);
	RegisterOutputConnection(&Result)
		.SetTypeDependencyGroup(MainTypeGroup);
}

void FDataflowBranchNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Result))
	{
		const bool InCondition = GetValue<bool>(Context, &bCondition);
		const void* SelectedInputReference = InCondition ? &TrueValue : &FalseValue;
		if (IsConnected(SelectedInputReference))
		{
			ForwardInput(Context, SelectedInputReference, &Result);
		}
		else
		{
			// set a null value so that the connected system are getting a default value
			Out->SetNullValue(Context);
		}
	}
}

const FName FDataflowSelectNode::MainTypeGroup = "Main";

FDataflowSelectNode::FDataflowSelectNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid)
	: Super(Param, InGuid)
{
	// Add two sets of pins to start.
	RegisterInputConnection(&SelectedIndex);
	for (int32 Index = 0; Index < NumInitialInputs; ++Index)
	{
		AddPins();
	}
	RegisterOutputConnection(&Result)
		.SetPassthroughInput(GetConnectionReference(0))
		.SetTypeDependencyGroup(MainTypeGroup);
	check(NumRequiredDataflowInputs + NumInitialInputs == GetNumInputs()); // Update NumRequiredDataflowInputs when adding more inputs. This is used by Serialize
}

void FDataflowSelectNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Result))
	{
		const int32 InSelectedIndex = GetValue<int32>(Context, &SelectedIndex);
		if (Inputs.IsValidIndex(InSelectedIndex))
		{
			const UE::Dataflow::TConnectionReference<FDataflowAnyType> SelectedInputReference = GetConnectionReference(InSelectedIndex);
			if (IsConnected(SelectedInputReference))
			{
				ForwardInput(Context, SelectedInputReference, &Result);
			}
			else
			{
				// set a null value so that the connected system are getting a default value
				Out->SetNullValue(Context);
			}
		}
	}
}

TArray<UE::Dataflow::FPin> FDataflowSelectNode::AddPins()
{
	const int32 Index = Inputs.AddDefaulted();
	FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	if (Index > 0)
	{
		// Set concrete type the same as Index0.
		FDataflowInput* const Input0 = FindInput(GetConnectionReference(0));
		check(Input0);
		SetConnectionConcreteType(&Input, Input0->GetType(), MainTypeGroup);
	}
	else
	{
		Input.SetTypeDependencyGroup(MainTypeGroup);
	}
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FDataflowSelectNode::GetPinsToRemove() const
{
	const int32 Index = Inputs.Num() - 1;
	check(Inputs.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FDataflowSelectNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = Inputs.Num() - 1;
	check(Inputs.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	Inputs.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FDataflowSelectNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		check(Inputs.Num() >= NumInitialInputs);
		for (int32 Index = 0; Index < NumInitialInputs; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialInputs; Index < Inputs.Num(); ++Index)
		{
			FDataflowInput& Input = FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
			// reset the type to allow the type group to be properly set as well 
			SetConnectionConcreteType(&Input, Input.GetType(), MainTypeGroup);
		}

		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs() - NumRequiredDataflowInputs;
			const int32 OrigNumInputs = Inputs.Num();
			if (OrigNumRegisteredInputs > OrigNumInputs)
			{
				// Inputs have been removed.
				// Temporarily expand Inputs so we can get connection references.
				Inputs.SetNum(OrigNumRegisteredInputs);
				for (int32 Index = OrigNumInputs; Index < Inputs.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				Inputs.SetNum(OrigNumInputs);
			}
		}
		else
		{
			// Index + all Inputs
			ensureAlways(Inputs.Num() + NumRequiredDataflowInputs == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<FDataflowAnyType> FDataflowSelectNode::GetConnectionReference(int32 Index) const
{
	return { &Inputs[Index], Index, &Inputs };
}

FDataflowPrintNode::FDataflowPrintNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName TypeDependencyGroup("Main");

	RegisterInputConnection(&Value)
		.SetTypeDependencyGroup(TypeDependencyGroup);

	RegisterOutputConnection(&Value, &Value)
		.SetTypeDependencyGroup(TypeDependencyGroup);
}

void FDataflowPrintNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Value))
	{
		SafeForwardInput(Context, &Value, &Value);
	}

	const FString InValue = "Result=" + GetValue(Context, &Value);
	Context.Info(InValue, this, nullptr);
}

FDataflowForceDependencyNode::FDataflowForceDependencyNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&DependentValue);
	RegisterInputConnection(&Value);
	RegisterOutputConnection(&Value, &Value);
}

void FDataflowForceDependencyNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Value))
	{
		FindInput(&DependentValue)->PullValue(Context);
		ForwardInput(Context, &Value, &Value);
	}
}
