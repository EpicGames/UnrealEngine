// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowDemandPullSlice.h"

#include "Dataflow/DataflowCompiledGraph.h"
#include "Dataflow/DataflowGraph.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowNode.h"

DEFINE_LOG_CATEGORY_STATIC(LogDataflowDemandPull, Log, All);

namespace UE::Dataflow
{
	void ComputeDemandPullSlice(
		const FCompiledGraph& CompiledGraph,
		FContext& Context,
		TConstArrayView<FGuid> TargetNodeIds,
		FDemandPullSlice& OutSlice)
	{
		// Slice walker reads/writes context cache state and the source FGraph's connection topology;
		// running concurrently with a TaskGraph evaluation on the same context is unsafe. The two existing
		// callers (FDataflowDirectEvaluator and FDataflowTickEvaluator) are game-thread only, so this matches.
		check(IsInGameThread());

		const int32 NumNodes = CompiledGraph.GetNumNodes();
		OutSlice.SliceMask.Init(false, NumNodes);
		OutSlice.OrderedNodeIndices.Reset();

		if (NumNodes == 0)
		{
			return;
		}

		TSharedRef<const FGraph> SourceGraph = CompiledGraph.GetSourceGraph();

		TArray<int32> Queue;

		auto TrySeed = [&OutSlice, &Queue](int32 NodeIndex)
		{
			if (OutSlice.SliceMask.IsValidIndex(NodeIndex) && !OutSlice.SliceMask[NodeIndex])
			{
				OutSlice.SliceMask[NodeIndex] = true;
				Queue.Add(NodeIndex);
			}
		};

		if (TargetNodeIds.IsEmpty())
		{
			for (int32 TerminalIndex : CompiledGraph.GetTerminalNodeIndices())
			{
				TrySeed(TerminalIndex);
			}
		}
		else
		{
			for (const FGuid& TargetId : TargetNodeIds)
			{
				const int32 Index = CompiledGraph.FindNodeIndex(TargetId);
				if (Index != INDEX_NONE)
				{
					TrySeed(Index);
				}
				else
				{
					UE_LOGF(LogDataflowDemandPull, Verbose,
						"ComputeDemandPullSlice: target node id [%ls] is not in the compiled graph",
						*TargetId.ToString());
				}
			}
		}

		// BFS upstream from seeds. Slice walker uses the source FGraph's input/output topology
		// (rather than FCompiledGraph::GetUpstreamNodeIndices) because it needs per-input freshness:
		// we want to skip an upstream node only if every output we read from it is already cached.
		// The SliceMask doubles as the visited-set guard so the walk is O(N + E) regardless of any
		// pathological topology and never revisits a node. Evaluation-time cycle detection remains
		// the responsibility of FContextScopedCallstack inside FDataflowOutput::EvaluateImpl.
		while (!Queue.IsEmpty())
		{
			const int32 NodeIndex = Queue.Pop(EAllowShrinking::No);
			const FGuid NodeId = CompiledGraph.GetNodeId(NodeIndex);
			TSharedPtr<const FDataflowNode> Node = SourceGraph->FindBaseNode(NodeId);
			if (!Node)
			{
				continue;
			}

			for (const FDataflowInput* Input : Node->GetInputs())
			{
				if (!Input || !Input->IsConnected())
				{
					continue;
				}
				const FDataflowOutput* UpstreamOutput = Input->GetConnection();
				if (!UpstreamOutput)
				{
					continue;
				}
				// HasValidData covers both the frozen-output case and the cache-fresh case
				// (DataflowInputOutput.cpp:252-263). Disabled / pass-through nodes are handled
				// transparently: their output's cache becomes valid after one passthrough write,
				// so subsequent slice walks naturally prune at them.
				if (UpstreamOutput->HasValidData(Context))
				{
					continue;
				}
				const FDataflowNode* UpstreamNode = UpstreamOutput->GetOwningNode();
				if (!UpstreamNode)
				{
					continue;
				}
				const int32 UpstreamIndex = CompiledGraph.FindNodeIndex(UpstreamNode->GetGuid());
				if (UpstreamIndex != INDEX_NONE && !OutSlice.SliceMask[UpstreamIndex])
				{
					OutSlice.SliceMask[UpstreamIndex] = true;
					Queue.Add(UpstreamIndex);
				}
			}
		}

		// Single forward sweep over the topo-ordered Nodes[] to materialise the execution order.
		OutSlice.OrderedNodeIndices.Reserve(NumNodes);
		for (int32 i = 0; i < NumNodes; ++i)
		{
			if (OutSlice.SliceMask[i])
			{
				OutSlice.OrderedNodeIndices.Add(i);
			}
		}

		UE_LOGF(LogDataflowDemandPull, Verbose,
			"ComputeDemandPullSlice: slice=[%d / %d] nodes",
			OutSlice.OrderedNodeIndices.Num(), NumNodes);
	}
}
