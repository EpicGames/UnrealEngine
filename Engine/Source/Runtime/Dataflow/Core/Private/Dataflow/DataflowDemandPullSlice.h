// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "Misc/Guid.h"

namespace UE::Dataflow
{
	struct FCompiledGraph;
	class FContext;

	/**
	* Result of a demand-pull slice computation.
	* SliceMask          : per-node bit, sized to CompiledGraph.GetNumNodes(); true if the node needs to evaluate.
	*                      Sync evaluators only need OrderedNodeIndices, but the async/TaskGraph variant
	*                      uses the mask for O(1) "is upstream node N in the slice?" prerequisite filtering.
	* OrderedNodeIndices : slice node indices in the compiled graph's topological order. Subset of [0, NumNodes).
	*/
	struct FDemandPullSlice
	{
		TBitArray<> SliceMask;
		TArray<int32> OrderedNodeIndices;
	};

	/**
	* Compute the minimal upstream slice required to bring a set of target nodes up to date.
	*
	* Walks the source FGraph's input/output topology starting from each target, pruning at any
	* upstream output whose cache is already valid (FDataflowOutput::HasValidData) or frozen.
	* The resulting slice is then sweep-ordered against FCompiledGraph::Nodes (which is itself
	* topologically ordered, sources-first).
	*
	* TargetNodeIds == empty => seed from every terminal in the compiled graph (full evaluation
	* with cache-aware pruning, equivalent to today's left-to-right behaviour but skipping cached nodes).
	*
	* Game-thread only. Must not overlap with a FDataflowTaskGraphEvaluator run on the same FContext.
	*/
	void ComputeDemandPullSlice(
		const FCompiledGraph& CompiledGraph,
		FContext& Context,
		TConstArrayView<FGuid> TargetNodeIds,
		FDemandPullSlice& OutSlice);
}
