// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowDynamicMeshDebugDrawMesh.h"
#include "DynamicMesh/DynamicMesh3.h"

FDynamicMeshDebugDrawMesh::FDynamicMeshDebugDrawMesh(const class UE::Geometry::FDynamicMesh3* DynamicMesh) :
	DynamicMesh(DynamicMesh)
{
}

int32 FDynamicMeshDebugDrawMesh::GetMaxVertexIndex() const
{
	if (DynamicMesh)
	{
		return DynamicMesh->MaxVertexID();
	}
	return 0;
}

bool FDynamicMeshDebugDrawMesh::IsValidVertex(int32 VertexIndex) const
{
	return (DynamicMesh && DynamicMesh->IsVertex(VertexIndex));
}

FVector FDynamicMeshDebugDrawMesh::GetVertexPosition(int32 VertexIndex) const
{
	if (IsValidVertex(VertexIndex))
	{
		return DynamicMesh->GetVertex(VertexIndex);
	}

	return FVector(0.0);
}

FVector FDynamicMeshDebugDrawMesh::GetVertexNormal(int32 VertexIndex) const
{
	if (IsValidVertex(VertexIndex))
	{
		return FVector(DynamicMesh->GetVertexNormal(VertexIndex));
	}

	return FVector(0.0);
}

int32 FDynamicMeshDebugDrawMesh::GetMaxTriangleIndex() const
{
	if (DynamicMesh)
	{
		return DynamicMesh->MaxTriangleID();
	}
	return 0;
}

bool FDynamicMeshDebugDrawMesh::IsValidTriangle(int32 TriangleIndex) const
{
	return (DynamicMesh && DynamicMesh->IsTriangle(TriangleIndex));
}

FIntVector3 FDynamicMeshDebugDrawMesh::GetTriangle(int32 TriangleIndex) const
{
	if (IsValidTriangle(TriangleIndex))
	{
		return DynamicMesh->GetTriangle(TriangleIndex);
	}
	return FIntVector3(UE::Geometry::FDynamicMesh3::InvalidID);
}
