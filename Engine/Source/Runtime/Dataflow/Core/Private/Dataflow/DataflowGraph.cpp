// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowGraph.h"

#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Logging/LogMacros.h"
#include "Dataflow/DataflowArchive.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowGraph)

DEFINE_LOG_CATEGORY_STATIC(DATAFLOW_LOG, Error, All);

namespace UE::Dataflow
{
	namespace Private
	{
		FString GetConnectionFullName(const FDataflowConnection& Connection)
		{
			static const FName InvalidName("[Invalid]");

			const FName NodeName = Connection.GetOwningNode()
				? Connection.GetOwningNode()->GetName()
				: InvalidName;
			const FName ConnectionName = Connection.GetName();

			return FString::Format(TEXT("{0}:{1}"), { NodeName.ToString(), ConnectionName.ToString() });
		}

		FLink MakeConnectionLink(FDataflowOutput& Output, FDataflowInput& Input)
		{
			return FLink(
				Output.GetOwningNode()->GetGuid(),
				Output.GetGuid(),
				Input.GetOwningNode()->GetGuid(),
				Input.GetGuid()
			);
		}
	}

	TSet<FName> FGraph::RegisteredFilters = {};

	FGraph::FGraph(FGuid InGuid)
		: Guid(InGuid)
	{
	}

	void FGraph::Reset()
	{
		Nodes.Reset();
		FilteredNodes.Reset();
		Connections.Reset();
		DisabledNodes.Reset();

		// Empty graphs have a zero topology Guid
		GraphTopologyGuid = FGuid();
	}

	void FGraph::OnChangeGraphTopology()
	{
		// Make sure we get a new topology Guid 
		GraphTopologyGuid = FGuid::NewGuid();
	}

	const TArray<FGraph::FNodeSharedPtr>& FGraph::GetFilteredNodes(const FName& NodeFilter) const
	{
		static TArray<FNodeSharedPtr> EmptyNodes;
		if (const TArray<FNodeSharedPtr>* FoundNodes = FilteredNodes.Find(NodeFilter))
		{
			return *FoundNodes;
		}
		return EmptyNodes;
	}

	FGraph::FNodeSharedPtr FGraph::AddNode(FNodeSharedPtr NewNode)
	{
		if (NewNode)
		{
			Nodes.AddUnique(NewNode);
			for (const FName& RegisteredType : RegisteredFilters)
			{
				if (NewNode->IsA(RegisteredType))
				{
					FilteredNodes.FindOrAdd(RegisteredType).Add(NewNode);
				}
			}
			OnChangeGraphTopology();
		}
		return NewNode;
	}

	FGraph::FNodeSharedPtr FGraph::FindBaseNode(FGuid InGuid)
	{
		for (const FNodeSharedPtr& Node : Nodes)
		{
			if (Node && Node->GetGuid() == InGuid)
			{
				return Node;
			}
		}
		return {};
	}

	FGraph::FConstNodeSharedPtr FGraph::FindBaseNode(FGuid InGuid) const
	{
		for (const FNodeSharedPtr& Node : Nodes)
		{
			if (Node && Node->GetGuid() == InGuid)
			{
				return Node;
			}
		}
		return {};
	}

	FGraph::FNodeSharedPtr FGraph::FindBaseNode(FName InName)
	{
		for (const FNodeSharedPtr& Node : Nodes)
		{
			if (Node && Node->GetName().IsEqual(InName))
			{
				return Node;
			}
		}
		return {};
	}

	FGraph::FConstNodeSharedPtr FGraph::FindBaseNode(FName InName) const
	{
		for (const FNodeSharedPtr& Node : Nodes)
		{
			if (Node && Node->GetName().IsEqual(InName))
			{
				return Node;
			}
		}
		return {};
	}

	FGraph::FNodeSharedPtr FGraph::FindFilteredNode(const FName& NodeFilter, FName InName) const
	{
		for (const FNodeSharedPtr& Node : GetFilteredNodes(NodeFilter))
		{
			if (Node && Node->GetName().IsEqual(InName))
			{
				return Node;
			}
		}
		return {};
	}

	void FGraph::RemoveNode(TSharedPtr<FDataflowNode> Node)
	{
		if (Node)
		{
			for (FDataflowOutput* Output : Node->GetOutputs())
			{
				ClearConnections(Output);
			}
			for (FDataflowInput* Input : Node->GetInputs())
			{
				ClearConnections(Input);
			}

			Nodes.Remove(Node);
			OnChangeGraphTopology();

			for (const FName& RegisteredType : RegisteredFilters)
			{
				if (Node->IsA(RegisteredType))
				{
					if (TArray<FNodeSharedPtr>* FoundNodes = FilteredNodes.Find(RegisteredType))
					{
						FoundNodes->Remove(Node);
					}
				}
			}
		}
	}

	void FGraph::ClearConnections(FDataflowConnection* InConnection)
	{
		if (InConnection)
		{
			// Todo(dataflow) : do this without triggering a invalidation. 
			//            or implement a better sync for the EdGraph and DataflowGraph
			if (InConnection->GetDirection() == FPin::EDirection::INPUT)
			{
				FDataflowInput* Input = static_cast<FDataflowInput*>(InConnection);
				TArray<FDataflowOutput*> Outputs = Input->GetConnectedOutputs();
				for (FDataflowOutput* Output : Outputs)
				{
					Disconnect(Output, Input);
				}
			}
			else if (InConnection->GetDirection() == FPin::EDirection::OUTPUT)
			{
				FDataflowOutput* Output = static_cast<FDataflowOutput*>(InConnection);
				TArray<FDataflowInput*> Inputs = Output->GetConnectedInputs();
				for (FDataflowInput* Input : Inputs)
				{
					Disconnect(Output, Input);
				}
			}
		}
	}

	void FGraph::ClearConnections(FDataflowInput* InConnection)
	{
		if (InConnection)
		{
			for (FDataflowOutput* Output : InConnection->GetConnectedOutputs())
			{
				Disconnect(Output, InConnection);
			}
		}
	}

	void FGraph::ClearConnections(FDataflowOutput* OutConnection)
	{
		if (OutConnection)
		{
			for (FDataflowInput* Input : OutConnection->GetConnectedInputs())
			{
				Disconnect(OutConnection, Input);
			}
		}
	}

	bool FGraph::CanConnect(const FDataflowOutput& Output, const FDataflowInput& Input) const
	{
		return (GetConnectType(Output, Input) != EConnectType::REJECTED);
	}

	FGraph::EConnectType FGraph::GetConnectType(const FDataflowOutput& Output, const FDataflowInput& Input) const
	{
		// Direct concrete type match 
		const bool bAreStrictlyTheSameType = (Output.GetType() == Input.GetType());
		const bool bBothHaveConcreteType = (Output.HasConcreteType() && Input.HasConcreteType());
		if (bAreStrictlyTheSameType && bBothHaveConcreteType)
		{
			return EConnectType::DIRECT;
		}

		// both are unassigned anytypes 
		const bool bIsInputUnassignedAnyType = Input.IsAnyType() && !Input.HasConcreteType();
		const bool bIsOutputUnassignedAnyType = Output.IsAnyType() && !Output.HasConcreteType();
		if (bIsInputUnassignedAnyType && bIsOutputUnassignedAnyType)
		{
			// todo(dataflow) currently unsupported by could be in the future by promoting both input and output 
			//				  using their default values if available
			return EConnectType::REJECTED;
		}

		// in case where both are anytypes but only one has concrete type 
		// we want to favor the unassigned one to be promoted
		const bool bFavorInputPromotion = 
			(bIsInputUnassignedAnyType && Output.IsAnyType() && Output.HasConcreteType())
			|| (Input.IsAnyType() && !Output.IsAnyType() && Output.HasConcreteType())
			;

		const bool bCanChangeInput = Input.IsAnyType() && Output.HasConcreteType();
		if (bCanChangeInput)
		{
			// Already connected input or type dependencies must be rejected
			// todo(dataflow) : the input being connected rule could be relaxed in the future 
			//					because inputs are always connected to a single output
			if (!Input.IsSafeToTryChangingType())
			{
				return EConnectType::REJECTED;
			}

			const bool bWouldBeCompatible = UE::Dataflow::FAnyTypesRegistry::AreTypesCompatibleStatic(Output.GetType(), Input.GetOriginalType());
			if (bWouldBeCompatible)
			{
				return EConnectType::INPUT_PROMOTION;
			}
		}

		const bool bCanChangeOutput = Output.IsAnyType() && Input.HasConcreteType();
		if (Output.IsAnyType())
		{
			// Already connected output or type dependencies must be rejected
			// because a output can be connected to multiple output 
			if (!Output.IsSafeToTryChangingType())
			{
				return EConnectType::REJECTED;
			}

			const bool bWouldBeCompatible = UE::Dataflow::FAnyTypesRegistry::AreTypesCompatibleStatic(Input.GetType(), Output.GetOriginalType());
			if (bWouldBeCompatible)
			{
				return EConnectType::OUTPUT_PROMOTION;
			}
		}

		return EConnectType::REJECTED;

	}

	bool FGraph::Connect(FDataflowConnection* ConnectionA, FDataflowConnection* ConnectionB)
	{
		using namespace UE::Dataflow;
		if (ConnectionA && ConnectionB)
		{
			if (ConnectionA->GetDirection() == FPin::EDirection::OUTPUT
				&& ConnectionB->GetDirection() == FPin::EDirection::INPUT)
			{
				FDataflowOutput* Output = static_cast<FDataflowOutput*>(ConnectionA);
				FDataflowInput* Input = static_cast<FDataflowInput*>(ConnectionB);
				Connect(Output, Input);
				return true;
			}
			if (ConnectionA->GetDirection() == FPin::EDirection::INPUT
				&& ConnectionB->GetDirection() == FPin::EDirection::OUTPUT)
			{
				FDataflowOutput* Output = static_cast<FDataflowOutput*>(ConnectionB);
				FDataflowInput* Input = static_cast<FDataflowInput*>(ConnectionA);
				Connect(Output, Input);
				return true;
			}
		}
		return false;
	}

	void FGraph::Connect(FDataflowOutput* OutputConnection, FDataflowInput* InputConnection)
	{
		if (ensure(OutputConnection && InputConnection))
		{
			Connect(*OutputConnection, *InputConnection);
		}
	}

	bool FGraph::Connect(FDataflowOutput& Output, FDataflowInput& Input)
	{
		const EConnectType ConnectType = GetConnectType(Output, Input);
		if (ConnectType == EConnectType::REJECTED)
		{
			UE_LOGF(LogChaosDataflow, Error, "FGraph::Connect(): failed to connect output [%ls] from input [%ls] - incompatible types",
				*Private::GetConnectionFullName(Output),
				*Private::GetConnectionFullName(Input)
			);
			return false;
		}

		if (ConnectType == EConnectType::INPUT_PROMOTION)
		{
			UE_LOGF(LogChaosDataflow, Verbose, "FGraph::Connect(): updating input [%ls] type to match output [%ls] type (%ls)",
				*Private::GetConnectionFullName(Output),
				*Private::GetConnectionFullName(Input),
				*Output.GetType().ToString()
			);
			Input.ResetToOriginalType();
			//Input.OwningNode->NotifyConnectionTypeChanged(&Input);
			Input.SetConcreteType(Output.GetType());
			Input.OwningNode->NotifyConnectionTypeChanged(&Input);
		}
		else if (ConnectType == EConnectType::OUTPUT_PROMOTION)
		{
			UE_LOGF(LogChaosDataflow, Verbose, "FGraph::Connect(): updating output [%ls] type to match input [%ls] type (%ls)",
				*Private::GetConnectionFullName(Output),
				*Private::GetConnectionFullName(Input),
				*Input.GetType().ToString()
			);
			Output.ResetToOriginalType();
			//Output.OwningNode->NotifyConnectionTypeChanged(&Output);
			Output.SetConcreteType(Input.GetType());
			Output.OwningNode->NotifyConnectionTypeChanged(&Output);
		}

		FDataflowOutput* const OldOutput = Input.GetConnection();
		if (OldOutput != &Output)
		{
			if (OldOutput)
			{
				UE_LOGF(LogChaosDataflow, Verbose, "FGraph::Connect(): Disconnecting output [%ls] from input [%ls]", 
					*Private::GetConnectionFullName(*OldOutput), 
					*Private::GetConnectionFullName(Input)
				);

				// Note: Do not remove the expired connection from the input to avoid an unnecessary invalidation.
				//       Simply clobber it with calling AddConnection() on the input instead.
				OldOutput->RemoveConnection(&Input);
				Connections.RemoveSwap(Private::MakeConnectionLink(*OldOutput, Input));
			}

			UE_LOGF(LogChaosDataflow, Verbose, "FGraph::Connect(): Connecting output [%ls] to input [%ls]",
				*Private::GetConnectionFullName(Output),
				*Private::GetConnectionFullName(Input)
			);

			Output.AddConnection(&Input);
			Input.AddConnection(&Output);
			Connections.Add(Private::MakeConnectionLink(Output, Input));

			OnChangeGraphTopology();
		}
		return true;
	}

	void FGraph::Disconnect(FDataflowOutput* OutputConnection, FDataflowInput* InputConnection)
	{
		if (OutputConnection && InputConnection)
		{
			UE_LOGF(LogChaosDataflow, Verbose, "FGraph::Disconnect(): Disconnecting output [%ls] from input [%ls]",
				*Private::GetConnectionFullName(*OutputConnection),
				*Private::GetConnectionFullName(*InputConnection)
			);

			OutputConnection->RemoveConnection(InputConnection);
			InputConnection->RemoveConnection(OutputConnection);
			Connections.RemoveSwap(Private::MakeConnectionLink(*OutputConnection, *InputConnection));

			OnChangeGraphTopology();
		}
	}

	void FGraph::AddReferencedObjects(FReferenceCollector& Collector)
	{
		for (TSharedPtr<FDataflowNode>& Node : Nodes)
		{
			Collector.AddPropertyReferencesWithStructARO(Node->TypedScriptStruct(), Node.Get());
		}
	}

	void FGraph::Serialize(FArchive& Ar, UObject* OwningObject)
	{
		Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
		Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

		Ar << Guid;
		if (Ar.IsSaving())
		{
			FGraph::SerializeForSaving(Ar, this, Nodes, Connections);
		}
		else if( Ar.IsLoading())
		{
			FGraph::SerializeForLoading(Ar, this, OwningObject);
		}
	}

	void FGraph::SerializeForSaving(FArchive& Ar, FGraph* InGraph, TArray<TSharedPtr<FDataflowNode>>& InNodes, TArray<FLink>& InConnections)
	{
		check(InGraph);

		FGuid ArGuid;
		FName ArType, ArName;
		int32 ArNum = InNodes.Num();

		Ar << InGraph->GraphTopologyGuid;

		Ar << ArNum;
		for (TSharedPtr<FDataflowNode> Node : InNodes)
		{
			ArGuid = Node->GetGuid();
			ArType = Node->GetType();
			ArName = Node->GetName();
			Ar << ArGuid << ArType << ArName;

			DATAFLOW_OPTIONAL_BLOCK_WRITE_BEGIN()
			{
				// Node needs to be serialized first to make sure it registers all the dynamic input/output for when input and output will be deserialized
				Node->SerializeInternal(Ar);

				// keep outputs and inputs separated even though their serialization code looks almost identical
				// this is to make sure we can handle when number of inputs or outputs have changed on the node
				int32 ArNumOutputs = Node->GetOutputs().Num();
				Ar << ArNumOutputs;
				for (FDataflowConnection* Output : Node->GetOutputs())
				{
					ArGuid = Output->GetGuid();
					ArType = Output->GetType();
					ArName = Output->GetName();
					Ar << ArGuid << ArType << ArName;

					bool bIsAnytype = Output->IsAnyType();
					Ar << bIsAnytype;
					bool bIsHidden = Output->GetPinIsHidden();
					Ar << bIsHidden;
				}

				int32 ArNumInputs = Node->GetInputs().Num();
				Ar << ArNumInputs;
				for (FDataflowConnection* Input : Node->GetInputs())
				{
					ArGuid = Input->GetGuid();
					ArType = Input->GetType();
					ArName = Input->GetName();
					Ar << ArGuid << ArType << ArName;

					bool bIsAnytype = Input->IsAnyType();
					Ar << bIsAnytype;
					bool bIsHidden = Input->GetPinIsHidden();
					Ar << bIsHidden;
				}
			}
			DATAFLOW_OPTIONAL_BLOCK_WRITE_END();
		}

		Ar << InConnections;
	}

	void FGraph::SerializeForLoading(FArchive& Ar, FGraph* InGraph, UObject* OwningObject)
	{
		check(InGraph);

		InGraph->Reset();

		const bool bDataflowSeparateInputOutputSerialization = (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::DataflowSeparateInputOutputSerialization);
		const bool bDataflowAnyTypeSupport = (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::DataflowAnyTypeSupport);
		const bool bDataflowTemplateTypeFix = (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::DataflowTemplatedTypeFix);
		const bool bDataflowAddGraphTopologyGuid = (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::DataflowAddGraphTopologyGuid);

		FGuid ArGuid;
		FName ArType, ArName;
		int32 ArNum = 0;

		TMap<FGuid, TSharedPtr<FDataflowNode> > NodeGuidMap;
		TMap<FGuid, FDataflowConnection* > ConnectionGuidMap;
		TArray<FDataflowConnection*> ConnectionsToFix;

		// returns true if the connection is to be fixed
		auto AddTemplateTypedConnectionToBeFixed = [&ConnectionsToFix, bDataflowTemplateTypeFix](FDataflowConnection* Connection, FName SerializedType) -> bool
			{
				if (Connection && !bDataflowTemplateTypeFix)
				{
					const bool bSametype = (Connection->GetType() == SerializedType);
					const bool bIsOldTemplatedType = !bSametype && Connection->GetType().ToString().StartsWith(SerializedType.ToString());
					if (bIsOldTemplatedType)
					{
						Connection->ForceSimpleType(SerializedType);
						ConnectionsToFix.Add(Connection);
						return true;
					}
				}
				return false;
			};

		if (bDataflowAddGraphTopologyGuid)
		{
			Ar << InGraph->GraphTopologyGuid;
		}
		else
		{
			InGraph->GraphTopologyGuid = FGuid::NewGuid();
		}

		// Adding and connecting the nodes below calls OnChangeGraphTopology, which causes 
		// GraphTopologyGuid to be regenerated, but the intended behavior is to restore the FGraph 
		// into the state it was in when it was saved.
		//
		// This guard restores the value of GraphTopologyGuid on leaving this scope.
		TGuardValue<FGuid> GraphTopologyGuidGuard(InGraph->GraphTopologyGuid, InGraph->GraphTopologyGuid);

		Ar << ArNum;
		for (int32 Ndx = ArNum; Ndx > 0; Ndx--)
		{
			FName ArNodeName;
			Ar << ArGuid << ArType << ArNodeName;

			TSharedPtr<FDataflowNode> Node = FNodeFactory::GetInstance()->NewNodeFromRegisteredType(*InGraph, { ArGuid, ArType, ArNodeName, OwningObject });
			DATAFLOW_OPTIONAL_BLOCK_READ_BEGIN(Node != nullptr)
			{
				ensure(!NodeGuidMap.Contains(ArGuid));
				NodeGuidMap.Add(ArGuid, Node);

				const bool bDataflowHideablePinSupport = (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::DataflowHideablePins);

				if (!bDataflowSeparateInputOutputSerialization)
				{

					// former input / output serialization method where we only store aggregate number of inputs and outputs
					// this has limitation when adding more outputs or inputs
					int ArNumInputsOutputs;
					Ar << ArNumInputsOutputs;
					TArray< FDataflowConnection* > InputsOutputs;
					InputsOutputs.Append(Node->GetOutputs());
					InputsOutputs.Append(Node->GetInputs());

					// skip offset is use to correct the mismatch of outputs have been added
					int SkipOffset = 0;
					for (int ConnectionIndex = 0; ConnectionIndex < ArNumInputsOutputs; ConnectionIndex++)
					{
						Ar << ArGuid << ArType << ArName;

						int AdjustedConnectionIndex = ConnectionIndex + SkipOffset;
						if (InputsOutputs.IsValidIndex(AdjustedConnectionIndex))
						{
							FDataflowConnection* Connection = InputsOutputs[AdjustedConnectionIndex];

							// if the name does not match this means the node has changed since the last serialization 
							// ( added outputs for example that shift the index )
							// in that case we try to recover by finding the next good node
							// note we cannot just find by name as some nodes have inputs and outputs named the same 
							// todo: implement a better way to serialize inputs and outputs seperately to avoid this case
							while (Connection && Connection->GetName() != ArName)
							{
								SkipOffset++;
								AdjustedConnectionIndex = ConnectionIndex + SkipOffset;
								if (InputsOutputs.IsValidIndex(AdjustedConnectionIndex))
								{
									Connection = InputsOutputs[AdjustedConnectionIndex];
								}
								else
								{
									Connection = nullptr;
								}
							}
							if (Connection)
							{
								if (!AddTemplateTypedConnectionToBeFixed(Connection, ArType))
								{
									check(Connection->GetType() == ArType);
								}
								Connection->SetGuid(ArGuid);
								ensure(!ConnectionGuidMap.Contains(ArGuid));
								ConnectionGuidMap.Add(ArGuid, Connection);
							}
						}
					}

					Node->SerializeInternal(Ar);
				}
				else
				{
					// we need to deserilaize the node first because if it may add more inputs that may 
					// be referenced when deserializing them below ( see Dataflow Node AddPin method )
					Node->SerializeInternal(Ar);

					bool bIsAnyType = false;
					bool bIsHidden = true;
					// Outputs deserialization
					{
						int32 ArNumOutputs;
						Ar << ArNumOutputs;

						for (int32 OutputIndex = 0; OutputIndex < ArNumOutputs; OutputIndex++)
						{
							Ar << ArGuid << ArType << ArName;
							if (bDataflowAnyTypeSupport)
							{
								Ar << bIsAnyType;
							}
							if (bDataflowHideablePinSupport)
							{
								Ar << bIsHidden;
							}

							FDataflowOutput* Output = Node->FindOutput(ArName);
							if (!Output)
							{
								// Find out if the output has recently been redirected
								Output = Node->RedirectSerializedOutput(ArName);
								UE_CLOGF(Output, LogChaos, Display, "Output (%ls) has been redirected to output (%ls) in Dataflow node (%ls)."
									, *ArName.ToString(), *Output->GetName().ToString(), *ArNodeName.ToString());
							}
							if (Output)
							{
								if (bIsAnyType)
								{
									Output->SetAsAnyType(bIsAnyType, ArType);
								}
								if (!AddTemplateTypedConnectionToBeFixed(Output, ArType))
								{
									if (Output->GetType() != ArType)
									{
										FString NoSpaceArType = ArType.ToString();
										NoSpaceArType.RemoveSpacesInline();
										check(Output->GetType() == *NoSpaceArType || bIsAnyType);
									}
								}
								Output->SetPinIsHidden(bIsHidden);
								Output->SetGuid(ArGuid);
								ensure(!ConnectionGuidMap.Contains(ArGuid));
								ConnectionGuidMap.Add(ArGuid, Output);
							}
							else
							{
								// output has been serialized but cannot be found
								// this means the definition of the node has changed and the output is no longer registered
								UE_LOGF(LogChaos, Display, "Cannot find registered output (%ls) in Dataflow node (%ls) - this may result in missing connection(s)."
									, *ArName.ToString(), *ArNodeName.ToString());
							}
						}
					}

					// Inputs deserialization
					{
						int32 ArNumInputs;
						Ar << ArNumInputs;

						for (int32 InputIndex = 0; InputIndex < ArNumInputs; InputIndex++)
						{
							Ar << ArGuid << ArType << ArName;
							if (bDataflowAnyTypeSupport)
							{
								Ar << bIsAnyType;
							}
							if (bDataflowHideablePinSupport)
							{
								Ar << bIsHidden;
							}

							FDataflowInput* Input = Node->FindInput(ArName);
							if (!Input)
							{
								// Find out if the input has recently been redirected
								Input = Node->RedirectSerializedInput(ArName);
								UE_CLOGF(Input, LogChaos, Display, "Input (%ls) has been redirected to input (%ls) in Dataflow node (%ls)."
									, *ArName.ToString(), *Input->GetName().ToString(), *ArNodeName.ToString());
							}
							if (Input)
							{
								if (bIsAnyType)
								{
									Input->SetAsAnyType(bIsAnyType, ArType);
								}
								if (!AddTemplateTypedConnectionToBeFixed(Input, ArType))
								{
									if (Input->GetType() != ArType)
									{
										FString NoSpaceArType = ArType.ToString();
										NoSpaceArType.RemoveSpacesInline();
										check(Input->GetType() == *NoSpaceArType || bIsAnyType);
									}
								}
								Input->SetPinIsHidden(bIsHidden);
								Input->SetGuid(ArGuid);
								ensure(!ConnectionGuidMap.Contains(ArGuid));
								ConnectionGuidMap.Add(ArGuid, Input);
							}
							else
							{
								// input has been serialized but cannot be found
								// this means the definition of the node has changed and the input is no longer registered
								UE_LOGF(LogChaos, Display, "Cannot find registered input (%ls) in Dataflow node (%ls) - this may result in missing connection(s)."
									, *ArName.ToString(), *ArNodeName.ToString());
							}
						}
					}
				}
			}
			DATAFLOW_OPTIONAL_BLOCK_READ_ELSE()
			{
				InGraph->DisabledNodes.Add(ArNodeName);
				if (!IsRunningCommandlet() && GIsEditor)  // Don't ensure during cooking or during non editor operations
				{
					ensureMsgf(false,
						TEXT("Error: Missing registered node type (%s) will be removed from graph on load. Graph will fail to evaluate due to missing node (%s).")
						, *ArType.ToString(), *ArNodeName.ToString());
				}
				else
				{
					UE_LOGF(LogChaos, Display, 
						"Error: Missing registered node type (%ls) will be removed from graph on load. Graph will fail to evaluate due to missing node (%ls)."
						, *ArType.ToString(), *ArNodeName.ToString());
				}
			}
			DATAFLOW_OPTIONAL_BLOCK_READ_END();
		}

		TArray< FLink > LocalConnections;
		Ar << LocalConnections;
		for (const FLink& Con : LocalConnections)
		{
			if (NodeGuidMap.Contains(Con.InputNode) && NodeGuidMap.Contains(Con.OutputNode))
			{
				if (ConnectionGuidMap.Contains(Con.Input) && ConnectionGuidMap.Contains(Con.Output))
				{
					if (ConnectionGuidMap[Con.Output] && ConnectionGuidMap[Con.Output]->Direction == FPin::EDirection::OUTPUT &&
						ConnectionGuidMap[Con.Input] && ConnectionGuidMap[Con.Input]->Direction == FPin::EDirection::INPUT)
					{
						FDataflowOutput* Output = static_cast<FDataflowOutput*>(ConnectionGuidMap[Con.Output]);
						FDataflowInput* Input = static_cast<FDataflowInput*>(ConnectionGuidMap[Con.Input]);
						if (Input->GetType() == Output->GetType())
						{
							InGraph->Connect(Output, Input);
						}
					}
				}
			}
		}

		// fix templated types if any : see bDataflowTemplateTypeFix
		for (FDataflowConnection* ConnectionToFix : ConnectionsToFix)
		{
			ConnectionToFix->FixAndPropagateType();
		}
	}

	void RegisterNodeFilter(const FName& NodeFilter)
	{
		FGraph::RegisteredFilters.Add(NodeFilter);
	}
}

