// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowImage)

int32 FDataflowImage::GetWidth() const
{
	return Image.GetWidth();
}

int32 FDataflowImage::GetHeight() const
{
	return Image.GetHeight();
}

const FImage& FDataflowImage::GetImage() const
{
	return Image;
}

void FDataflowImage::CreateR32F(EDataflowImageResolution Resolution, TArrayView64<float> InPixels)
{
	CreateR32F((int32)Resolution, (int32)Resolution, InPixels);
}

void FDataflowImage::CreateR32F(int32 Width, int32 Height, TArrayView64<float> InPixels)
{
	Image.Init(Width, Height, ERawImageFormat::R32F, EGammaSpace::Linear);

	if (Width * Height == InPixels.Num())
	{
		CopyPixels(InPixels);
	}
}

void FDataflowImage::CreateRGBA32F(EDataflowImageResolution Resolution, TArrayView64<FVector4f> InPixels)
{
	CreateRGBA32F((int32)Resolution, (int32)Resolution, InPixels);
}

void FDataflowImage::CreateRGBA32F(int32 Width, int32 Height, TArrayView64<FVector4f> InPixels)
{
	Image.Init(Width, Height, ERawImageFormat::RGBA32F, EGammaSpace::Linear);

	if (Width * Height == InPixels.Num())
	{
		CopyPixels(InPixels);
	}
}

void FDataflowImage::CreateFromColor(EDataflowImageResolution Resolution, FLinearColor Color)
{
	CreateFromColor((int32)Resolution, (int32)Resolution, Color);
}

void FDataflowImage::CreateFromColor(int32 Width, int32 Height, FLinearColor Color)
{
	CreateRGBA32F(Width, Height);

	TArray<FLinearColor> FillData;
	FillData.Init(Color, Width * Height);
	const FImageView SourceImage(FillData.GetData(), Width, Height);
	FImageCore::CopyImage(SourceImage, Image);
}

bool FDataflowImage::CopyPixels(TArrayView64<float> Pixels)
{
	if (Image.Format != ERawImageFormat::Type::R32F)
	{
		return false;
	}
	if ((GetWidth() * GetHeight()) != Pixels.Num())
	{
		return false;
	}
	const FImageView SrcImage(Pixels.GetData(), GetWidth(), GetHeight(), /*NumSlices*/1, ERawImageFormat::Type::R32F, EGammaSpace::Linear);
	FImageCore::CopyImage(SrcImage, Image);
	return true;
}

bool FDataflowImage::CopyPixels(TArrayView64<FVector4f> InPixels)
{
	return CopyRGBAPixels(InPixels);
}

bool FDataflowImage::CopyRGBAPixels(TArrayView64<FVector4f> Pixels)
{
	if (Image.Format != ERawImageFormat::Type::RGBA32F)
	{
		return false;
	}
	if ((GetWidth() * GetHeight()) != Pixels.Num())
	{
		return false;
	}
	const FImageView SrcImage(Pixels.GetData(), GetWidth(), GetHeight(), /*NumSlices*/1, ERawImageFormat::Type::RGBA32F, EGammaSpace::Linear);
	FImageCore::CopyImage(SrcImage, Image);
	return true;
}

void FDataflowImage::ConvertToRGBA32F()
{
	Image.ChangeFormat(ERawImageFormat::RGBA32F, EGammaSpace::Linear);
}

void FDataflowImage::ReadChannel(EDataflowImageChannel Channel, FDataflowImage& OutImage) const
{
	if (Image.Format == ERawImageFormat::R32F)
	{
		OutImage.CreateR32F(GetWidth(), GetHeight());
		// single channel image, all channels get the same data 
		FImageCore::CopyImage(this->Image, OutImage.Image);
	}
	else if (Image.Format == ERawImageFormat::RGBA32F)
	{
		OutImage.CreateR32F(GetWidth(), GetHeight());
		TArrayView64<float> DstPixels = OutImage.Image.AsR32F();

		const TArrayView64<const FLinearColor> SrcPixels = Image.AsRGBA32F();
		for (int32 Pixel = 0; Pixel < SrcPixels.Num(); ++Pixel)
		{
			const FLinearColor& SrcColor = SrcPixels[Pixel];
			DstPixels[Pixel] = SrcColor.Component((int32)Channel);
		}
	}
	else 
	{
		// unsupported format - image is likely not initialized likely fill with black  
		OutImage.CreateFromColor(GetWidth(), GetHeight(), FLinearColor::Black);
	}
}

void FDataflowImage::WriteChannel(EDataflowImageChannel Channel, const FDataflowImage& SrcImage)
{
	FDataflowImage DefaultMinImage;
	
	FImageView SrcImageView = SrcImage.Image;
	if (SrcImageView.GetWidth() == 0 || SrcImageView.GetHeight() == 0)
	{
		DefaultMinImage.CreateFromColor(4, 4, FLinearColor::Black);
		SrcImageView = DefaultMinImage.Image;
	}

	FImage ResizedSrcImage;
	if (SrcImageView.GetWidth() != GetWidth() && SrcImageView.GetHeight() != GetHeight())
	{
		// resize source image into ResizedSrcImage
		ResizedSrcImage.Init(GetWidth(), GetHeight(), SrcImageView.Format, EGammaSpace::Linear);
		FImageCore::ResizeImage(SrcImageView, ResizedSrcImage);
		SrcImageView = ResizedSrcImage;
	}

	FImage SingleChannelSrcImage;
	if (SrcImageView.Format != ERawImageFormat::R32F)
	{
		SingleChannelSrcImage.Init(GetWidth(), GetHeight(), ERawImageFormat::R32F, EGammaSpace::Linear);
		if (SrcImageView.Format == ERawImageFormat::RGBA32F)
		{
			// convert the image grayscale and single channel
			TArrayView64<float> DstPixels = SingleChannelSrcImage.AsR32F();

			const TArrayView64<const FLinearColor> SrcPixels = SrcImageView.AsRGBA32F();
			for (int32 Pixel = 0; Pixel < SrcPixels.Num(); ++Pixel)
			{
				const FLinearColor& SrcColor = SrcPixels[Pixel];
				const float Value = SrcColor.LinearRGBToHSV().B; // blue store the value of the color 
				DstPixels[Pixel] = Value;
			}
		}
		else
		{
			// all zero texture
			SingleChannelSrcImage.RawData.Init(0, SingleChannelSrcImage.RawData.Num());
		}
		SrcImageView = SingleChannelSrcImage;
	}

	ensure(SrcImageView.GetWidth() == GetWidth());
	ensure(SrcImageView.GetHeight() == GetHeight());
	ensure(SrcImageView.Format == ERawImageFormat::R32F);

	// finally write to the channel
	if (Image.Format == ERawImageFormat::R32F)
	{
		FImageCore::CopyImage(SrcImageView, Image);
	}
	else if (Image.Format == ERawImageFormat::RGBA32F)
	{
		const TArrayView64<const float> SrcPixels = SrcImageView.AsR32F();

		TArrayView64<FLinearColor> DstPixels = Image.AsRGBA32F();
		for (int32 Pixel = 0; Pixel < DstPixels.Num(); ++Pixel)
		{
			const float& SrcValue = SrcPixels[Pixel];
			DstPixels[Pixel].Component((int32)Channel) = SrcValue;
		}
	}
	else
	{
		// trying to write a channel to an probably non initialize image
		ensure(false);
	}
}

bool FDataflowImage::Serialize(FArchive& Ar)
{
	FImageInfo Info(Image);
	uint8 FormatAsInt = static_cast<uint8>(Info.Format);
	uint8 GammaSpaceAsInt = static_cast<uint8>(Info.GammaSpace);

	Ar << Info.SizeX;
	Ar << Info.SizeY;
	Ar << Info.NumSlices;
	Ar << FormatAsInt;
	Ar << GammaSpaceAsInt;

	if (Ar.IsLoading())
	{
		Info.Format = static_cast<ERawImageFormat::Type>(FormatAsInt);
		Info.GammaSpace = static_cast<EGammaSpace>(GammaSpaceAsInt);
		Image.Init(Info);
	}

	Ar << Image.RawData;

	return true;
}
