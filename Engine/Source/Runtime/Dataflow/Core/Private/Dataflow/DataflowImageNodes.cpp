// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowImageNodes.h"

#include "Dataflow/DataflowNodeFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowImageNodes)

namespace UE::Dataflow
{
	void RegisterDataflowImageNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowConvertToResolutionNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowImageFromColorNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowImageSplitChannelsNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowImageCombineChannelsNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowConvertToResolutionNode::FDataflowConvertToResolutionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	:FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Value);
	RegisterOutputConnection(&Resolution);
}

void FDataflowConvertToResolutionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using FResolutionPair = TPair<EDataflowImageResolution, EDataflowImageResolution>;
	constexpr EDataflowImageResolution ResolutionsArray[] =
	{
		EDataflowImageResolution::Resolution16,
		EDataflowImageResolution::Resolution32,
		EDataflowImageResolution::Resolution64,
		EDataflowImageResolution::Resolution128,
		EDataflowImageResolution::Resolution256,
		EDataflowImageResolution::Resolution512,
		EDataflowImageResolution::Resolution1024,
		EDataflowImageResolution::Resolution2048,
		EDataflowImageResolution::Resolution4096,
		EDataflowImageResolution::Resolution8192,
	};
	constexpr int32 NumResolutions = int32(sizeof(ResolutionsArray) / sizeof(EDataflowImageResolution));

	auto GetPrevious = [&ResolutionsArray](int32 Value) -> EDataflowImageResolution
		{
			for (int32 Index = 1; Index < NumResolutions; ++Index)
			{
				if (Value < (int32)ResolutionsArray[Index])
				{
					return ResolutionsArray[Index - 1];
				}
			}
			return EDataflowImageResolution::Resolution8192;
		};

	auto GetNext = [&ResolutionsArray](int32 Value) -> EDataflowImageResolution
		{
			for (int32 Index = (NumResolutions-2); Index >= 0 ; --Index)
			{
				if (Value > (int32)ResolutionsArray[Index])
				{
					return ResolutionsArray[Index + 1];
				}
			}
			return EDataflowImageResolution::Resolution16;
		};
	
	if (Out->IsA(&Resolution))
	{
		// clampo to reasonable number before converting to integer
		const int32 InValue = FMath::Clamp(GetValue(Context, &Value), 0.0, (double)TNumericLimits<int32>::Max());

		const EDataflowImageResolution PrevResolution = GetPrevious(InValue);
		const EDataflowImageResolution NextResolution = GetNext(InValue);

		EDataflowImageResolution OutResolution = EDataflowImageResolution::Resolution16;
		if (Method == EDataflowConvertToResolutionMethod::UseClosest)
		{
			if ((InValue - (int32)PrevResolution) < ((int32)NextResolution - InValue))
			{
				OutResolution = PrevResolution;
			}
			else
			{
				OutResolution = NextResolution;
			}
		}
		else if (Method == EDataflowConvertToResolutionMethod::UsePrevious)
		{
			OutResolution = PrevResolution;
		}
		else if (Method == EDataflowConvertToResolutionMethod::UseNext)
		{
			OutResolution = NextResolution;
		}

		SetValue(Context, MoveTemp(OutResolution), &Resolution);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowImageFromColorNode::FDataflowImageFromColorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	:FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&FillColor)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Resolution)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Image);
}

void FDataflowImageFromColorNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Image))
	{
		const FLinearColor& InColor = GetValue(Context, &FillColor);
		const EDataflowImageResolution& InResolution = GetValue(Context, &Resolution);

		FDataflowImage OutImage;
		OutImage.CreateFromColor(InResolution, InColor);
		SetValue(Context, MoveTemp(OutImage), &Image);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowImageSplitChannelsNode::FDataflowImageSplitChannelsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	:FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Image);
	RegisterOutputConnection(&Red);
	RegisterOutputConnection(&Green);
	RegisterOutputConnection(&Blue);
	RegisterOutputConnection(&Alpha);
}

void FDataflowImageSplitChannelsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	auto EvaluateForChannel = [this](UE::Dataflow::FContext& Context, EDataflowImageChannel Channel, const FDataflowImage* OutputRef)
		{
			FDataflowImage OutImage;
			const FDataflowImage& InImage = GetValue(Context, &Image);
			InImage.ReadChannel(Channel, OutImage);
			SetValue(Context, MoveTemp(OutImage), OutputRef);
		};

	if (Out->IsA(&Red))
	{
		EvaluateForChannel(Context, EDataflowImageChannel::Red, &Red);
	}
	else if (Out->IsA(&Green))
	{
		EvaluateForChannel(Context, EDataflowImageChannel::Green, &Green);
	}
	else if (Out->IsA(&Blue))
	{
		EvaluateForChannel(Context, EDataflowImageChannel::Blue, &Blue);
	}
	else if (Out->IsA(&Alpha))
	{
		EvaluateForChannel(Context, EDataflowImageChannel::Alpha, &Alpha);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowImageCombineChannelsNode::FDataflowImageCombineChannelsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	:FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Red);
	RegisterInputConnection(&Green);
	RegisterInputConnection(&Blue);
	RegisterInputConnection(&Alpha);
	RegisterOutputConnection(&Image);
}

void FDataflowImageCombineChannelsNode::GetUserDefinedResolution(const TArray<const FDataflowImage*>& Images, int32& OutWidth, int32& OutHeight)  const
{
	// first set safe non zero values
	OutWidth  = (int32)Resolution;
	OutHeight = (int32)Resolution;

	if (ResolutionOptions == EDataflowImageCombineResolutionOption::UserDefined 
		|| Images.Num() == 0)
	{
		return;
	}

	if (Images[0])
	{
		int32 PotentialW = Images[0]->GetWidth();
		int32 PotentialH = Images[0]->GetHeight();
		for (const FDataflowImage* ImageEntry : Images)
		{
			if (ImageEntry)
			{
				const int32 ImageW = ImageEntry->GetWidth();
				const int32 ImageH = ImageEntry->GetHeight();
				if (ImageW > 0 && ImageH > 0)
				{
					if (ResolutionOptions == EDataflowImageCombineResolutionOption::Highest)
					{
						PotentialW = FMath::Max(PotentialW, ImageW);
						PotentialH = FMath::Max(PotentialH, ImageH);
					}
					else if (ResolutionOptions == EDataflowImageCombineResolutionOption::Lowest)
					{
						PotentialW = FMath::Min(PotentialW, ImageW);
						PotentialH = FMath::Min(PotentialH, ImageH);
					}
				}
			}
		}

		if (PotentialW > 0 && PotentialH > 0)
		{
			OutWidth = PotentialW;
			OutHeight = PotentialH;
		}
	}
}

void FDataflowImageCombineChannelsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Image))
	{
		const FDataflowImage& InRed   = GetValue(Context, &Red);
		const FDataflowImage& InGreen = GetValue(Context, &Green);
		const FDataflowImage& InBlue  = GetValue(Context, &Blue);
		const FDataflowImage& InAlpha = GetValue(Context, &Alpha);
		const TArray<const FDataflowImage*> InImages = { &InRed, &InGreen, &InBlue, &InAlpha };

		FDataflowImage OutImage;

		int32 OutWidth = 0;
		int32 OutHeight= 0;
		GetUserDefinedResolution(InImages, OutWidth, OutHeight);
		OutImage.CreateRGBA32F(OutWidth, OutHeight);

		OutImage.WriteChannel(EDataflowImageChannel::Red, InRed);
		OutImage.WriteChannel(EDataflowImageChannel::Green, InGreen);
		OutImage.WriteChannel(EDataflowImageChannel::Blue, InBlue);
		OutImage.WriteChannel(EDataflowImageChannel::Alpha, InAlpha);

		SetValue(Context, MoveTemp(OutImage), &Image);
	}
}
