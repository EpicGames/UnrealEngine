// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowImage.h"

#include "DataflowImageNodes.generated.h"

#define DATAFLOW_IMAGE_NODES_CATEGORY "Image"

UENUM()
enum class EDataflowConvertToResolutionMethod : int32
{
	UseClosest = 0 UMETA(DisplayName = "Closest resolution"),
	UsePrevious = 1 UMETA(DisplayName = "Previous resolution"),
	UseNext = 2 UMETA(DisplayName = "Next resolution"),
};

/**
* Convert a numeric value into a resolution 
*/
USTRUCT()
struct FDataflowConvertToResolutionNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowConvertToResolutionNode, "ConvertToResolution", DATAFLOW_IMAGE_NODES_CATEGORY, "")

public:
	FDataflowConvertToResolutionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Arbitrary value to convert to a image resolution type */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput));
	FDataflowNumericTypes Value;

	/** Method to use to match the resolution */
	UPROPERTY(EditAnywhere, Category = Options);
	EDataflowConvertToResolutionMethod Method = EDataflowConvertToResolutionMethod::UseClosest;

	/** Output resolution */
	UPROPERTY(meta = (DataflowOutput));
	EDataflowImageResolution Resolution = EDataflowImageResolution::Resolution16;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
* Create a RGBA image from a single color at a specific resolution
* Outputs are single channel images
*/
USTRUCT()
struct FDataflowImageFromColorNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowImageFromColorNode, "ImageFromColor", DATAFLOW_IMAGE_NODES_CATEGORY, "")

public:
	FDataflowImageFromColorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** color  to fill the image with */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput));
	FLinearColor FillColor = FLinearColor::Black;

	/** resolution of the output image */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput));
	EDataflowImageResolution Resolution = EDataflowImageResolution::Resolution16;

	/** Output image  */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowImage Image;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** 
* Split a image in individual channels 
* Outputs are single channel images 
*/
USTRUCT()
struct FDataflowImageSplitChannelsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowImageSplitChannelsNode, "ImageSplitChannels", DATAFLOW_IMAGE_NODES_CATEGORY, "")

public:
	FDataflowImageSplitChannelsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Input image to split per channel */
	UPROPERTY(meta = (DataflowInput));
	FDataflowImage Image;

	/** Red channel */
	UPROPERTY(meta = (DataflowOutput, PinColor = "R=1, G=0, B=0"));
	FDataflowImage Red;

	/** Green channel */
	UPROPERTY(meta = (DataflowOutput, PinColor = "R=0, G=1, B=0"));
	FDataflowImage Green;

	/** Blue channel */
	UPROPERTY(meta = (DataflowOutput, PinColor = "R=0, G=0.2, B=1"));
	FDataflowImage Blue;

	/** Alpha channel */
	UPROPERTY(meta = (DataflowOutput, PinColor = "R=0.7, G=0.7, B=0.7"));
	FDataflowImage Alpha;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

UENUM()
enum class EDataflowImageCombineResolutionOption : int32
{
	Lowest = 0 UMETA(DisplayName = "Lowest Resolution"),
	Highest = 1 UMETA(DisplayName = "Highest Resolution"),
	UserDefined = 2 UMETA(DisplayName = "User Defined Resolution "),
};

/**
* Combine channels into a single RGBA image
* Outputs are single channel images
*/
USTRUCT()
struct FDataflowImageCombineChannelsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowImageCombineChannelsNode, "ImageCombineChannels", DATAFLOW_IMAGE_NODES_CATEGORY, "")

public:
	FDataflowImageCombineChannelsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Red channel - if not connected, use black color */
	UPROPERTY(meta = (DataflowInput, PinColor = "R=1, G=0, B=0"));
	FDataflowImage Red;

	/** Green channel - if not connected, use black color */
	UPROPERTY(meta = (DataflowInput, PinColor = "R=0, G=1, B=0"));
	FDataflowImage Green;

	/** Blue channel - if not connected, use black color */
	UPROPERTY(meta = (DataflowInput, PinColor = "R=0, G=0.2, B=1"));
	FDataflowImage Blue;

	/** Alpha channel - if not connected, use black color */
	UPROPERTY(meta = (DataflowInput, PinColor = "R=0.7, G=0.7, B=0.7"));
	FDataflowImage Alpha;

	/** Output image recombined from input channels */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowImage Image;

	/** resolution option */
	UPROPERTY(EditAnywhere, Category = Options);
	EDataflowImageCombineResolutionOption ResolutionOptions = EDataflowImageCombineResolutionOption::Highest;

	/** resolution of the output image if the resolution option is set to user defined*/
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "ResolutionOptions == EDataflowImageCombineResolutionOption::UserDefined"))
	EDataflowImageResolution Resolution = EDataflowImageResolution::Resolution512;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	void GetUserDefinedResolution(const TArray<const FDataflowImage*>& Images, int32& OutWidth, int32& OutHeight) const;
};

namespace UE::Dataflow
{
	void RegisterDataflowImageNodes();
}
