// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowInputOutput.h"

#include "ChaosLog.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowCoreNodes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowInputOutput)

#define LOCTEXT_NAMESPACE "DataflowINputOutput"

namespace UE::Dataflow::Private
{
	static bool bDisableFreezingState = true;
	static FAutoConsoleVariableRef CVar(TEXT("p.Dataflow.DisableFreezingState"), bDisableFreezingState, TEXT("Disable the freezing state of Dataflow nodes. Useful for recovering from crashes caused by frozen nodes."));
}

FDataflowInput FDataflowInput::NoOpInput = FDataflowInput();
FDataflowOutput FDataflowOutput::NoOpOutput = FDataflowOutput();

FDataflowInput::FDataflowInput(const UE::Dataflow::FInputParameters& Param)
	: FDataflowConnection(UE::Dataflow::FPin::EDirection::INPUT, Param)
	, Connection(nullptr)
{
}

bool FDataflowInput::IsConnected() const
{
	return (Connection != nullptr);
}

void FDataflowInput::CheckRequired(UE::Dataflow::FContext& Context) const
{
	if (IsRequired() && !IsConnected())
	{
		const FText WarningMessage = FText::Format(LOCTEXT("InputRequiredMessage", "Input {0} is required but not connected"), FText::FromName(GetName()));
		Context.Warning(WarningMessage, this->OwningNode);
	}
}

bool FDataflowInput::AddConnection(FDataflowConnection* InOutput)
{
	if (GetType() == InOutput->GetType())
	{
		Connection = (FDataflowOutput*)InOutput;
		GetOwningNode()->Invalidate();
		return true;
	}
	return false;
}

bool FDataflowInput::RemoveConnection(FDataflowConnection* InOutput)
{
	if (ensure(Connection == (FDataflowOutput*)InOutput))
	{
		Connection = nullptr;
		GetOwningNode()->Invalidate();
		return true;
	}
	return false;
}

void FDataflowInput::GetConnections(TArray<FDataflowConnection*>& OutConnections) const
{
	OutConnections.Add(Connection);
}

TArray< FDataflowOutput* > FDataflowInput::GetConnectedOutputs()
{
	TArray<FDataflowOutput* > RetList;
	if (FDataflowOutput* Conn = GetConnection())
	{
		RetList.Add(Conn);
	}
	return RetList;
}

const TArray< const FDataflowOutput* > FDataflowInput::GetConnectedOutputs() const
{
	TArray<const FDataflowOutput* > RetList;
	if (const FDataflowOutput* Conn = GetConnection())
	{
		RetList.Add(Conn);
	}
	return RetList;
}

void FDataflowInput::Invalidate(const UE::Dataflow::FTimestamp& ModifiedTimestamp)
{
	OwningNode->Invalidate(ModifiedTimestamp);
}

void FDataflowInput::PullValue(UE::Dataflow::FContext& Context) const
{
	if (GetConnectedOutputs().Num())
	{
		ensure(GetConnectedOutputs().Num() == 1);
		if (const FDataflowOutput* ConnectionOut = GetConnection())
		{
			ConnectionOut->Evaluate(Context);
		}
	}
}

void FDataflowInput::FixAndPropagateType(FName InType)
{
	check(InType.ToString().StartsWith(GetType().ToString()));
	check(!FDataflowConnection::IsAnyType(InType));
	
	if (GetType() != InType)
	{
		SetTypeInternal(InType);
		bHasConcreteType = true;

		// if we have a reroute node propagate through to make sure each reroute segment is properly handled
		// IMPORTANT : this needs to be done before we propagate through the input connections
		if (const FDataflowReRouteNode* const ReRouteNode = OwningNode->AsType<FDataflowReRouteNode>())
		{
			for (FDataflowOutput* const ReRouteOutput : ReRouteNode->GetOutputs())
			{
				if (ReRouteOutput)
				{
					ReRouteOutput->FixAndPropagateType(InType);
				}
			}
		}

		OwningNode->NotifyConnectionTypeChanged(this);

		// Now propagate to the connected output
		if (FDataflowOutput* const Output = GetConnection())
		{
			Output->FixAndPropagateType(InType);
		}
	}
}

FDataflowArrayInput::FDataflowArrayInput(int32 InIndex, const UE::Dataflow::FArrayInputParameters& Param)
	: FDataflowInput(Param)
	, Index(InIndex)
	, ElementOffset(Param.InnerOffset)
	, ArrayProperty(Param.ArrayProperty)
{}

void* FDataflowArrayInput::RealAddress() const
{
	if (void* ContainerRealAddress = Super::RealAddress())
	{
		if (ArrayProperty)
		{
			if (void* const AddressAtIndex = ArrayProperty->GetValueAddressAtIndex_Direct(ArrayProperty->Inner, ContainerRealAddress, Index))
			{
				return (void*)((size_t)AddressAtIndex + (size_t)ElementOffset);
			}
		}
	}
	return nullptr;
}

//
//
//  Output
//
//
//

FDataflowOutput::FDataflowOutput(const UE::Dataflow::FOutputParameters& Param)
	: FDataflowConnection(UE::Dataflow::FPin::EDirection::OUTPUT, Param)
{
	OutputLock = MakeShared<FCriticalSection>();
}

const TArray<FDataflowInput*>& FDataflowOutput::GetConnections() const { return Connections; }
TArray<FDataflowInput*>& FDataflowOutput::GetConnections() { return Connections; }

const TArray< const FDataflowInput*> FDataflowOutput::GetConnectedInputs() const
{
	TArray<const FDataflowInput*> RetList;
	RetList.Reserve(Connections.Num());
	for (FDataflowInput* Ptr : Connections) 
	{ 
		RetList.Add(Ptr); 
	}
	return RetList;
}

TArray< FDataflowInput*> FDataflowOutput::GetConnectedInputs()
{
	TArray<FDataflowInput*> RetList;
	RetList.Reserve(Connections.Num());
	for (FDataflowInput* Ptr : Connections) 
	{ 
		RetList.Add(Ptr); 
	}
	return RetList;
}

bool FDataflowOutput::AddConnection(FDataflowConnection* InOutput)
{
	if (GetType() == InOutput->GetType())
	{
		Connections.Add((FDataflowInput*)InOutput);
		return true;
	}
	return false;
}

bool FDataflowOutput::RemoveConnection(FDataflowConnection* InInput)
{
	Connections.RemoveSwap((FDataflowInput*)InInput); return true;
}

void FDataflowOutput::GetConnections(TArray<FDataflowConnection*>& OutConnections) const
{
	OutConnections.Append(Connections);
}

FDataflowOutput& FDataflowOutput::SetPassthroughInput(const UE::Dataflow::FConnectionReference& Reference)
{
	check(OwningNode);
	const FDataflowInput* const PassthroughInput = OwningNode->FindInput(Reference);
	check(PassthroughInput);
	PassthroughKey = PassthroughInput->GetConnectionKey();
	return *this;
}

FDataflowOutput& FDataflowOutput::SetPassthroughInput(const UE::Dataflow::FConnectionKey& Key)
{
	check(Key == UE::Dataflow::FConnectionKey::Invalid || !OwningNode || OwningNode->FindInput(Key));
	PassthroughKey = Key;
	return *this;
}

const FDataflowInput* FDataflowOutput::GetPassthroughInput() const
{
	return OwningNode ? OwningNode->FindInput(PassthroughKey) : nullptr;
}

void FDataflowOutput::Invalidate(const UE::Dataflow::FTimestamp& ModifiedTimestamp)
{
	for (FDataflowConnection* Connection : GetConnections())
	{
		if (Connection)
		{
			Connection->Invalidate(ModifiedTimestamp);
		}
	}
}

bool FDataflowOutput::HasValidData(UE::Dataflow::FContext& Context) const
{
	if (HasFrozenValue())
	{
		return true;
	}
	if (OwningNode && Context.HasData(CacheKey(), OwningNode->GetTimestamp()))
	{
		return true;
	}
	return false;
}

bool FDataflowOutput::Evaluate(UE::Dataflow::FContext& Context) const
{
	check(OwningNode);

	if (IsOwningNodeEnabled())
	{
		return Context.Evaluate(*this);
	}
	else if (const FDataflowInput* const PassthroughInput = GetPassthroughInput())
	{
		ForwardInput(PassthroughInput, Context);
		return true;
	}
	return false;
}

bool FDataflowOutput::EvaluateImpl(UE::Dataflow::FContext& Context) const
{
	if (HasFrozenValue())
	{
		UE_LOGF(LogChaosDataflow, VeryVerbose, "FDataflowOutput::EvaluateImpl(): Node [%ls], Output [%ls] [FROZEN]", *GetOwningNode()->GetName().ToString(), *GetName().ToString());
		return true;
	}
	else
	{
		UE_LOGF(LogChaosDataflow, VeryVerbose, "FDataflowOutput::EvaluateImpl(): Node [%ls], Output [%ls]", *GetOwningNode()->GetName().ToString(), *GetName().ToString());

		if (Context.IsInCallstack(this))
		{
			ensureMsgf(false, TEXT("Connection %s is already in the callstack, this is certainly because of a loop in the graph"), *GetName().ToString());
			return false;
		}

		// check if the cache has a valid version
		if (Context.HasData(CacheKey(), OwningNode->GetTimestamp()))
		{
			UE_LOGF(LogChaosDataflow, VeryVerbose, "FDataflowOutput::EvaluateImpl(): Context has data, NodeTimestamp [%llu], CacheTimestamp [%llu]", OwningNode->GetTimestamp().Value, Context.GetTimestamp(CacheKey()).Value);
			return true;
		}

		// if not, add to the callstack and evaluate
		UE_LOGF(LogChaosDataflow, Verbose, "FDataflowNode::Evaluate(): Node [%ls], Output [%ls], NodeTimestamp [%llu]", *GetOwningNode()->GetName().ToString(), *GetName().ToString(), OwningNode->GetTimestamp().Value);

		UE::Dataflow::FContextScopedCallstack Callstack(Context, this);
		OwningNode->Evaluate(Context, this);
		
		// if no output was set, place a null entry 
		if (!Context.HasData(CacheKey(), OwningNode->GetTimestamp()))
		{
			this->SetNullValue(Context);
			return false;
		}
	}
	return true;
}

// Deprecated 5.8
TFuture<bool> FDataflowOutput::EvaluateParallel(UE::Dataflow::FContext& Context) const
{
	return Async(EAsyncExecution::TaskGraph, [&]() -> bool { return this->Evaluate(Context); });
}

void FDataflowOutput::Freeze(UE::Dataflow::FContext& Context, FInstancedPropertyBag& FrozenProperties)
{
	if (GetOwningNode() && Property)
	{
		UE_LOGF(LogChaosDataflow, Verbose, "FDataflowOutput::Freeze(): Node [%ls], Output [%ls]", *GetOwningNode()->GetName().ToString(), *GetName().ToString());

		if (Evaluate(Context))
		{
			// Set the cached value
			const uint8* const Data = (const uint8*)Context.GetUntypedData(CacheKey(), Property);  // Pretend that the cache element is a property container 
			const uint8* const ContainerPtr = Data - Property->GetOffset_ForInternal();     // TODO: Replace cache elements by property bags?

			FrozenProperties.AddProperty(GetName(), Property);
			FrozenProperties.SetValue(GetName(), Property, ContainerPtr);
		}
		// Else the output is not evaluate-able (e.g. deactivated node) but is still considered frozen and will return the provided default when GetValue is called.
	}
}

bool FDataflowOutput::HasFrozenValue() const
{
	return GetOwningNode() && GetOwningNode()->IsFrozen() && !UE::Dataflow::Private::bDisableFreezingState;
}

const uint8* FDataflowOutput::GetFrozenPropertyValue(const uint8* Default) const
{
	if (GetOwningNode())
	{
		const FInstancedPropertyBag& FrozenFroperties = GetOwningNode()->GetFrozenFroperties();

		if (const FPropertyBagPropertyDesc* PropertyDesc = FrozenFroperties.FindPropertyDescByName(GetName()))
		{
			const FConstStructView Value = FrozenFroperties.GetValue();
			if (Value.IsValid())
			{
				return Value.GetMemory() + PropertyDesc->CachedProperty->GetOffset_ForInternal();
			}
		}
	}
	return Default;
}

void FDataflowOutput::ForwardInput(const UE::Dataflow::FConnectionReference& InputReference, UE::Dataflow::FContext& Context) const
{
	if (Property && OwningNode)
	{
		const FDataflowInput* InputToForward = OwningNode->FindInput(InputReference);
		ForwardInput(InputToForward, Context);
	}
}

void FDataflowOutput::ForwardInput(const FDataflowInput* Input, UE::Dataflow::FContext& Context) const
{
	if (Property && OwningNode)
	{
		if (Input && Input->GetConnectedOutputs().Num())
		{
			ensure(Input->GetType() == GetType());
			ensure(Input->GetConnectedOutputs().Num() == 1);
			if (const FDataflowOutput* ConnectionOut = Input->GetConnection())
			{
				Input->PullValue(Context);
				if (Context.HasData(ConnectionOut->CacheKey()))
				{
					Context.SetDataReference(CacheKey(), Property, ConnectionOut->CacheKey(), GetOwningNodeTimestamp());
					return;
				}
			}
		}
	}
	// Fallback , set a null value so that the downstream nodes get a default value
	SetNullValue(Context);

}

void FDataflowOutput::SetNullValue(UE::Dataflow::FContext& Context) const
{
	if (Property && OwningNode)
	{
		// so when the input is going to pull the cached value , it will return a default value instead
		Context.SetNullData(CacheKey(), Property, GetOwningNodeGuid(), GetOwningNodeValueHash(), GetOwningNodeTimestamp());
	}
}

void FDataflowOutput::FixAndPropagateType(FName InType)
{
	check(InType.ToString().StartsWith(GetType().ToString()));
	check(!FDataflowConnection::IsAnyType(InType));

	if (GetType() != InType)
	{
		SetTypeInternal(InType);
		bHasConcreteType = true;

		// if we have a reroute node propagate through to make sure each reroute segment is properly handled
		// IMPORTANT : this needs to be done before we propagate through the output connections
		if (const FDataflowReRouteNode* const ReRouteNode = OwningNode->AsType<FDataflowReRouteNode>())
		{
			for (FDataflowInput* const RerouteInput : ReRouteNode->GetInputs())
			{
				if (RerouteInput)
				{
					RerouteInput->FixAndPropagateType(InType);
				}
			}
		}

		OwningNode->NotifyConnectionTypeChanged(this);

		// Now propagate through the connected inputs
		for (FDataflowInput* const Input : Connections)
		{
			if (Input)
			{
				Input->FixAndPropagateType(InType);
			}
		}
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowArrayOutput::FDataflowArrayOutput(int32 InIndex, const UE::Dataflow::FArrayOutputParameters& Param)
	: FDataflowOutput(Param)
	, Index(InIndex)
	, ElementOffset(Param.InnerOffset)
	, ArrayProperty(Param.ArrayProperty)
{}

void* FDataflowArrayOutput::RealAddress() const
{
	if (void* ContainerRealAddress = Super::RealAddress())
	{
		if (ArrayProperty)
		{
			if (void* const AddressAtIndex = ArrayProperty->GetValueAddressAtIndex_Direct(ArrayProperty->Inner, ContainerRealAddress, Index))
			{
				return (void*)((size_t)AddressAtIndex + (size_t)ElementOffset);
			}
		}
	}
	return nullptr;
}

bool FDataflowOutput::HasNodeFailedOrErrored(UE::Dataflow::FContext& Context) const
{
	return false; // deprecated 5.8
}

#undef LOCTEXT_NAMESPACE 