// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowMathNodes.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"

#include "MathUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowMathNodes)

namespace UE::Dataflow
{
	void RegisterDataflowMathNodes()
	{
		// scalar
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathAbsNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathAddNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathCeilNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathClampNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathConstantNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathCubeNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathDivideNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathExpNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathFloorNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathFracNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathInverseSquareRootNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathLogNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathLogXNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathMinimumNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathMaximumNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathMultiplyNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathNegateNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathOneMinusNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathPowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathReciprocalNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathRoundNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathSignNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathSquareNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathSquareRootNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathSubtractNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathTruncNode);

		// trigonometric
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathCosNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathSinNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathTanNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathArcCosNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathArcSinNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathArcTanNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathArcTan2Node)
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathDegToRadNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathRadToDegNode);

		// Deprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathMinimumNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMathMaximumNode);
	}
}

//-----------------------------------------------------------------------------------------------

FDataflowMathOneInputOperatorNode::FDataflowMathOneInputOperatorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	:FDataflowNode(InParam, InGuid)
{
}

void FDataflowMathOneInputOperatorNode::RegisterInputsAndOutputs()
{
	RegisterInputConnection(&A);
	RegisterOutputConnection(&Result);

	// Set the output to double for now so that it is strongly type and easy to connect to the next node
	// Once we can change the output type from the UI, this could be removed 
	SetOutputConcreteType(&Result, TDataflowSingleTypePolicy<float>::TypeName);
}

void FDataflowMathOneInputOperatorNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Result))
	{
		const double InA = GetValue(Context, &A);
		const double OutResult = ComputeResult(Context, InA);
		SetValue(Context, OutResult, &Result);
	}
}

//-----------------------------------------------------------------------------------------------

FDataflowMathTwoInputsOperatorNode::FDataflowMathTwoInputsOperatorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	:FDataflowNode(InParam, InGuid)
{
}

void FDataflowMathTwoInputsOperatorNode::RegisterInputsAndOutputs()
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterOutputConnection(&Result);

	// Set the output to double for now so that it is strongly type and easy to connect to the next node
	// Once we can change the output type from the UI, this could be removed 
	SetOutputConcreteType(&Result, TDataflowSingleTypePolicy<float>::TypeName);
}

void FDataflowMathTwoInputsOperatorNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Result))
	{
		const double InA = GetValue(Context, &A);
		const double InB = GetValue(Context, &B);
		const double OutResult = ComputeResult(Context, InA, InB);
		SetValue(Context, OutResult, &Result);
	}
}

//-----------------------------------------------------------------------------------------------

FDataflowMathAddNode::FDataflowMathAddNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathTwoInputsOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathAddNode::ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const
{
	return (InA + InB);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathSubtractNode::FDataflowMathSubtractNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathTwoInputsOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathSubtractNode::ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const
{
	return (InA - InB);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathMultiplyNode::FDataflowMathMultiplyNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathTwoInputsOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathMultiplyNode::ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const
{
	return (InA * InB);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathDivideNode::FDataflowMathDivideNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathTwoInputsOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
	RegisterInputConnection(&Fallback);
}

double FDataflowMathDivideNode::ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const
{
	if (InB == 0)
	{
		return GetValue(Context, &Fallback);
	}
	return (InA / InB);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathMinimumNode::FDataflowMathMinimumNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathTwoInputsOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathMinimumNode::ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const
{
	return FMath::Min(InA, InB);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathMinimumNode_v2::FDataflowMathMinimumNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Result);

	// Add initial variable inputs
	for (int32 Index = 0; Index < NumInitialVariableInputs; ++Index)
	{
		AddPins();
	}
}

void FDataflowMathMinimumNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Result))
	{
		double MinValue = std::numeric_limits<double>::max();
		for (int32 Idx = 0; Idx < Inputs.Num(); ++Idx)
		{
			const double InputValue = GetValue(Context, GetConnectionReference(Idx));
			MinValue = FMath::Min(InputValue, MinValue);
		}

		SetValue(Context, MinValue, &Result);
	}
}

bool FDataflowMathMinimumNode_v2::CanAddPin() const
{
	return true;
}

bool FDataflowMathMinimumNode_v2::CanRemovePin() const
{
	return Inputs.Num() > 0;
}

UE::Dataflow::TConnectionReference<FDataflowNumericTypes> FDataflowMathMinimumNode_v2::GetConnectionReference(int32 Index) const
{
	return { &Inputs[Index], Index, &Inputs };
}

TArray<UE::Dataflow::FPin> FDataflowMathMinimumNode_v2::AddPins()
{
	const int32 Index = Inputs.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FDataflowMathMinimumNode_v2::GetPinsToRemove() const
{
	const int32 Index = (Inputs.Num() - 1);
	check(Inputs.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FDataflowMathMinimumNode_v2::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = Inputs.Num() - 1;
	check(Inputs.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	Inputs.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FDataflowMathMinimumNode_v2::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		check(Inputs.Num() >= 0);
		// register new elements from the array as inputs
		for (int32 Index = 0; Index < Inputs.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			// if we have more inputs than materials then we need to unregister the inputs 
			const int32 NumVariableInputs = (GetNumInputs() - NumOtherInputs);
			const int32 NumInputs = Inputs.Num();
			if (NumVariableInputs > NumInputs)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				Inputs.SetNum(NumVariableInputs);
				for (int32 Index = NumInputs; Index < Inputs.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				Inputs.SetNum(NumInputs);
			}
		}
		else
		{
			ensureAlways(Inputs.Num() + NumOtherInputs == GetNumInputs());
		}
	}
}

//-----------------------------------------------------------------------------------------------

FDataflowMathMaximumNode::FDataflowMathMaximumNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathTwoInputsOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathMaximumNode::ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const
{
	return FMath::Max(InA, InB);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathMaximumNode_v2::FDataflowMathMaximumNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Result);

	// Add initial variable inputs
	for (int32 Index = 0; Index < NumInitialVariableInputs; ++Index)
	{
		AddPins();
	}
}

void FDataflowMathMaximumNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Result))
	{
		double MaxValue = std::numeric_limits<double>::min();
		for (int32 Idx = 0; Idx < Inputs.Num(); ++Idx)
		{
			const double InputValue = GetValue(Context, GetConnectionReference(Idx));
			MaxValue = FMath::Max(InputValue, MaxValue);
		}

		SetValue(Context, MaxValue, &Result);
	}
}

bool FDataflowMathMaximumNode_v2::CanAddPin() const
{
	return true;
}

bool FDataflowMathMaximumNode_v2::CanRemovePin() const
{
	return Inputs.Num() > 0;
}

UE::Dataflow::TConnectionReference<FDataflowNumericTypes> FDataflowMathMaximumNode_v2::GetConnectionReference(int32 Index) const
{
	return { &Inputs[Index], Index, &Inputs };
}

TArray<UE::Dataflow::FPin> FDataflowMathMaximumNode_v2::AddPins()
{
	const int32 Index = Inputs.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FDataflowMathMaximumNode_v2::GetPinsToRemove() const
{
	const int32 Index = (Inputs.Num() - 1);
	check(Inputs.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FDataflowMathMaximumNode_v2::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = Inputs.Num() - 1;
	check(Inputs.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	Inputs.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FDataflowMathMaximumNode_v2::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		check(Inputs.Num() >= 0);
		// register new elements from the array as inputs
		for (int32 Index = 0; Index < Inputs.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			// if we have more inputs than materials then we need to unregister the inputs 
			const int32 NumVariableInputs = (GetNumInputs() - NumOtherInputs);
			const int32 NumInputs = Inputs.Num();
			if (NumVariableInputs > NumInputs)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				Inputs.SetNum(NumVariableInputs);
				for (int32 Index = NumInputs; Index < Inputs.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				Inputs.SetNum(NumInputs);
			}
		}
		else
		{
			ensureAlways(Inputs.Num() + NumOtherInputs == GetNumInputs());
		}
	}
}

//-----------------------------------------------------------------------------------------------

FDataflowMathReciprocalNode::FDataflowMathReciprocalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
	RegisterInputConnection(&Fallback);
}

double FDataflowMathReciprocalNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	if (InA == 0)
	{
		return GetValue(Context, &Fallback);
	}
	return (1.0 / InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathSquareNode::FDataflowMathSquareNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathSquareNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return (InA * InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathCubeNode::FDataflowMathCubeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathCubeNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return (InA * InA * InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathSquareRootNode::FDataflowMathSquareRootNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathSquareRootNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	if (InA < 0)
	{
		// Context.Error()
		return 0.0;
	}
	return FMath::Sqrt(InA * InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathInverseSquareRootNode::FDataflowMathInverseSquareRootNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
	RegisterInputConnection(&Fallback);
}

double FDataflowMathInverseSquareRootNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	if (InA == 0)
	{
		return GetValue(Context, &Fallback);
	}
	return FMath::InvSqrt(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathNegateNode::FDataflowMathNegateNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathNegateNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return -InA;
}

//-----------------------------------------------------------------------------------------------

FDataflowMathAbsNode::FDataflowMathAbsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathAbsNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Abs(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathFloorNode::FDataflowMathFloorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathFloorNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::FloorToDouble(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathCeilNode::FDataflowMathCeilNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathCeilNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::CeilToDouble(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathRoundNode::FDataflowMathRoundNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathRoundNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::RoundToDouble(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathTruncNode::FDataflowMathTruncNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathTruncNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::TruncToDouble(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathFracNode::FDataflowMathFracNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathFracNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Frac(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathPowNode::FDataflowMathPowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathTwoInputsOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathPowNode::ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const
{
	return FMath::Pow(InA, InB);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathLogXNode::FDataflowMathLogXNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	Base.Value = 10.0; // default is base 10
	RegisterInputsAndOutputs();
	RegisterInputConnection(&Base);
}

double FDataflowMathLogXNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	const double InBase = GetValue(Context, &Base);
	if (InBase <= 0.f)
	{
		return 0.0;
	}
	return FMath::LogX(InBase, InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathLogNode::FDataflowMathLogNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathLogNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Loge(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathExpNode::FDataflowMathExpNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathExpNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Exp(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathSignNode::FDataflowMathSignNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathSignNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Sign(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathOneMinusNode::FDataflowMathOneMinusNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathOneMinusNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return (1.0 - InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathConstantNode::FDataflowMathConstantNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Result);

	// Set the output to double for now so that it is strongly type and easy to connect to the next node
	// Once we can change the output type from the UI, this could be removed 
	SetOutputConcreteType(&Result, TDataflowSingleTypePolicy<float>::TypeName);
}

double FDataflowMathConstantNode::GetConstant() const
{
	switch (Constant)
	{
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_Pi:			return FMathd::Pi;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_HalfPi:		return FMathd::HalfPi;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_TwoPi:			return FMathd::TwoPi;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_FourPi:		return FMathd::FourPi;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_InvPi:			return FMathd::InvPi;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_InvTwoPi:		return FMathd::InvTwoPi;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_Sqrt2:			return FMathd::Sqrt2;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_InvSqrt2:		return FMathd::InvSqrt2;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_Sqrt3:			return FMathd::Sqrt3;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_InvSqrt3:		return FMathd::InvSqrt3;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_E:				return 2.71828182845904523536;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_Gamma:			return 0.577215664901532860606512090082;
	case EDataflowMathConstantsEnum::Dataflow_Math_Constants_GoldenRatio:	return 1.618033988749894;
	default:
		break;
	}
	ensureMsgf(false, TEXT("Unexpected constant enum, returning a zero value. Is it missing from the list above ?"));
	return 0.0;
}

void FDataflowMathConstantNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Result))
	{
		SetValue(Context, GetConstant(), &Result);
	}
}

//-----------------------------------------------------------------------------------------------

FDataflowMathClampNode::FDataflowMathClampNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
	RegisterInputConnection(&Min);
	RegisterInputConnection(&Max);
	Min.Value = 0.0;
	Max.Value = 1.0;
}

double FDataflowMathClampNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	const double InMin = GetValue(Context, &Min);
	const double InMax = GetValue(Context, &Max);
	return FMath::Clamp(InA, InMin, InMax);
}

//--------------------------------------------------------------------------
//
// Trigonometric nodes
//
//--------------------------------------------------------------------------

FDataflowMathSinNode::FDataflowMathSinNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathSinNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Sin(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathCosNode::FDataflowMathCosNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathCosNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Cos(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathTanNode::FDataflowMathTanNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathTanNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Tan(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathArcSinNode::FDataflowMathArcSinNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathArcSinNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Asin(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathArcCosNode::FDataflowMathArcCosNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathArcCosNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Acos(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathArcTanNode::FDataflowMathArcTanNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathArcTanNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::Atan(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathArcTan2Node::FDataflowMathArcTan2Node(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathTwoInputsOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathArcTan2Node::ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const
{
	return FMath::Atan2(InA, InB);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathDegToRadNode::FDataflowMathDegToRadNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathDegToRadNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::DegreesToRadians(InA);
}

//-----------------------------------------------------------------------------------------------

FDataflowMathRadToDegNode::FDataflowMathRadToDegNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowMathOneInputOperatorNode(InParam, InGuid)
{
	RegisterInputsAndOutputs();
}

double FDataflowMathRadToDegNode::ComputeResult(UE::Dataflow::FContext& Context, double InA) const
{
	return FMath::RadiansToDegrees(InA);
}

//-----------------------------------------------------------------------------------------------

