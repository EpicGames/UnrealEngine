// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosLog.h"
#include "CoreMinimal.h"
#include "Dataflow/DataflowConnection.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNode.h"

#include "DataflowMathNodes.generated.h"

//--------------------------------------------------------------------------
//
// Trigonometric nodes
//
//--------------------------------------------------------------------------

#define DATAFLOW_MATH_NODES_CATEGORY "Math|Scalar"

/** One input operators base class */
USTRUCT()
struct FDataflowMathOneInputOperatorNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes A;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes Result;

public:
	FDataflowMathOneInputOperatorNode() {};
	FDataflowMathOneInputOperatorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

protected:
	void RegisterInputsAndOutputs();
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const { ensure(false); return 0.0; };
};

/** Two inputs operators base class */
USTRUCT()
struct FDataflowMathTwoInputsOperatorNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes A;

	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes B;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes Result;

public:
	FDataflowMathTwoInputsOperatorNode() {};
	FDataflowMathTwoInputsOperatorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

protected:
	void RegisterInputsAndOutputs();
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const { ensure(false); return 0.0; };
};

/** Addition (A + B) */
USTRUCT()
struct FDataflowMathAddNode : public FDataflowMathTwoInputsOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathAddNode, "Add", DATAFLOW_MATH_NODES_CATEGORY, "+ Addition Plus")

public:
	FDataflowMathAddNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const override;
};

/** Subtraction (A - B) */
USTRUCT()
struct FDataflowMathSubtractNode : public FDataflowMathTwoInputsOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathSubtractNode, "Subtract", DATAFLOW_MATH_NODES_CATEGORY, "- Subtraction Minus")

public:
	FDataflowMathSubtractNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const override;
};

/** Multiplication (A * B) */
USTRUCT()
struct FDataflowMathMultiplyNode : public FDataflowMathTwoInputsOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathMultiplyNode, "Multiply", DATAFLOW_MATH_NODES_CATEGORY, "* Multiplication Time")

public:
	FDataflowMathMultiplyNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const override;
};

/**
* Division (A / B)
* if B is equal to 0, 0 is returned Fallback value
*/
USTRUCT()
struct FDataflowMathDivideNode : public FDataflowMathTwoInputsOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathDivideNode, "Divide", DATAFLOW_MATH_NODES_CATEGORY, "/ Division")

	UPROPERTY(EditAnywhere, Category = SafeDivide, meta = (DataflowInput));
	FDataflowNumericTypes Fallback;

public:
	FDataflowMathDivideNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const override;
};

/** 
* Minimum ( Min(A, B) ) 
* Deprecated (5.6)
* Use Minimum (V2) with variable number of inputs instead
*/
USTRUCT(meta = (Deprecated = "5.6"))
struct FDataflowMathMinimumNode : public FDataflowMathTwoInputsOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathMinimumNode, "Minimum", DATAFLOW_MATH_NODES_CATEGORY, "Lowest Smallest")

public:
	FDataflowMathMinimumNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const override;
};

/**
* Minimum ( Min(A, B, C, ... ) )
*/
USTRUCT()
struct FDataflowMathMinimumNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathMinimumNode_v2, "Minimum", DATAFLOW_MATH_NODES_CATEGORY, "Lowest Smallest")

private:
	UPROPERTY()
	TArray<FDataflowNumericTypes> Inputs;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes Result;

	//~ Begin FDataflowNode interface
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override;
	virtual bool CanRemovePin() const override;
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	UE::Dataflow::TConnectionReference<FDataflowNumericTypes> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumOtherInputs = 0; // No other inputs
	static constexpr int32 NumInitialVariableInputs = 2;

public:
	FDataflowMathMinimumNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Maximum ( Max(A, B) )
* Deprecated (5.6)
* Use Maximum (V2) with variable number of inputs instead
*/
USTRUCT(meta = (Deprecated = "5.6"))
struct FDataflowMathMaximumNode : public FDataflowMathTwoInputsOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathMaximumNode, "Maximum", DATAFLOW_MATH_NODES_CATEGORY, "Highest Larger")

public:
	FDataflowMathMaximumNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const override;
};

/** Maximum ( Max(A, B, C, ... ) ) */
USTRUCT()
struct FDataflowMathMaximumNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathMaximumNode_v2, "Maximum", DATAFLOW_MATH_NODES_CATEGORY, "Highest Larger")

private:
	UPROPERTY()
	TArray<FDataflowNumericTypes> Inputs;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes Result;

	//~ Begin FDataflowNode interface
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override;
	virtual bool CanRemovePin() const override;
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	UE::Dataflow::TConnectionReference<FDataflowNumericTypes> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumOtherInputs = 0; // No other inputs
	static constexpr int32 NumInitialVariableInputs = 2;

public:
	FDataflowMathMaximumNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/** 
* Reciprocal( 1 / A )
* if A is equal to 0, returns Fallback
*/
USTRUCT()
struct FDataflowMathReciprocalNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathReciprocalNode, "Reciprocal", DATAFLOW_MATH_NODES_CATEGORY, "OneOver OneDividedBy")

	UPROPERTY(EditAnywhere, Category = SafeDivide, meta = (DataflowInput));
	FDataflowNumericTypes Fallback;

public:
	FDataflowMathReciprocalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Square ( A * A ) */
USTRUCT()
struct FDataflowMathSquareNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathSquareNode, "Square", DATAFLOW_MATH_NODES_CATEGORY, "")

public:
	FDataflowMathSquareNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Cube ( A * A * A ) */
USTRUCT()
struct FDataflowMathCubeNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathCubeNode, "Cube", DATAFLOW_MATH_NODES_CATEGORY, "")

public:
	FDataflowMathCubeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Square Root ( sqrt(A) ) */
USTRUCT()
struct FDataflowMathSquareRootNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathSquareRootNode, "SquareRoot", DATAFLOW_MATH_NODES_CATEGORY, "")

public:
	FDataflowMathSquareRootNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** 
* Inverse Square Root ( 1 / sqrt(A) ) 
* if A is equal to 0, returns Fallback
*/
USTRUCT()
struct FDataflowMathInverseSquareRootNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathInverseSquareRootNode, "InverseSquareRoot", DATAFLOW_MATH_NODES_CATEGORY, "OneOverSquareRoot")

	UPROPERTY(EditAnywhere, Category = SafeDivide, meta = (DataflowInput));
	FDataflowNumericTypes Fallback;

public:
	FDataflowMathInverseSquareRootNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Negate ( -A ) */
USTRUCT()
struct FDataflowMathNegateNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathNegateNode, "Negate", DATAFLOW_MATH_NODES_CATEGORY, "Minus Negative")

public:
	FDataflowMathNegateNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Absolute value ( |A| ) */
USTRUCT()
struct FDataflowMathAbsNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathAbsNode, "Abs", DATAFLOW_MATH_NODES_CATEGORY, "Absolute Positive")

public:
	FDataflowMathAbsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Floor ( 1.4 => 1.0 | 1.9 => 1.0 | -5.3 => -6.0 ) */
USTRUCT()
struct FDataflowMathFloorNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathFloorNode, "Floor", DATAFLOW_MATH_NODES_CATEGORY, "Round Lowest Integer")

public:
	FDataflowMathFloorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Ceil ( 1.4 => 2.0 | 1.9 => 2.0 | -5.3 => -5.0) */
USTRUCT()
struct FDataflowMathCeilNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathCeilNode, "Ceil", DATAFLOW_MATH_NODES_CATEGORY, "Round Higher Integer")

public:
	FDataflowMathCeilNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Round ( 1.4 => 1.0 | 1.9 => 2.0 | -5.3 => -5.0) */
USTRUCT()
struct FDataflowMathRoundNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathRoundNode, "Round", DATAFLOW_MATH_NODES_CATEGORY, "Round Mid 0.5")

public:
	FDataflowMathRoundNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Trunc ( 1.4 => 1.0 | 1.9 => 1.0 | -5.3 => -5.0) */
USTRUCT()
struct FDataflowMathTruncNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathTruncNode, "Trunc", DATAFLOW_MATH_NODES_CATEGORY, "Integer Truncate")

public:
	FDataflowMathTruncNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Frac ( 1.4 => 0.4 | 1.9 => 0.9 | -5.3 => 0.3 ) */
USTRUCT()
struct FDataflowMathFracNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathFracNode, "Frac", DATAFLOW_MATH_NODES_CATEGORY, "Fractional Decimal Point")

public:
	FDataflowMathFracNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** power ( A ^ B) */
USTRUCT()
struct FDataflowMathPowNode : public FDataflowMathTwoInputsOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathPowNode, "Pow", DATAFLOW_MATH_NODES_CATEGORY, "Power Exponent")

public:
	FDataflowMathPowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const override;
};

/**
* Log for a specific base ( Log[Base](A) ) 
* If base is negative or zero returns 0
*/
USTRUCT()
struct FDataflowMathLogXNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathLogXNode, "LogX", DATAFLOW_MATH_NODES_CATEGORY, "Logarithm")

	UPROPERTY(EditAnywhere, Category = SafeDivide, meta = (DataflowInput));
	FDataflowNumericTypes Base;

public:
	FDataflowMathLogXNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Natural log ( Log(A) ) */
USTRUCT()
struct FDataflowMathLogNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathLogNode, "Log", DATAFLOW_MATH_NODES_CATEGORY, "Logarithm")

public:
	FDataflowMathLogNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Exponential ( Exp(A) ) */
USTRUCT()
struct FDataflowMathExpNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathExpNode, "Exp", DATAFLOW_MATH_NODES_CATEGORY, "Exponential")

public:
	FDataflowMathExpNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** return -1, 0, +1 whether the input is respectively negative, zero or positive ( Sign(A) ) */
USTRUCT()
struct FDataflowMathSignNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathSignNode, "Sign", DATAFLOW_MATH_NODES_CATEGORY, "Minus Plus Positive Negative")

public:
	FDataflowMathSignNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** One minus (1 - A) */
USTRUCT()
struct FDataflowMathOneMinusNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathOneMinusNode, "OneMinus", DATAFLOW_MATH_NODES_CATEGORY, "")

public:
	FDataflowMathOneMinusNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

UENUM(BlueprintType)
enum class EDataflowMathConstantsEnum : uint8
{
	Dataflow_Math_Constants_Pi			UMETA(DisplayName = "Pi"),
	Dataflow_Math_Constants_HalfPi		UMETA(DisplayName = "HalfPi"),
	Dataflow_Math_Constants_TwoPi		UMETA(DisplayName = "TwoPi"),
	Dataflow_Math_Constants_FourPi		UMETA(DisplayName = "FourPi"),
	Dataflow_Math_Constants_InvPi		UMETA(DisplayName = "InvPi"),
	Dataflow_Math_Constants_InvTwoPi	UMETA(DisplayName = "InvTwoPi"),
	Dataflow_Math_Constants_Sqrt2		UMETA(DisplayName = "Sqrt2"),
	Dataflow_Math_Constants_InvSqrt2	UMETA(DisplayName = "InvSqrt2"),
	Dataflow_Math_Constants_Sqrt3		UMETA(DisplayName = "Sqrt3"),
	Dataflow_Math_Constants_InvSqrt3	UMETA(DisplayName = "InvSqrt3"),
	Dataflow_Math_Constants_E			UMETA(DisplayName = "e"),
	Dataflow_Math_Constants_Gamma		UMETA(DisplayName = "Gamma"),
	Dataflow_Math_Constants_GoldenRatio	UMETA(DisplayName = "GoldenRatio"),
	//~~~
	//256th entry
	Dataflow_Math_Constants_Max UMETA(Hidden)
};

/** Math constants ( see EDataflowMathConstantsEnum ) */
USTRUCT()
struct FDataflowMathConstantNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathConstantNode, "Constants", DATAFLOW_MATH_NODES_CATEGORY, "Pi Half Two Four Inverse SquareRoot Sqrt Cube Square e Gamma Golden Ratio")

	/** Math constant to output */
	UPROPERTY(EditAnywhere, Category = "Constants");
	EDataflowMathConstantsEnum Constant = EDataflowMathConstantsEnum::Dataflow_Math_Constants_Pi;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes Result;

public:
	FDataflowMathConstantNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:
	double GetConstant() const;
};

/** Clamp(A, Min, Max) clamp a value to specific range (inclusive) */
USTRUCT()
struct FDataflowMathClampNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathClampNode, "Clamp", DATAFLOW_MATH_NODES_CATEGORY, "Saturate Limits")

	UPROPERTY(EditAnywhere, Category = "Clamp", meta = (DataflowInput))
	FDataflowNumericTypes Min;

	UPROPERTY(EditAnywhere, Category = "Clamp", meta = (DataflowInput))
	FDataflowNumericTypes Max;

public:
	FDataflowMathClampNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

//--------------------------------------------------------------------------
//
// Trigonometric nodes
//
//--------------------------------------------------------------------------

#define DATAFLOW_MATH_TRIG_NODES_CATEGORY "Math|Trig"

/** Sin(A) with A in radians  */
USTRUCT()
struct FDataflowMathSinNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathSinNode, "Sin", DATAFLOW_MATH_TRIG_NODES_CATEGORY, "Sine Trigonometry Circle Angle Degrees Radians")

public:
	FDataflowMathSinNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Cos(A) with A in radians  */
USTRUCT()
struct FDataflowMathCosNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathCosNode, "Cos", DATAFLOW_MATH_TRIG_NODES_CATEGORY, "Cosine Trigonometry Circle Angle Degrees Radians")

public:
	FDataflowMathCosNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** Tan(A) with A in radians  */
USTRUCT()
struct FDataflowMathTanNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathTanNode, "Tan", DATAFLOW_MATH_TRIG_NODES_CATEGORY, "Tangent Trigonometry Circle Angle Degrees Radians")

public:
	FDataflowMathTanNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** ArcSin(A) returns a value in radians  */
USTRUCT()
struct FDataflowMathArcSinNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathArcSinNode, "ArcSin", DATAFLOW_MATH_TRIG_NODES_CATEGORY, "Sine Trigonometry Circle Angle Degrees Radians")

public:
	FDataflowMathArcSinNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** ArcCos(A) returns a value in radians  */
USTRUCT()
struct FDataflowMathArcCosNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathArcCosNode, "ArcCos", DATAFLOW_MATH_TRIG_NODES_CATEGORY, "Cosine Trigonometry Circle Angle Degrees Radians")

public:
	FDataflowMathArcCosNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** ArcTan(A) returns a value in radians  */
USTRUCT()
struct FDataflowMathArcTanNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathArcTanNode, "ArcTan", DATAFLOW_MATH_TRIG_NODES_CATEGORY, "Tangent Trigonometry Circle Angle Degrees Radians")

public:
	FDataflowMathArcTanNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** ArcTan2(A, B) returns a value in radians  */
USTRUCT()
struct FDataflowMathArcTan2Node : public FDataflowMathTwoInputsOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathArcTan2Node, "ArcTan2", DATAFLOW_MATH_TRIG_NODES_CATEGORY, "Tangent Trigonometry Circle Angle Degrees Radians")

public:
	FDataflowMathArcTan2Node(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA, double InB) const override;
};


/** DegToRad(A) convert degrees to radians */
USTRUCT()
struct FDataflowMathDegToRadNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathDegToRadNode, "DegToRad", DATAFLOW_MATH_TRIG_NODES_CATEGORY, "Trigonometry Circle Angle Degrees Radians Convert Unit")

public:
	FDataflowMathDegToRadNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

/** RadToDeg(A) convert radians to degrees */
USTRUCT()
struct FDataflowMathRadToDegNode : public FDataflowMathOneInputOperatorNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMathRadToDegNode, "RadToDeg", DATAFLOW_MATH_TRIG_NODES_CATEGORY, "Trigonometry Circle Angle Degrees Radians Convert Unit")

public:
	FDataflowMathRadToDegNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual double ComputeResult(UE::Dataflow::FContext& Context, double InA) const override;
};

//--------------------------------------------------------------------------
//
// Other
//
//--------------------------------------------------------------------------
/*
Nodes Left to be converted from the Geometry collection math nodes:
	FFloatMathExpressionDataflowNode);
	FMathExpressionDataflowNode);
	FFitDataflowNode);
	FEFitDataflowNode);
	FLerpDataflowNode);
	FWrapDataflowNode);

	// random 
	FRandomFloatDataflowNode);
	FRandomFloatInRangeDataflowNode);
	FRandomUnitVectorDataflowNode);
	FRandomUnitVectorInConeDataflowNode);
*/

namespace UE::Dataflow
{
	void RegisterDataflowMathNodes();
}
