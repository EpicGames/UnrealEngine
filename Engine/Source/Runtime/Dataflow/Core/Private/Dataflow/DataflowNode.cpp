// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowNode.h"

#include "ChaosLog.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowArchive.h"
#include "Dataflow/DataflowGraph.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "Templates/TypeHash.h"
#include "Dataflow/DataflowNodeFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowNode)

const FName FDataflowNode::DataflowInput = TEXT("DataflowInput");
const FName FDataflowNode::DataflowOutput = TEXT("DataflowOutput");
const FName FDataflowNode::DataflowPassthrough = TEXT("DataflowPassthrough");
const FName FDataflowNode::DataflowIntrinsic = TEXT("DataflowIntrinsic");
const FName FDataflowNode::DataflowSkipConnection = TEXT("DataflowSkipConnection");

const FLinearColor FDataflowNode::DefaultNodeTitleColor = FLinearColor(1.f, 1.f, 0.8f);
const FLinearColor FDataflowNode::DefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);

const FName FDataflowAnyType::TypeName = TEXT("FDataflowAnyType");

#if WITH_EDITOR
FDataflowNodeDelegates::FOnNodeInvalidated FDataflowNodeDelegates::OnNodeInvalidated;
#endif

bool bDataflowEnableGraphEval = true;
FAutoConsoleVariableRef CVARDataflowEnableGraphEval(TEXT("p.Dataflow.EnableGraphEval"), bDataflowEnableGraphEval,
	TEXT("Enable automatic graph evaluation in the Dataflow Editor. [def:true]"));

namespace UE::Dataflow::Private
{
static uint32 GetArrayElementOffsetFromReference(const FArrayProperty* const ArrayProperty, const UE::Dataflow::FConnectionReference& Reference)
{
	check(ArrayProperty);
	if (const void* const AddressAtIndex = ArrayProperty->GetValueAddressAtIndex_Direct(ArrayProperty->Inner, const_cast<void*>(Reference.ContainerReference), Reference.Index))
	{
		check((size_t)Reference.Reference >= (size_t)AddressAtIndex);
		check((int32)((size_t)Reference.Reference - (size_t)AddressAtIndex) < ArrayProperty->Inner->GetElementSize());
		return (uint32)((size_t)Reference.Reference - (size_t)AddressAtIndex);
	}
	return INDEX_NONE;
}

static const FProperty* FindProperty(const UStruct* Struct, const void* StructValue, const void* InProperty, const FName& PropertyName, TArray<const FProperty*>* OutPropertyChain)
{
	const FProperty* Property = nullptr;
	for (FPropertyValueIterator PropertyIt(FProperty::StaticClass(), Struct, StructValue); PropertyIt; ++PropertyIt)
	{
		if (InProperty == PropertyIt.Value() && (PropertyName == NAME_None || PropertyName == PropertyIt.Key()->GetName()))
		{
			Property = PropertyIt.Key();
			if (OutPropertyChain)
			{
				PropertyIt.GetPropertyChain(*OutPropertyChain);
			}
			break;
		}
	}
	return Property;
}

static const FProperty& FindPropertyChecked(const UStruct* Struct, const void* StructValue, const void* InProperty, const FName& PropertyName, TArray<const FProperty*>* OutPropertyChain)
{
	const FProperty* Property = FindProperty(Struct, StructValue, InProperty, PropertyName, OutPropertyChain);
	check(Property);
	return *Property;
}

static FString GetPinToolTipFromProperty(const FProperty* Property)
{
#if WITH_EDITORONLY_DATA
	check(Property);
	if (Property->HasMetaData(TEXT("Tooltip")))
	{
		const FString ToolTipStr = Property->GetToolTipText(true).ToString();
		if (ToolTipStr.Len() > 0)
		{
			TArray<FString> OutArr;
			const int32 NumElems = ToolTipStr.ParseIntoArray(OutArr, TEXT(":\r\n"));

			if (NumElems == 2)
			{
				return OutArr[1];  // Return tooltip meta text
			}
			else if (NumElems == 1)
			{
				return OutArr[0];  // Return doc comment
			}
		}
	}
#endif
	return "";
}

static TArray<FString> GetPinMetaDataFromProperty(const FProperty* Property)
{
	TArray<FString> MetaDataStrArr;
#if WITH_EDITORONLY_DATA
	check(Property);
	if (Property->HasMetaData(FDataflowNode::DataflowPassthrough))
	{
		MetaDataStrArr.Add("Passthrough");
	}
	if (Property->HasMetaData(FDataflowNode::DataflowIntrinsic))
	{
		MetaDataStrArr.Add("Intrinsic");
	}
#endif
	return MetaDataStrArr;
}
};

FDataflowNode::FDataflowNode()
	: Guid(FGuid())
	, Name("Invalid")
	, LastModifiedTimestamp(UE::Dataflow::FTimestamp::Invalid)
{
}

FDataflowNode::FDataflowNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid)
	: Guid(InGuid)
	, Name(Param.Name)
	, LastModifiedTimestamp(UE::Dataflow::FTimestamp::Invalid)
{
	if (IDataflowGraphInterface* Interface = Cast<IDataflowGraphInterface>(Param.OwningObject))
	{
		WeakDataflowGraph = Interface->GetDataflowGraph().ToWeakPtr();
	}
}

FDataflowConnection* FDataflowNode::FindConnection(const UE::Dataflow::FConnectionKey& Key)
{
	FDataflowConnection* Connection = FindInput(Key);
	if (!Connection)
	{
		Connection = FindOutput(Key);
	}
	return Connection;
}

const FDataflowConnection* FDataflowNode::FindConnection(const UE::Dataflow::FConnectionKey& Key) const
{
	const FDataflowConnection* Connection = FindInput(Key);
	if (!Connection)
	{
		Connection = FindOutput(Key);
	}
	return Connection;
}

FDataflowConnection* FDataflowNode::FindConnection(const UE::Dataflow::FConnectionReference& Reference)
{
	FDataflowConnection* Connection = FindInput(Reference);
	if (!Connection)
	{
		Connection = FindOutput(Reference);
	}
	return Connection;
}

//
// Inputs
//

bool FDataflowNode::OutputSupportsType(FName InName, FName InType) const
{
	if (const FDataflowOutput* Output = FindOutput(InName))
	{
		return Output->SupportsType(InType);
	}
	return false;
}

bool FDataflowNode::InputSupportsType(FName InName, FName InType) const
{
	if (const FDataflowInput* Input = FindInput(InName))
	{
		return Input->SupportsType(InType);
	}
	return false;
}

void FDataflowNode::AddInput(FDataflowInput* InPtr)
{
	if (InPtr)
	{
		for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
		{
			const FDataflowInput* const In = Elem.Value;
			ensureMsgf(!In->GetName().IsEqual(InPtr->GetName()), TEXT("Add Input Failed: Existing Node input already defined with name (%s)"), *InPtr->GetName().ToString());
		}

		check(InPtr->GetOwningNode() == this);

		const UE::Dataflow::FConnectionKey Key(InPtr->GetOffset(), InPtr->GetContainerIndex(), InPtr->GetContainerElementOffset());
		if (ensure(!ExpandedInputs.Contains(Key)))
		{
			ExpandedInputs.Add(Key, InPtr);
		}
	}
}

int32 FDataflowNode::GetNumInputs() const
{
	return ExpandedInputs.Num();
}

FDataflowInput* FDataflowNode::FindInput(FName InName)
{
	for (TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
	{
		FDataflowInput* const Con = Elem.Value;
		if (Con->GetName().IsEqual(InName))
		{
			return Con;
		}
	}
	return nullptr;
}

const FDataflowInput* FDataflowNode::FindInput(FName InName) const
{
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
	{
		const FDataflowInput* const Con = Elem.Value;
		if (Con->GetName().IsEqual(InName))
		{
			return Con;
		}
	}
	return nullptr;
}

const FDataflowInput* FDataflowNode::FindInput(const UE::Dataflow::FConnectionKey& Key) const
{
	if (const FDataflowInput* const* Con = ExpandedInputs.Find(Key))
	{
		check(*Con);
		return *Con;
	}
	return nullptr;
}

const FDataflowInput* FDataflowNode::FindInput(const UE::Dataflow::FConnectionReference& Reference) const
{
	const UE::Dataflow::FConnectionKey Key = GetKeyFromReference(Reference);
	if (const FDataflowInput* const Con = FindInput(Key))
	{
		check(Con->RealAddress() == Reference.Reference);
		return Con;
	}
	if (Reference.ContainerReference == nullptr && !InputArrayProperties.IsEmpty())
	{
		// Search through all connections to see if Reference is the RealAddress of an array property.
		for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
		{
			const FDataflowInput* const Con = Elem.Value;
			if (Con->RealAddress() == Reference.Reference)
			{
				return Con;
			}
		}
	}
	return nullptr;
}

FDataflowInput* FDataflowNode::FindInput(const UE::Dataflow::FConnectionKey& Key)
{
	if (FDataflowInput* const* Con = ExpandedInputs.Find(Key))
	{
		check(*Con);
		return *Con;
	}
	return nullptr;
}

FDataflowInput* FDataflowNode::FindInput(const UE::Dataflow::FConnectionReference& Reference)
{
	const UE::Dataflow::FConnectionKey Key = GetKeyFromReference(Reference);
	if (FDataflowInput* const Con = FindInput(Key))
	{
		check(Con->RealAddress() == Reference.Reference);
		return Con;
	}
	if (Reference.ContainerReference == nullptr && !InputArrayProperties.IsEmpty())
	{
		// Search through all connections to see if Reference is the RealAddress of an array property.
		for (TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
		{
			FDataflowInput* const Con = Elem.Value;
			if (Con->RealAddress() == Reference.Reference)
			{
				return Con;
			}
		}
	}
	return nullptr;
}

const FDataflowInput* FDataflowNode::FindInput(const FGuid& InGuid) const
{
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
	{
		const FDataflowInput* const Con = Elem.Value;
		if (Con->GetGuid() == InGuid)
		{
			return Con;
		}
	}
	return nullptr;
}

TArray< FDataflowInput* > FDataflowNode::GetInputs() const
{
	TArray< FDataflowInput* > Result;
	ExpandedInputs.GenerateValueArray(Result);
	return Result;
}

void FDataflowNode::ClearInputs()
{
	for (TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
	{
		FDataflowInput* const Con = Elem.Value;
		delete Con;
	}
	ExpandedInputs.Reset();
	InputArrayProperties.Reset();
}

bool FDataflowNode::HasHideableInputs() const
{
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
	{
		const FDataflowInput* const Con = Elem.Value;
		if (Con->GetCanHidePin())
		{
			return true;
		}
	}
	return false;
}

bool FDataflowNode::HasHiddenInputs() const
{
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
	{
		const FDataflowInput* const Con = Elem.Value;
		if (Con->GetPinIsHidden())
		{
			return true;
		}
	}
	return false;
}

//
// Outputs
//


void FDataflowNode::AddOutput(FDataflowOutput* InPtr)
{
	if (InPtr)
	{
		for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
		{
			const FDataflowOutput* const Out = Elem.Value;
			ensureMsgf(!Out->GetName().IsEqual(InPtr->GetName()), TEXT("Add Output Failed: Existing Node output already defined with name (%s)"), *InPtr->GetName().ToString());
		}

		check(InPtr->GetOwningNode() == this);


		check(InPtr->GetOwningNode() == this);

		const UE::Dataflow::FConnectionKey Key(InPtr->GetOffset(), InPtr->GetContainerIndex(), InPtr->GetContainerElementOffset());
		if (ensure(!ExpandedOutputs.Contains(Key)))
		{
			ExpandedOutputs.Add(Key, InPtr);
		}
	}
}

FDataflowOutput* FDataflowNode::FindOutput(uint32 InGuidHash)
{
	for (TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
	{
		FDataflowOutput* const Con = Elem.Value;
		if (GetTypeHash(Con->GetGuid()) == InGuidHash)
		{
			return Con;
		}
	}
	return nullptr;
}


FDataflowOutput* FDataflowNode::FindOutput(FName InName)
{
	for (TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
	{
		FDataflowOutput* const Con = Elem.Value;
		if (Con->GetName().IsEqual(InName))
		{
			return Con;
		}
	}
	return nullptr;
}

const FDataflowOutput* FDataflowNode::FindOutput(FName InName) const
{
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
	{
		const FDataflowOutput* const Con = Elem.Value;
		if (Con->GetName().IsEqual(InName))
		{
			return Con;
		}
	}
	return nullptr;
}

const FDataflowOutput* FDataflowNode::FindOutput(uint32 InGuidHash) const
{
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
	{
		const FDataflowOutput* const Con = Elem.Value;
		if (GetTypeHash(Con->GetGuid()) == InGuidHash)
		{
			return Con;
		}
	}
	return nullptr;
}

const FDataflowOutput* FDataflowNode::FindOutput(const UE::Dataflow::FConnectionKey& Key) const
{
	if (const FDataflowOutput* const* Con = ExpandedOutputs.Find(Key))
	{
		check(*Con);
		return *Con;
	}
	return nullptr;
}

const FDataflowOutput* FDataflowNode::FindOutput(const UE::Dataflow::FConnectionReference& Reference) const
{
	const UE::Dataflow::FConnectionKey Key = GetKeyFromReference(Reference);
	if (const FDataflowOutput* const Con = FindOutput(Key))
	{
		check(Con->RealAddress() == Reference.Reference);
		return Con;
	}
	if (Reference.ContainerReference == nullptr && !OutputArrayProperties.IsEmpty())
	{
		// Search through all connections to see if Reference is the RealAddress of an array property.
		for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
		{
			FDataflowOutput* const Con = Elem.Value;
			if (Con->RealAddress() == Reference.Reference)
			{
				return Con;
			}
		}
	}
	return nullptr;
}

FDataflowOutput* FDataflowNode::FindOutput(const UE::Dataflow::FConnectionKey& Key)
{
	if (FDataflowOutput* const* Con = ExpandedOutputs.Find(Key))
	{
		check(*Con);
		return *Con;
	}
	return nullptr;
}

FDataflowOutput* FDataflowNode::FindOutput(const UE::Dataflow::FConnectionReference& Reference)
{
	const UE::Dataflow::FConnectionKey Key = GetKeyFromReference(Reference);
	if (FDataflowOutput* const Con = FindOutput(Key))
	{
		check(Con->RealAddress() == Reference.Reference);
		return Con;
	}
	if (Reference.ContainerReference == nullptr && !OutputArrayProperties.IsEmpty())
	{
		// Search through all connections to see if Reference is the RealAddress of an array property.
		for (TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
		{
			FDataflowOutput* Con = Elem.Value;
			if (Con->RealAddress() == Reference.Reference)
			{
				return Con;
			}
		}
	}
	return nullptr;
}

const FDataflowOutput* FDataflowNode::FindOutput(const FGuid& InGuid) const
{
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
	{
		const FDataflowOutput* const Con = Elem.Value;
		if (Con->GetGuid() == InGuid)
		{
			return Con;
		}
	}
	return nullptr;
}

int32 FDataflowNode::NumOutputs() const
{
	return ExpandedOutputs.Num();
}


TArray< FDataflowOutput* > FDataflowNode::GetOutputs() const
{
	TArray< FDataflowOutput* > Result;
	ExpandedOutputs.GenerateValueArray(Result);
	return Result;
}


void FDataflowNode::ClearOutputs()
{
	for (TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
	{
		FDataflowOutput* const Con = Elem.Value;
		delete Con;
	}
	ExpandedOutputs.Reset();
	OutputArrayProperties.Reset();
}

bool FDataflowNode::HasHideableOutputs() const
{
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
	{
		const FDataflowOutput* const Con = Elem.Value;
		if (Con->GetCanHidePin())
		{
			return true;
		}
	}
	return false;
}

bool FDataflowNode::HasHiddenOutputs() const
{
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
	{
		const FDataflowOutput* const Con = Elem.Value;
		if (Con->GetPinIsHidden())
		{
			return true;
		}
	}
	return false;
}

bool FDataflowNode::TryRenameInput(const UE::Dataflow::FConnectionReference& Reference, FName NewName)
{
	if (FDataflowInput* InputToRename = FindInput(Reference))
	{
		return TryRenameInput(*InputToRename, NewName);
	}
	return false;
}

bool FDataflowNode::TryRenameInput(FDataflowInput& InputToRename, FName NewName)
{
	const FDataflowInput* ExistingInputFromName = FindInput(NewName);
	if (ExistingInputFromName == nullptr || ExistingInputFromName == &InputToRename)
	{
		InputToRename.Rename(NewName);
		return true;
	}
	return false;
}

bool FDataflowNode::TryRenameOutput(const UE::Dataflow::FConnectionReference& Reference, FName NewName)
{
	if (FDataflowOutput* OutputToRename = FindOutput(Reference))
	{
		return TryRenameOutput(*OutputToRename, NewName);
	}
	return false;
}

bool FDataflowNode::TryRenameOutput(FDataflowOutput& OutputToRename, FName NewName)
{
	const FDataflowOutput* ExistingOutputFromName = FindOutput(NewName);
	if (ExistingOutputFromName == nullptr || ExistingOutputFromName == &OutputToRename)
	{
		OutputToRename.Rename(NewName);
		return true;
	}
	return false;
}

TArray<UE::Dataflow::FPin> FDataflowNode::GetPins() const
{
	TArray<UE::Dataflow::FPin> RetVal;
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
	{
		const FDataflowInput* const Con = Elem.Value;
		RetVal.Add({ UE::Dataflow::FPin::EDirection::INPUT,Con->GetType(), Con->GetName(), Con->GetPinIsHidden()});
	}
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
	{
		const FDataflowOutput* const Con = Elem.Value;
		RetVal.Add({ UE::Dataflow::FPin::EDirection::OUTPUT,Con->GetType(), Con->GetName(), Con->GetPinIsHidden() });
	}
	return RetVal;
}

void FDataflowNode::UnregisterPinConnection(const UE::Dataflow::FPin& Pin)
{
	if (Pin.Direction == UE::Dataflow::FPin::EDirection::INPUT)
	{
		for (TMap<UE::Dataflow::FConnectionKey, FDataflowInput*>::TIterator Iter = ExpandedInputs.CreateIterator(); Iter; ++Iter)
		{
			FDataflowInput* Con = Iter.Value();
			if (Con->GetName().IsEqual(Pin.Name) && Con->GetType().IsEqual(Pin.Type))
			{
				Iter.RemoveCurrent();
				delete Con;

				// Invalidate graph as this input might have had connections
				Invalidate();
				break;
			}
		}
	}
	else if (Pin.Direction == UE::Dataflow::FPin::EDirection::OUTPUT)
	{
		for (TMap<UE::Dataflow::FConnectionKey, FDataflowOutput*>::TIterator Iter = ExpandedOutputs.CreateIterator(); Iter; ++Iter)
		{
			FDataflowOutput* Con = Iter.Value();
			if (Con->GetName().IsEqual(Pin.Name) && Con->GetType().IsEqual(Pin.Type))
			{
				Iter.RemoveCurrent();
				delete Con;

				// Invalidate graph as this input might have had connections
				Invalidate();
				break;
			}
		}
	}
}

void FDataflowNode::Freeze(UE::Dataflow::FContext& Context)
{
	bIsFrozen = true;
	for (FDataflowOutput* const Output : GetOutputs())
	{
		if (Output && Output->HasConcreteType())
		{
			Output->Freeze(Context, FrozenProperties);
		}
	}
}

void FDataflowNode::Unfreeze(UE::Dataflow::FContext& Context)
{
	FrozenProperties.Reset();
	bIsFrozen = false;

	Invalidate();
}

bool FDataflowNode::IsActive(bool bCheckFlagOnly) const
{
	return bActive && (bCheckFlagOnly || bDataflowEnableGraphEval);
}

void FDataflowNode::Invalidate(const UE::Dataflow::FTimestamp& InModifiedTimestamp)
{
	if (bPauseInvalidations)
	{
		if (PausedModifiedTimestamp < InModifiedTimestamp)
		{
			PausedModifiedTimestamp = InModifiedTimestamp;
		}
		return;
	}
	if (LastModifiedTimestamp < InModifiedTimestamp)
	{
		LastModifiedTimestamp = InModifiedTimestamp;

		if (OnNodeInvalidatedDelegate.IsBound())
		{
			OnNodeInvalidatedDelegate.Broadcast(this);
		}
#if WITH_EDITOR
		if (IsInGameThread() || IsInSlateThread())
		{
			if (FDataflowNodeDelegates::OnNodeInvalidated.IsBound())
			{
				FDataflowNodeDelegates::OnNodeInvalidated.Broadcast(*this);
			}
		}
#endif
		// propagate to downstream
		for (TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
		{
			FDataflowOutput* const Con = Elem.Value;
			Con->Invalidate(InModifiedTimestamp);
		}

		OnInvalidate();
	}
}

bool FDataflowNode::EvaluateNode(UE::Dataflow::FContext& Context) const
{
	// TODO(dataflow) : in the future we should also have the possibility to evaluate the watched output if needed by the context 
	bool bWasEvaluatedOrForwarded = false;
	if (IsActive())
	{
		if (ExpandedOutputs.IsEmpty())
		{
			Evaluate(Context, nullptr);
			bWasEvaluatedOrForwarded = true;
		}
		else if (!IsFrozen())
		{
			for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& OutputByKey : ExpandedOutputs)
			{
				if (const FDataflowOutput* const Output = OutputByKey.Value)
				{
					// todo(ccaillaud) : shoudl we have an option to skip the non connected outputs ? 
					if (!Context.HasData(Output->CacheKey(), GetTimestamp()))
					{
						Evaluate(Context, Output);
						bWasEvaluatedOrForwarded = true;
					}
				}
			}
		}
	}
	else
	{
		for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& OutputByKey : ExpandedOutputs)
		{
			if (const FDataflowOutput* const Output = OutputByKey.Value)
			{
				if (const FDataflowInput* const PassthroughInput = Output->GetPassthroughInput())
				{
					Output->ForwardInput(PassthroughInput, Context);
					bWasEvaluatedOrForwarded = true;
				}
			}
		}
	}
	return bWasEvaluatedOrForwarded;
}

const FProperty* FDataflowNode::FindProperty(const UStruct* Struct, const void* InProperty, const FName& PropertyName, TArray<const FProperty*>* OutPropertyChain) const
{
	return UE::Dataflow::Private::FindProperty(Struct, this, InProperty, PropertyName, OutPropertyChain);
}

const FProperty& FDataflowNode::FindPropertyChecked(const UStruct* Struct, const void* InProperty, const FName& PropertyName, TArray<const FProperty*>* OutPropertyChain) const
{
	return UE::Dataflow::Private::FindPropertyChecked(Struct, this, InProperty, PropertyName, OutPropertyChain);
}

const FProperty* FDataflowNode::FindProperty(const UStruct* Struct, const FName& PropertyFullName, TArray<const FProperty*>* OutPropertyChain) const
{
	// If PropertyFullName corresponds with an array property, it will contain a [ContainerIndex]. We don't care about which element in the array we're in--the FProperty will be the same.
	const FString PropertyFullNameStringIndexNone = StripContainerIndexFromPropertyFullName(PropertyFullName.ToString());

	const FProperty* Property = nullptr;
	for (FPropertyValueIterator PropertyIt(FProperty::StaticClass(), Struct, this); PropertyIt; ++PropertyIt)
	{
		TArray<const FProperty*> PropertyChain;
		PropertyIt.GetPropertyChain(PropertyChain);
		const FString FullName = GetPropertyFullNameString(PropertyChain, INDEX_NONE);
		
		if (PropertyFullNameStringIndexNone.Left(FullName.Len()) != FullName)
		{
			PropertyIt.SkipRecursiveProperty();
			continue;
		}
		if(PropertyFullNameStringIndexNone.Len() == FullName.Len())
		{
			Property = PropertyIt.Key();
			if (OutPropertyChain)
			{
				*OutPropertyChain = MoveTemp(PropertyChain);
			}
			break;
		}
	}
	return Property;
}

uint32 FDataflowNode::GetPropertyOffset(const TArray<const FProperty*>& PropertyChain)
{
	uint32 Offset = 0;
	for (const FProperty* const Property : PropertyChain)
	{
		Offset += (uint32)Property->GetOffset_ForInternal();
	}
	return Offset;
}

uint32 FDataflowNode::GetPropertyOffset(const FName& PropertyFullName) const
{
	uint32 Offset = 0;
	if (const TUniquePtr<const FStructOnScope> ScriptOnStruct =
		TUniquePtr<FStructOnScope>(const_cast<FDataflowNode*>(this)->NewStructOnScope()))  // The mutable Struct Memory is not accessed here, allowing for the const_cast and keeping this method const
	{
		if (const UStruct* const Struct = ScriptOnStruct->GetStruct())
		{
			TArray<const FProperty*> PropertyChain;
			FindProperty(Struct, PropertyFullName, &PropertyChain);
			Offset = GetPropertyOffset(PropertyChain);
		}
	}
	return Offset;
}

uint32 FDataflowNode::GetConnectionOffsetFromReference(const void* Reference) const
{
	return (uint32)((size_t)Reference - (size_t)this);
}

UE::Dataflow::FConnectionKey FDataflowNode::GetKeyFromReference(const UE::Dataflow::FConnectionReference& Reference) const
{
	UE::Dataflow::FConnectionKey Key;
	Key.Offset = Reference.ContainerReference ? GetConnectionOffsetFromReference(Reference.ContainerReference) : GetConnectionOffsetFromReference(Reference.Reference);
	Key.ContainerIndex = Reference.Index;
	Key.ContainerElementOffset = INDEX_NONE;
	if (const FArrayProperty* const* ArrayProperty = InputArrayProperties.Find(Key.Offset))
	{
		Key.ContainerElementOffset = UE::Dataflow::Private::GetArrayElementOffsetFromReference(*ArrayProperty, Reference);
	}
	if (const FArrayProperty* const* ArrayProperty = OutputArrayProperties.Find(Key.Offset))
	{
		Key.ContainerElementOffset = UE::Dataflow::Private::GetArrayElementOffsetFromReference(*ArrayProperty, Reference);
	}
	return Key;
}

FString FDataflowNode::GetPropertyFullNameString(const TConstArrayView<const FProperty*>& PropertyChain, int32 ContainerIndex)
{
	FString PropertyFullName;
	bool bFoundArrayProperty = false;
	for (int32 Index = PropertyChain.Num()-1; Index >= 0 ; --Index)
	{
		const FProperty* const Property = PropertyChain[Index];
		FString PropertyName = Property->GetName();
		if (const FArrayProperty* const ArrayProperty = CastField<FArrayProperty>(Property))
		{
			if (ContainerIndex != INDEX_NONE)
			{
				check(!bFoundArrayProperty); // We only expect to find one array to substitute in.
				bFoundArrayProperty = true;
				PropertyName = FString::Format(TEXT("{0}[{1}]"), { PropertyName, ContainerIndex });
			}

			// Skip the next property. It has the same name as the container (e.g., otherwise you'll get MyFloatArray[5].MyFloatArray)
			--Index;
		}

		PropertyFullName = PropertyFullName.IsEmpty() ?
			PropertyName :
			FString::Format(TEXT("{0}.{1}"), { PropertyFullName, PropertyName });
	}
	return PropertyFullName;
}

FName FDataflowNode::GetPropertyFullName(const TArray<const FProperty*>& PropertyChain, int32 ContainerIndex)
{
	const FString PropertyFullName = GetPropertyFullNameString(TConstArrayView<const FProperty*>(PropertyChain), ContainerIndex);
	return FName(*PropertyFullName);
}

FString FDataflowNode::StripContainerIndexFromPropertyFullName(const FString& InPropertyFullName)
{
	FString PropertyFullName = InPropertyFullName;
	FString PropertyFullNameStripped;

	int32 OpenSquareBracketIndex, CloseSquareBracketIndex;
	while((PropertyFullName.FindChar('[', OpenSquareBracketIndex) && PropertyFullName.FindChar(']', CloseSquareBracketIndex)
		&& OpenSquareBracketIndex < CloseSquareBracketIndex))
	{
		if (CloseSquareBracketIndex > OpenSquareBracketIndex + 1 && PropertyFullName.Mid(OpenSquareBracketIndex + 1, CloseSquareBracketIndex - OpenSquareBracketIndex - 1).IsNumeric())
		{
			// number within brackets. remove it.
			PropertyFullNameStripped += PropertyFullName.Left(OpenSquareBracketIndex);
		}
		else
		{
			// We found some other brackets like [foo] or []. These didn't come from our ContainerIndex. Just leave them and move on.
			PropertyFullNameStripped += PropertyFullName.Left(CloseSquareBracketIndex + 1);
		}
		PropertyFullName.RightChopInline(CloseSquareBracketIndex + 1);
	}
	PropertyFullNameStripped += PropertyFullName;
	return PropertyFullNameStripped;
}

FText FDataflowNode::GetPropertyDisplayNameText(const TArray<const FProperty*>& PropertyChain, int32 ContainerIndex)
{
#if WITH_EDITORONLY_DATA  // GetDisplayNameText() is only available if WITH_EDITORONLY_DATA
	static const FTextFormat TextFormat(NSLOCTEXT("DataflowNode", "PropertyDisplayNameTextConcatenator", "{0}.{1}"));
	FText PropertyText;
	bool bIsPropertyTextEmpty = true;
	bool bFoundArrayProperty = false;
	for (int32 Index = PropertyChain.Num() - 1; Index >= 0; --Index)
	{
		const FProperty* const Property = PropertyChain[Index];
		if (!Property->HasMetaData(FName("SkipInDisplayNameChain")))
		{
			FText PropertyDisplayName = Property->GetDisplayNameText();
			PropertyText = bIsPropertyTextEmpty ?
				MoveTemp(PropertyDisplayName) :
				FText::Format(TextFormat, PropertyText, MoveTemp(PropertyDisplayName));
			bIsPropertyTextEmpty = false;
		}
		if (const FArrayProperty* const ArrayProperty = CastField<FArrayProperty>(Property))
		{
			check(!bFoundArrayProperty); // We only expect to find one array to substitute in.
			bFoundArrayProperty = (ContainerIndex != INDEX_NONE);
			--Index;  // Skip ElemProperty. Otherwise you get names like "MyFloatArray[0].MyFloatArray" when you just want "MyFloatArray[0]"
		}
	}
	if (bFoundArrayProperty)
	{
		static const FTextFormat TextFormatContainer(NSLOCTEXT("DataflowNode", "PropertyDisplayNameTextContainer", "{0}[{1}]"));
		PropertyText = FText::Format(TextFormatContainer, PropertyText, ContainerIndex);
	}

	return PropertyText;
#else
	return FText::FromName(GetPropertyFullName(PropertyChain, ContainerIndex));
#endif
}

void FDataflowNode::InitConnectionParametersFromPropertyReference(const FStructOnScope& StructOnScope, const void* PropertyRef, const FName& PropertyName, UE::Dataflow::FConnectionParameters& OutParams)
{
	const UStruct* Struct = StructOnScope.GetStruct();
	check(Struct);
	TArray<const FProperty*> PropertyChain;
	const FProperty& Property = FindPropertyChecked(Struct, PropertyRef, PropertyName, &PropertyChain);
	check(PropertyChain.Num());

	OutParams.Type = FDataflowConnection::GetTypeNameFromProperty(&Property);
	OutParams.Name = GetPropertyFullName(PropertyChain);
	OutParams.Property = &Property;
	OutParams.Owner = this;
	OutParams.Offset = GetConnectionOffsetFromReference(PropertyRef);
	check(OutParams.Offset == GetPropertyOffset(PropertyChain));
}

FDataflowInput& FDataflowNode::RegisterInputConnectionInternal(const UE::Dataflow::FConnectionReference& Reference, const FName& PropertyName)
{
	TUniquePtr<FStructOnScope> ScriptOnStruct = TUniquePtr<FStructOnScope>(NewStructOnScope());
	check(ScriptOnStruct);
	UE::Dataflow::FInputParameters InputParams;
	InitConnectionParametersFromPropertyReference(*ScriptOnStruct, Reference.Reference, PropertyName, InputParams);
	FDataflowInput* const Input = new FDataflowInput(InputParams);
	check(Input->RealAddress() == Reference.Reference);
	AddInput(Input);
	check(FindInput(Reference) == Input);

#if WITH_EDITORONLY_DATA
	if (Input->GetProperty()->HasMetaData("DataflowIntrinsic"))
	{
		Input->SetIsRequired(true);
	}
#endif

	return *Input;
}

FDataflowInput& FDataflowNode::RegisterInputArrayConnectionInternal(const UE::Dataflow::FConnectionReference& Reference, const FName& ElementPropertyName,
	const FName& ArrayPropertyName)
{
	TUniquePtr<FStructOnScope> ScriptOnStruct = TUniquePtr<FStructOnScope>(NewStructOnScope());
	check(ScriptOnStruct);
	const UStruct* Struct = ScriptOnStruct->GetStruct();
	check(Struct);
	UE::Dataflow::FArrayInputParameters InputParams;
	InputParams.Owner = this;

	// Find the Array property.
	TArray<const FProperty*> ArrayPropertyChain;
	for (FPropertyValueIterator PropertyIt(FArrayProperty::StaticClass(), Struct, this); PropertyIt; ++PropertyIt)
	{
		if (Reference.ContainerReference == PropertyIt.Value() && (ArrayPropertyName == NAME_None || ArrayPropertyName == PropertyIt.Key()->GetName()))
		{
			InputParams.ArrayProperty = CastFieldChecked<FArrayProperty>(PropertyIt.Key());
			InputParams.Offset = GetConnectionOffsetFromReference(Reference.ContainerReference);
			PropertyIt.GetPropertyChain(ArrayPropertyChain);
			break;
		}
	}

	check(InputParams.ArrayProperty);

	// Find the element property
	TArray<const FProperty*> PropertyChain;
	const void* const AddressAtIndex = InputParams.ArrayProperty->GetValueAddressAtIndex_Direct(InputParams.ArrayProperty->Inner, const_cast<void*>(Reference.ContainerReference), Reference.Index);
	if (AddressAtIndex == Reference.Reference && (ElementPropertyName == NAME_None || ElementPropertyName == InputParams.ArrayProperty->Inner->GetName()))
	{
		InputParams.Property = InputParams.ArrayProperty->Inner;
		PropertyChain = { InputParams.ArrayProperty->Inner };
	}
	else if (const FStructProperty* const InnerStruct = CastField<FStructProperty>(InputParams.ArrayProperty->Inner))
	{
		InputParams.Property = &UE::Dataflow::Private::FindPropertyChecked(InnerStruct->Struct, AddressAtIndex, Reference.Reference, ElementPropertyName, &PropertyChain);
		PropertyChain.Add(InnerStruct);
	}

	check(InputParams.Property);

	PropertyChain.Append(ArrayPropertyChain);
	InputParams.Type = FDataflowConnection::GetTypeNameFromProperty(InputParams.Property);
	InputParams.Name = GetPropertyFullName(PropertyChain, Reference.Index);
	InputParams.InnerOffset = UE::Dataflow::Private::GetArrayElementOffsetFromReference(InputParams.ArrayProperty, Reference);

	InputArrayProperties.Emplace(InputParams.Offset, InputParams.ArrayProperty);

	FDataflowInput* const Input = new FDataflowArrayInput(Reference.Index, InputParams);
	AddInput(Input);
	check(FindInput(Reference) == Input);
	return *Input;
}

FDataflowOutput& FDataflowNode::RegisterOutputArrayConnectionInternal(const UE::Dataflow::FConnectionReference& Reference, const FName& ElementPropertyName, const FName& ArrayPropertyName)
{
	TUniquePtr<FStructOnScope> ScriptOnStruct = TUniquePtr<FStructOnScope>(NewStructOnScope());
	check(ScriptOnStruct);
	const UStruct* Struct = ScriptOnStruct->GetStruct();
	check(Struct);
	UE::Dataflow::FArrayOutputParameters OutputParams;
	OutputParams.Owner = this;

	// Find the Array property.
	TArray<const FProperty*> ArrayPropertyChain;
	for (FPropertyValueIterator PropertyIt(FArrayProperty::StaticClass(), Struct, this); PropertyIt; ++PropertyIt)
	{
		if (Reference.ContainerReference == PropertyIt.Value() && (ArrayPropertyName == NAME_None || ArrayPropertyName == PropertyIt.Key()->GetName()))
		{
			OutputParams.ArrayProperty = CastFieldChecked<FArrayProperty>(PropertyIt.Key());
			OutputParams.Offset = GetConnectionOffsetFromReference(Reference.ContainerReference);
			PropertyIt.GetPropertyChain(ArrayPropertyChain);
			break;
		}
	}

	check(OutputParams.ArrayProperty);

	// Find the element property
	TArray<const FProperty*> PropertyChain;
	const void* const AddressAtIndex = OutputParams.ArrayProperty->GetValueAddressAtIndex_Direct(OutputParams.ArrayProperty->Inner, const_cast<void*>(Reference.ContainerReference), Reference.Index);
	if (AddressAtIndex == Reference.Reference && (ElementPropertyName == NAME_None || ElementPropertyName == OutputParams.ArrayProperty->Inner->GetName()))
	{
		OutputParams.Property = OutputParams.ArrayProperty->Inner;
		PropertyChain = { OutputParams.ArrayProperty->Inner };
	}
	else if (const FStructProperty* const InnerStruct = CastField<FStructProperty>(OutputParams.ArrayProperty->Inner))
	{
		OutputParams.Property = &UE::Dataflow::Private::FindPropertyChecked(InnerStruct->Struct, AddressAtIndex, Reference.Reference, ElementPropertyName, &PropertyChain);
		PropertyChain.Add(InnerStruct);
	}

	check(OutputParams.Property);

	PropertyChain.Append(ArrayPropertyChain);
	OutputParams.Type = FDataflowConnection::GetTypeNameFromProperty(OutputParams.Property);
	OutputParams.Name = GetPropertyFullName(PropertyChain, Reference.Index);
	OutputParams.InnerOffset = UE::Dataflow::Private::GetArrayElementOffsetFromReference(OutputParams.ArrayProperty, Reference);

	OutputArrayProperties.Emplace(OutputParams.Offset, OutputParams.ArrayProperty);

	FDataflowOutput* const Output = new FDataflowArrayOutput(Reference.Index, OutputParams);
	AddOutput(Output);
	check(FindOutput(Reference) == Output);
	return *Output;
}

void FDataflowNode::UnregisterInputConnection(const UE::Dataflow::FConnectionKey& Key)
{
	if (ExpandedInputs.RemoveStable(Key))
	{
		// Invalidate graph as this input might have had connections
		Invalidate();
	}
}

void FDataflowNode::UnregisterOutputConnection(const UE::Dataflow::FConnectionKey& Key)
{
	if (ExpandedOutputs.RemoveStable(Key))
	{
		// Invalidate graph as this output might have had connections
		Invalidate();
	}
}

FDataflowOutput& FDataflowNode::RegisterOutputConnectionInternal(const UE::Dataflow::FConnectionReference& Reference, const FName& PropertyName)
{
	TUniquePtr<FStructOnScope> ScriptOnStruct = TUniquePtr<FStructOnScope>(NewStructOnScope());
	check(ScriptOnStruct);
	UE::Dataflow::FOutputParameters OutputParams;
	InitConnectionParametersFromPropertyReference(*ScriptOnStruct, Reference.Reference, PropertyName, OutputParams);
	FDataflowOutput* OutputConnection = new FDataflowOutput(OutputParams);
	check(OutputConnection->RealAddress() == Reference.Reference);

	AddOutput(OutputConnection);
	check(FindOutput(Reference) == OutputConnection);
	check(FindOutput(OutputConnection->GetConnectionKey()) == OutputConnection);
	return *OutputConnection;
}

uint32 FDataflowNode::GetValueHash()
{
	//UE_LOGF(LogChaos, Warning, "%ls", *GetName().ToString())

	uint32 Hash = 0;
	if (const TUniquePtr<FStructOnScope> ScriptOnStruct = TUniquePtr<FStructOnScope>(NewStructOnScope()))
	{
		if (const UStruct* const Struct = ScriptOnStruct->GetStruct())
		{
			for (FPropertyValueIterator PropertyIt(FProperty::StaticClass(), Struct, this); PropertyIt; ++PropertyIt)
			{
				if (const FProperty* const Property = PropertyIt.Key())
				{
					if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
					{
						//
						// Note : [CacheContextPropertySupport]
						// 
						// Some UPROPERTIES do not support hash values.For example, FFilePath, is a struct 
						// that is not defined using USTRUCT, and does not support the GetTypeValue() function.
						// These types of attributes need to return a Zero(0) hash, to indicate that the Hash 
						// is not supported.To add property hashing support, add GetTypeValue to the properties 
						// supporting USTRUCT(See Class.h  UScriptStruct::GetStructTypeHash)
						// 
						if (!StructProperty->Struct) return 0;
						if (!StructProperty->Struct->GetCppStructOps()) return 0;
					}

					if (Property->PropertyFlags & CPF_HasGetValueTypeHash)
					{
						// uint32 CrcHash = FCrc::MemCrc32(PropertyIt.Value(), Property->ElementSize);
						// UE_LOGF(LogChaos, Warning, "( %lu \t%ls", (unsigned long)CrcHash, *Property->GetName())

						if (Property->PropertyFlags & CPF_TObjectPtr)
						{
							// @todo(dataflow) : Do something about TObjectPtr<T>
						}
						else
						{
							Hash = HashCombine(Hash, Property->GetValueTypeHash(PropertyIt.Value()));
						}
					}
				}
			}
		}
	}
	return Hash;
}

void FDataflowNode::ValidateProperties()
{
	if (const TUniquePtr<FStructOnScope> ScriptOnStruct = TUniquePtr<FStructOnScope>(NewStructOnScope()))
	{
		if (const UStruct* const Struct = ScriptOnStruct->GetStruct())
		{
			for (FPropertyValueIterator PropertyIt(FProperty::StaticClass(), Struct, this); PropertyIt; ++PropertyIt)
			{
				if (const FProperty* const Property = PropertyIt.Key())
				{
					if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
					{
						if (!StructProperty->Struct || !StructProperty->Struct->GetCppStructOps())
						{
							// See Note : [CacheContextPropertySupport]
							FString StructPropertyName;
							StructProperty->GetName(StructPropertyName);
							UE_LOGF(LogChaos, Warning, 
								"Dataflow: Context caching disable for graphs with node '%ls' due to non-hashed UPROPERTY '%ls'.", 
								*GetName().ToString(), *StructPropertyName)
						}
					}
				}
			}
		}
	}
}

bool FDataflowNode::ValidateConnections()
{
	bHasValidConnections = true;
#if WITH_EDITORONLY_DATA
	if (const TUniquePtr<FStructOnScope> ScriptOnStruct = TUniquePtr<FStructOnScope>(NewStructOnScope()))
	{
		if (const UStruct* const Struct = ScriptOnStruct->GetStruct())
		{
			for (FPropertyValueIterator PropertyIt(FProperty::StaticClass(), Struct, ScriptOnStruct->GetStructMemory()); PropertyIt; ++PropertyIt)
			{
				const FProperty* const Property = PropertyIt.Key();
				check(Property);
				TArray<const FProperty*> PropertyChain;
				PropertyIt.GetPropertyChain(PropertyChain);
				const FName PropName(GetPropertyFullName(PropertyChain));
				bool bSkipConnectionCheck = false;
				for (const FProperty* const PropertyChainProperty : PropertyChain)
				{
					if (PropertyChainProperty && PropertyChainProperty->HasMetaData(FDataflowNode::DataflowSkipConnection))
					{
						bSkipConnectionCheck = true;
						continue;
					}
				}

				if (bSkipConnectionCheck)
				{
					continue;
				}

				if (Property->HasMetaData(FDataflowNode::DataflowInput))
				{
					if (!FindInput(PropertyIt.Value()))
					{
						UE_LOGF(LogChaos, Warning, "Missing dataflow RegisterInputConnection in constructor for (%ls:%ls)", *GetName().ToString(), *PropName.ToString())
							bHasValidConnections = false;
						ensure(false);
					}
				}
				if (Property->HasMetaData(FDataflowNode::DataflowOutput))
				{
					const FDataflowOutput* const OutputConnection = FindOutput(PropertyIt.Value());
					if(!OutputConnection)
					{
						UE_LOGF(LogChaos, Warning, "Missing dataflow RegisterOutputConnection in constructor for (%ls:%ls)", *GetName().ToString(),*PropName.ToString());
						bHasValidConnections = false;
						ensure(false);
					}

					// If OutputConnection is valid, validate passthrough connections if they exist
					else if (const FString* PassthroughName = Property->FindMetaData(FDataflowNode::DataflowPassthrough))
					{
						// Assume passthrough name is relative to current property name.
						FString FullPassthroughName;
						if (PropertyChain.Num() <= 1)
						{
							FullPassthroughName = *PassthroughName;
						}
						else
						{
							FullPassthroughName = FString::Format(TEXT("{0}.{1}"), { GetPropertyFullNameString(TConstArrayView<const FProperty*>(&PropertyChain[1], PropertyChain.Num() - 1)), *PassthroughName });
						}

						const FDataflowInput* const PassthroughConnectionInput = OutputConnection->GetPassthroughInput();
						if (PassthroughConnectionInput == nullptr)
						{
							UE_LOGF(LogChaos, Warning, "Missing DataflowPassthrough registration for (%ls:%ls)", *GetName().ToString(), *PropName.ToString());
							bHasValidConnections = false;
							ensure(false);

							return false;
						}

						const FDataflowInput* const PassthroughConnectionInputFromMetadata = FindInput(FName(FullPassthroughName));

						if(PassthroughConnectionInput != PassthroughConnectionInputFromMetadata)
						{
							UE_LOGF(LogChaos, Warning, "Mismatch in declared and registered DataflowPassthrough connection; (%ls:%ls vs %ls)", *GetName().ToString(), *FullPassthroughName, *PassthroughConnectionInput->GetName().ToString());
							bHasValidConnections = false;
							ensure(false);
						}

						if(!PassthroughConnectionInputFromMetadata)
						{
							UE_LOGF(LogChaos, Warning, "Incorrect DataflowPassthrough Connection set for (%ls:%ls)", *GetName().ToString(), *PropName.ToString());
							bHasValidConnections = false;
							ensure(false);
						}

						else if(OutputConnection->GetType() != PassthroughConnectionInput->GetType())
						{
							UE_LOGF(LogChaos, Warning, "DataflowPassthrough connection types mismatch for (%ls:%ls)", *GetName().ToString(), *PropName.ToString());
							bHasValidConnections = false;
							ensure(false);
						}
					}
					else if(OutputConnection->GetPassthroughInput()) 
					{
						UE_LOGF(LogChaos, Warning, "Missing DataflowPassthrough declaration for (%ls:%ls)", *GetName().ToString(), *PropName.ToString());
						bHasValidConnections = false;
						ensure(false);
					}
				}
			}

#if 0
		 // disabling this out this for now as this fail all over the place for some dataflow graphs 
		 // we may get rid of the metadata constraints we may not need it anymore ( to be decided later )
			for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& ExpandedInput : ExpandedInputs)
			{
				const FDataflowInput* Input = ExpandedInput.Value;

				if (const FProperty* Property = Input->GetProperty())
				{
					if (!Property->HasMetaData(FDataflowNode::DataflowInput))
					{
						ensure(false);
						UE_LOGF(LogChaos, Warning, "Missing dataflow DataflowInput declaration for (%ls:%ls)", *GetName().ToString(), *Property->GetFName().ToString());
					}
				}
			}

			for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Output : ExpandedOutputs)
			{
				FDataflowOutput* OutputVal = Output.Value;

				if (const FProperty* Property = OutputVal->GetProperty())
				{
					if (!Property->HasMetaData(FDataflowNode::DataflowOutput))
					{
						ensure(false);
						UE_LOGF(LogChaos, Warning, "Missing dataflow DataflowOutput declaration for (%ls:%ls)", *GetName().ToString(), *Property->GetFName().ToString());
					}
				}
			}
#endif	
		}
	}
#endif
	return bHasValidConnections;
}

bool FDataflowNode::ShouldInvalidateOnPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Will be made private in 5.9, use IsColorOverriden() instead.")
	return InPropertyChangedEvent.GetPropertyName() != GET_MEMBER_NAME_CHECKED(FDataflowNode, bOverrideColor) &&
	InPropertyChangedEvent.GetPropertyName() != GET_MEMBER_NAME_CHECKED(FDataflowNode, OverrideColor);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TUniquePtr<const FStructOnScope> FDataflowNode::NewStructOnScopeConst() const
{
	// New StructOnScopeConst is non-const and virtual , changing it would be quite difficult
	// we have to use a const cast though to accomodate this current constraint
	// but we make sure to return a const version of the FStructOnScope 
	return TUniquePtr<const FStructOnScope>(const_cast<FDataflowNode*>(this)->NewStructOnScope());
}

FString FDataflowNode::GetToolTip() const
{
	const UE::Dataflow::FFactoryParameters& FactoryParameters = UE::Dataflow::FNodeFactory::GetInstance()->GetParameters(GetType());
	return FactoryParameters.ToolTip;
}

FText FDataflowNode::GetPinDisplayName(const FName& PropertyFullName, const UE::Dataflow::FPin::EDirection Direction) const
{
	int32 ContainerIndex = INDEX_NONE;

	if (Direction == UE::Dataflow::FPin::EDirection::INPUT)
	{
		if (const FDataflowInput* const Input = FindInput(PropertyFullName))
		{
			ContainerIndex = Input->GetContainerIndex();
		}
	}
	else if (Direction == UE::Dataflow::FPin::EDirection::OUTPUT)
	{
		if (const FDataflowOutput* const Output = FindOutput(PropertyFullName))
		{
			ContainerIndex = Output->GetContainerIndex();
		}
	}

	if (const TUniquePtr<const FStructOnScope> ScriptOnStruct = NewStructOnScopeConst())
	{
		if (const UStruct* const Struct = ScriptOnStruct->GetStruct())
		{
			TArray<const FProperty*> PropertyChain;
			if (FindProperty(Struct, PropertyFullName, &PropertyChain))
			{
				return GetPropertyDisplayNameText(PropertyChain, ContainerIndex);
			}
		}
	}

	return FText();
}

FString FDataflowNode::GetPinToolTip(const FName& PropertyFullName, const UE::Dataflow::FPin::EDirection Direction) const
{
#if WITH_EDITORONLY_DATA
	if (Direction == UE::Dataflow::FPin::EDirection::INPUT)
	{
		if (const FDataflowInput* const Input = FindInput(PropertyFullName))
		{
			if (const FProperty* const Property = Input->GetProperty())
			{
				return UE::Dataflow::Private::GetPinToolTipFromProperty(Property);
			}
		}
	}
	else if (Direction == UE::Dataflow::FPin::EDirection::OUTPUT)
	{
		if (const FDataflowOutput* const Output = FindOutput(PropertyFullName))
		{
			if (const FProperty* const Property = Output->GetProperty())
			{
				return UE::Dataflow::Private::GetPinToolTipFromProperty(Property);
			}
		}
	}
	else if (const TUniquePtr<const FStructOnScope> ScriptOnStruct = NewStructOnScopeConst())
	{
		if (const UStruct* const Struct = ScriptOnStruct->GetStruct())
		{
			if (const FProperty* const Property = FindProperty(Struct, PropertyFullName))
			{
				return UE::Dataflow::Private::GetPinToolTipFromProperty(Property);
			}
		}
	}
#endif

	return {};
}

TArray<FString> FDataflowNode::GetPinMetaData(const FName& PropertyFullName, const UE::Dataflow::FPin::EDirection Direction) const
{
#if WITH_EDITORONLY_DATA
	if (Direction == UE::Dataflow::FPin::EDirection::INPUT)
	{
		if (const FDataflowInput* const Input = FindInput(PropertyFullName))
		{
			if (const FProperty* const Property = Input->GetProperty())
			{
				return UE::Dataflow::Private::GetPinMetaDataFromProperty(Property);
			}
		}
	}
	else if (Direction == UE::Dataflow::FPin::EDirection::OUTPUT)
	{
		if (const FDataflowOutput* const Output = FindOutput(PropertyFullName))
		{
			if (const FProperty* const Property = Output->GetProperty())
			{
				return UE::Dataflow::Private::GetPinMetaDataFromProperty(Property);
			}
		}
	}
	else if (const TUniquePtr<const FStructOnScope> ScriptOnStruct = NewStructOnScopeConst())
	{
		if (const UStruct* const Struct = ScriptOnStruct->GetStruct())
		{
			if (const FProperty* const Property = FindProperty(Struct, PropertyFullName))
			{
				return UE::Dataflow::Private::GetPinMetaDataFromProperty(Property);
			}
		}
	}
#endif

	return TArray<FString>();
}

TSharedPtr<UE::Dataflow::FGraph> FDataflowNode::GetDataflowGraph() const
{
	return WeakDataflowGraph.Pin();
}

void FDataflowNode::ForwardInput(UE::Dataflow::FContext& Context, const UE::Dataflow::FConnectionReference& InputReference, const UE::Dataflow::FConnectionReference& Reference) const
{
	if (const FDataflowOutput* Output = FindOutput(Reference))
	{
		if (const FDataflowInput* Input = FindInput(InputReference))
		{
			// we need to pull the value first so the upstream of the graph evaluate 
			Output->ForwardInput(Input, Context);
		}
		else
		{
			checkfSlow(false, TEXT("This input could not be found within this node, check this has been properly registered in the node constructor"));
		}
	}
	else
	{
		checkfSlow(false, TEXT("This output could not be found within this node, check this has been properly registered in the node constructor"));
	}
}

// DEPRECATED 5.7
void FDataflowNode::SetArraySizeFromInput(UE::Dataflow::FContext& Context, const UE::Dataflow::FConnectionReference& InputReference, const int32* OutputReference) const
{
	int32 OutSize = 0;
	if (const FDataflowInput* ArrayInput = FindInput(InputReference))
	{
		if (ArrayInput->IsConnected())
		{
			// Pull the value to populate the cache
			ArrayInput->PullValue(Context);

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OutSize = Context.GetArraySizeFromData(ArrayInput->GetConnection()->CacheKey());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
	SetValue(Context, OutSize, OutputReference);
}

// DEPRECATED 5.7
void FDataflowNode::SetArrayElementFromInput(UE::Dataflow::FContext& Context, const UE::Dataflow::FConnectionReference& InputReference, int32 Index, const void* OutputReference) const
{
	if (const FDataflowOutput* Output = FindOutput(OutputReference))
	{
		if (const FDataflowInput* ArrayInput = FindInput(InputReference))
		{
			if (ArrayInput->IsConnected())
			{
				// Pull the value to populate the cache
				ArrayInput->PullValue(Context);

				const UE::Dataflow::FContextCacheKey ArrayKey = ArrayInput->GetConnection()->CacheKey();
				const UE::Dataflow::FContextCacheKey ElementKey = Output->CacheKey();

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Context.SetArrayElementFromData(ArrayKey, Index, ElementKey, Output->GetProperty(), Output->GetOwningNodeGuid(), Output->GetOwningNodeValueHash(), Output->GetOwningNodeTimestamp());
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				return;
			}
		}
		// at least return a default value 
		Output->SetNullValue(Context);
	}
}

bool FDataflowNode::TrySetConnectionType(FDataflowConnection* Connection, FName NewType)
{
	if (Connection)
	{
		if (Connection->IsAnyType() && Connection->GetType() != NewType && !FDataflowConnection::IsAnyType(NewType))
		{
			if (SetConnectionConcreteType(Connection, NewType))
			{
				NotifyConnectionTypeChanged(Connection);
				return true;
			}
		}
	}
	return false;
}

void FDataflowNode::NotifyConnectionTypeChanged(FDataflowConnection* Connection)
{
	if (Connection && Connection->IsAnyType())
	{
		OnConnectionTypeChanged(*Connection);
	}
}

FName FDataflowNode::GetDependentConnectionType(const FDataflowConnection& SourceConnection, const FDataflowConnection& DependentConnection) const
{
	// by default return the same type 
	return SourceConnection.GetType();
}

bool FDataflowNode::OnConnectionTypeChanged(const FDataflowConnection& Connection)
{
	// detect if any of the dependent connections is connected
	// because we do not allow connected connectio nto change type
	const FName DependencyGroup = Connection.GetTypeDependencyGroup();
	if (DependencyGroup.IsNone())
	{
		return false; // no changes
	}

	// checks if any connection is connected 
	if (IsAnytypeDependencyConnected(DependencyGroup, &Connection))
	{
		return false; // no changes
	}

	// Change all connection that share the same type dependecy group
	bool bTypeChanged = false;
	ForEachConnection(
		[this, &Connection, &bTypeChanged, &DependencyGroup](FDataflowConnection* OtherConnection)
		{
			if (OtherConnection && (OtherConnection->GetTypeDependencyGroup() == DependencyGroup))
			{
				const FName NewType = GetDependentConnectionType(Connection, *OtherConnection);
				SetConnectionConcreteType(OtherConnection, NewType);
				bTypeChanged = true;
			}
			return true;
		});

	return bTypeChanged;
}

bool FDataflowNode::SetConnectionConcreteType(const UE::Dataflow::FConnectionKey& ConnectionKey, FName NewType)
{
	return SetConnectionConcreteType(FindConnection(ConnectionKey), NewType);
}

bool FDataflowNode::SetInputConcreteType(const UE::Dataflow::FConnectionReference& InputReference, FName NewType)
{
	return SetConnectionConcreteType(FindInput(InputReference), NewType);
}

bool FDataflowNode::SetOutputConcreteType(const UE::Dataflow::FConnectionReference& OutputReference, FName NewType)
{
	return SetConnectionConcreteType(FindOutput(OutputReference), NewType);
}

bool FDataflowNode::SetOutputConnectionConcreteType(FDataflowConnection* Output, FName NewType)
{
	if (Output && Output->GetOwningNode() == this && Output->GetDirection() == UE::Dataflow::FPin::EDirection::OUTPUT)
	{
		return SetConnectionConcreteType(Output, NewType);
	}
	return false;
}

bool FDataflowNode::SetAllConnectionConcreteType(FName NewType)
{
	bool bChanged = false;
	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& InputEntry : ExpandedInputs)
	{
		bChanged |= SetConnectionConcreteType(InputEntry.Value, NewType);
	}

	for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& OutputEntry : ExpandedOutputs)
	{
		bChanged |= SetConnectionConcreteType(OutputEntry.Value, NewType);
	}
	return bChanged;
}

bool FDataflowNode::SetConnectionConcreteType(FDataflowConnection* Connection, FName NewType, FName InTypeDependencyGroup)
{
	bool bSuccess = false;
	if (Connection)
	{
		if (Connection->GetType() != NewType)
		{
			if (TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = GetDataflowGraph())
			{
				// first save connections  and disconnect fomr them 
				TArray<FDataflowConnection*> RemoteConnections;
				Connection->GetConnections(RemoteConnections);
				DataflowGraph->ClearConnections(Connection);

				// try changing the type
				const bool bWasTypeLocked = Connection->IsTypeLocked();
				Connection->UnlockType();

				bSuccess = Connection->SetConcreteType(NewType);

				// we need to set the lock state before reconnecting because the connect logic will use it 
				// to know which side of the connection allows changes
				if (bWasTypeLocked)
				{
					Connection->LockType();
				}

				// now try to reconnect (if the types are no longer compatible, connection will be dropped)
				for (FDataflowConnection* RemoteConnection : RemoteConnections)
				{
					DataflowGraph->Connect(Connection, RemoteConnection);
				}
			}
		}
		if (InTypeDependencyGroup != NAME_None && Connection->GetTypeDependencyGroup() == NAME_None)
		{
			Connection->ForceTypeDependencyGroup(InTypeDependencyGroup);
		}
	}
	return bSuccess;
}

bool FDataflowNode::IsAnytypeDependencyConnected(FName DependencyGroup, const FDataflowConnection* IgnoreConnection) const
{
	if (DependencyGroup.IsNone())
	{
		return false; 
	}

	// first check if any connection is connected 
	bool bIsAnyDependencyConnected = false;
	ForEachConnection(
		[&bIsAnyDependencyConnected, &IgnoreConnection, &DependencyGroup](const FDataflowConnection* Connection)
		{
			// ignore the connection that is actively being notified
			if (Connection
				&& (Connection->GetTypeDependencyGroup() == DependencyGroup)
				&& (IgnoreConnection == nullptr || Connection != IgnoreConnection)
				&& Connection->IsConnected()
				)
			{
				bIsAnyDependencyConnected = true;
				return false; // stop the iteration
			}
			return true;
		});
	return bIsAnyDependencyConnected;
}
