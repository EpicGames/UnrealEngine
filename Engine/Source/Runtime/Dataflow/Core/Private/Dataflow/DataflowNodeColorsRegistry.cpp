// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowNodeColorsRegistry.h"

#include "Dataflow/DataflowNode.h"
#include "Misc/LazySingleton.h"
#include "Styling/AppStyle.h"

namespace UE::Dataflow
{
	FNodeColorsRegistry::FNodeColorsRegistry()
	{
		UDataflowSettings* DataflowSettings = GetMutableDefault<UDataflowSettings>();
		DataflowSettingsChangedDelegateHandle = DataflowSettings->GetOnDataflowSettingsChangedDelegate().AddRaw(this, &FNodeColorsRegistry::NodeColorsChangedInSettings);

		const FNodeColorsMap NodeColorsMap = DataflowSettings->GetNodeColorsMap();

		for (auto& Elem : NodeColorsMap)
		{
			ColorsMap.FindOrAdd(Elem.Key) = Elem.Value;
		}
	}

	FNodeColorsRegistry::~FNodeColorsRegistry()
	{
		if (UObjectInitialized())
		{
			if (IsClassLoaded<UDataflowSettings>())
			{
				UDataflowSettings* DataflowSettings = GetMutableDefault<UDataflowSettings>();
				DataflowSettings->GetOnDataflowSettingsChangedDelegate().Remove(DataflowSettingsChangedDelegateHandle);
			}
		}
	}

	FNodeColorsRegistry& FNodeColorsRegistry::Get()
	{
		return TLazySingleton<FNodeColorsRegistry>::Get();
	}

	void FNodeColorsRegistry::TearDown()
	{
		return TLazySingleton<FNodeColorsRegistry>::TearDown();
	}

	void FNodeColorsRegistry::RegisterNodeColors(const FName& Category, const FNodeColors& NodeColors)
	{
		if (!ColorsMap.Contains(Category))
		{
			ColorsMap.Add(Category, NodeColors);
		}

		// Register colors in DataflowSettings
		GetMutableDefault<UDataflowSettings>()->RegisterColors(Category, NodeColors);
	}

	FLinearColor FNodeColorsRegistry::GetNodeTitleColor(const FName& Category) const
	{
		if (ColorsMap.Contains(Category))
		{
			return ColorsMap[Category].NodeTitleColor;
		}
		else
		{
			// Check if any of the parent category registered
			FString CategoryString = Category.ToString();
			if (CategoryString.Contains(TEXT("|")))
			{
				do {
					FString LfString, RtString;
					if (CategoryString.Split(TEXT("|"), &LfString, &RtString, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
					{
						if (ColorsMap.Contains(FName(*LfString)))
						{
							return ColorsMap[FName(*LfString)].NodeTitleColor;
						}

						CategoryString = LfString;
					}
				} while (CategoryString.Contains(TEXT("|")));
			}
		}
		return FNodeColors().NodeTitleColor;
	}

	FLinearColor FNodeColorsRegistry::GetNodeBodyTintColor(const FName& Category) const
	{
		if (ColorsMap.Contains(Category))
		{
			return ColorsMap[Category].NodeBodyTintColor;
		}
		else
		{
			// Check if any of the parent category registered
			FString CategoryString = Category.ToString();
			if (CategoryString.Contains(TEXT("|")))
			{
				do {
					FString LfString, RtString;
					if (CategoryString.Split(TEXT("|"), &LfString, &RtString, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
					{
						if (ColorsMap.Contains(FName(*LfString)))
						{
							return ColorsMap[FName(*LfString)].NodeBodyTintColor;
						}

						CategoryString = LfString;
					}
				} while (CategoryString.Contains(TEXT("|")));
			}
		}
		return FNodeColors().NodeBodyTintColor;
	}

	void FNodeColorsRegistry::NodeColorsChangedInSettings(const FNodeColorsMap& NodeColorsMap)
	{
		for (auto& Elem : NodeColorsMap)
		{
			ColorsMap.FindOrAdd(Elem.Key) = Elem.Value;
		}
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FPinSettingsRegistry::FPinSettingsRegistry()
	{
		UDataflowSettings* DataflowSettings = GetMutableDefault<UDataflowSettings>();
		DataflowSettingsChangedDelegateHandle = DataflowSettings->GetOnDataflowSettingsChangedPinSettingsDelegate().AddRaw(this, &FPinSettingsRegistry::PinSettingsChangedInSettings);

		const FPinSettingsMap PinColorMap = DataflowSettings->GetPinSettingsMap();

		for (auto& Elem : PinColorMap)
		{
			SettingsMap.FindOrAdd(Elem.Key) = Elem.Value;
		}
	}

	FPinSettingsRegistry::~FPinSettingsRegistry()
	{
		if (UObjectInitialized())
		{
			if (IsClassLoaded<UDataflowSettings>())
			{
				UDataflowSettings* DataflowSettings = GetMutableDefault<UDataflowSettings>();
				DataflowSettings->GetOnDataflowSettingsChangedDelegate().Remove(DataflowSettingsChangedDelegateHandle);
			}
		}
	}

	FPinSettingsRegistry& FPinSettingsRegistry::Get()
	{
		return TLazySingleton<FPinSettingsRegistry>::Get();
	}

	void FPinSettingsRegistry::TearDown()
	{
		return TLazySingleton<FPinSettingsRegistry>::TearDown();
	}

	void FPinSettingsRegistry::RegisterPinSettings(const FName& PinType, const FPinSettings& InSettings)
	{
		if (!SettingsMap.Contains(PinType))
		{
			SettingsMap.Add(PinType, InSettings);
		}

		// Register colors in DataflowSettings
		GetMutableDefault<UDataflowSettings>()->RegisterPinSettings(PinType, InSettings);
	}

	FLinearColor FPinSettingsRegistry::GetPinColor(const FName& PinType) const
	{
		if (SettingsMap.Contains(PinType))
		{
			return SettingsMap[PinType].PinColor;
		}

		return FLinearColor(0.f, 0.f, 0.f);
	}

	float FPinSettingsRegistry::GetPinWireThickness(const FName& PinType) const
	{
		if (SettingsMap.Contains(PinType))
		{
			return SettingsMap[PinType].WireThickness;
		}

		return 1.f;
	}

	void FPinSettingsRegistry::PinSettingsChangedInSettings(const FPinSettingsMap& PinSettingsrMap)
	{
		for (auto& Elem : PinSettingsrMap)
		{
			SettingsMap.FindOrAdd(Elem.Key) = Elem.Value;
		}
	}

	bool FPinSettingsRegistry::IsPinTypeRegistered(const FName& PinType) const
	{
		if (SettingsMap.Contains(PinType))
		{
			return true;
		}

		return false;
	}

	/* --------------------------------------------------------------------------------------------------------------- */

	FIconCategoryRegistry& FIconCategoryRegistry::Get()
	{
		static FIconCategoryRegistry Instance;
		return Instance;
	}

	void FIconCategoryRegistry::RegisterCategoryIcon(const TCHAR* Category, const TCHAR* IconName, const TCHAR* StyleName)
	{
		if (Category && IconName)
		{
			FName CategoryName(Category);
			if (!IconsByCategory.Contains(CategoryName))
			{
				FIconInfo IconInfo;
				IconInfo.IconName = FName(IconName);
				IconInfo.StyleName = StyleName ? FName(StyleName) : FAppStyle::GetAppStyleSetName();
				IconsByCategory.Add(Category, IconInfo);
			}
		}
	}


	FSlateIcon FIconCategoryRegistry::GetCategoryIcon(const FName Category) const
	{
		if (const FIconInfo* IconInfo = IconsByCategory.Find(Category))
		{
			return FSlateIcon(IconInfo->StyleName, IconInfo->IconName);
		}
		return FSlateIcon();
	}
}
