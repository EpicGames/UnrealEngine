// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowNodeFactory.h"

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowCoreNodes.h"
#include "Dataflow/DataflowImageNodes.h"
#include "Dataflow/DataflowMathNodes.h"
#include "Dataflow/DataflowVectorNodes.h"
#include "Dataflow/DataflowConvertNodes.h"
#include "Misc/MessageDialog.h"
#include "Logging/LogMacros.h"
#include "Dataflow/DataflowCoreNodeAndPinTypeColors.h"
#include "Dataflow/DataflowPrimitiveTypes.h"

DEFINE_LOG_CATEGORY_STATIC(LogDataflowFactory, Warning, All);

namespace UE::Dataflow
{
	FNodeFactory* FNodeFactory::Instance = nullptr;

	FNodeFactory::FNodeFactory()
	{
	}

	FNodeFactory::~FNodeFactory()
	{
		delete Instance; 
	}

	FNodeFactory* FNodeFactory::GetInstance()
	{
		if (!Instance)
		{
			Instance = new FNodeFactory();
			Instance->RegisterDefaultNodes();
		}
		return Instance;
	}

	// This part is common for all RegisterNode calls, so share it!
	void FNodeFactory::RegisterNodeStaticInternal(
		UScriptStruct* StaticStruct, FName StaticType, const char* StaticDisplay, const char* StaticCategory,
		const FString& StaticTags, FRawNewNodeFunction CreationFunction
	)
	{
		FNewNodeParameters CreateParams
		{
			.Guid = FGuid::NewGuid(),
			.Type = FName{},
			.Name = StaticType,
			.OwningObject = GetTransientPackage()
		};

		// Create a node and take over ownership temporarily...
		FDataflowNode* DefaultNode = CreationFunction(CreateParams).Release();

		FFactoryParameters FactoryParameters
		{
			.TypeName		= StaticType,
			.DisplayName	= StaticDisplay,
			.Category		= StaticCategory,
			.Tags			= StaticTags,
			.ToolTip		= GetToolTipFromStruct(StaticStruct, StaticType, StaticDisplay),
			.bIsDeprecated	= IsNodeDeprecated(StaticStruct),
			.bIsExperimental= IsNodeExperimental(StaticStruct),
			.NodeVersion	= GetVersionFromTypeName(StaticType),
			.DefaultNodeObject = TSharedPtr<FDataflowNode>(DefaultNode), // ...then transfer ownership to the SharedPtr
		};

		GetInstance()->RegisterNode(FactoryParameters, CreationFunction);
	}

	void FNodeFactory::RegisterDefaultNodes()
	{
		// anytypes need to be at the top as the connection registration in the nodes are using it
		RegisterAnyTypes();
		RegisterPrimitiveTypes();

		RegisterDataflowCoreColors();
				
		RegisterCoreNodes();
		RegisterDataflowMathNodes();
		RegisterDataflowVectorNodes();
		RegisterDataflowImageNodes();
		RegisterDataflowConvertNodes();
	}

	TSharedPtr<FDataflowNode> FNodeFactory::NewNodeFromRegisteredType(FGraph& Graph, const FNewNodeParameters& Param)
	{ 
		if (ClassMap.Contains(Param.Type))
		{
			TUniquePtr<FDataflowNode> NewNode = ClassMap[Param.Type](Param);
			if(NewNode->HasValidConnections())
			{
				return Graph.AddNode(TSharedPtr<FDataflowNode>(NewNode.Release()));
			}
			
			const FText ErrorTitle = FText::FromString("Node Factory");
			const FString ErrorMessageString = FString::Printf(TEXT("Cannot create Node %s. Node Type %s is not well defined."), *NewNode->GetName().ToString(), *NewNode->GetDisplayName().ToString());
			const FText ErrorMessage = FText::FromString(ErrorMessageString);
			FMessageDialog::Debugf(ErrorMessage, ErrorTitle);
		}
		return TSharedPtr<FDataflowNode>(nullptr);
	}

	void FNodeFactory::RegisterNode(const FFactoryParameters& Parameters, FNewNodeFunction NewFunction)
	{
		bool bRegisterNode = true;

		// To specify a new version of a node the node TypeName have to be versioned up in the definition in the header file, for example: FLogStringDataflowNode_v2
		// or leave it without a version number and it will be modified automatically to <...>_V1
		// The DisplayName doesn't have to have a version, just for example: LogString. But it has to be unique for DisplayMap and therefore it automatically 
		// gets modified to NewDisplayName = DisplayName + _v<VERSION>
		// 
		// Update DisplayName with version from TypeName
		FFactoryParameters NewParameters{ Parameters };
		NewParameters.DisplayName = FName(Parameters.DisplayName.ToString() + "_" + Parameters.NodeVersion.ToString());

		if (ClassMap.Contains(NewParameters.TypeName))
		{
			if (ParametersMap[NewParameters.TypeName].DisplayName.IsEqual(NewParameters.DisplayName))
			{
				UE_LOG(LogDataflowFactory, Warning,
					TEXT("Warning : Dataflow node registration mismatch with type(%s).The \
						nodes have inconsistent display names(%s) vs(%s).There are two nodes \
						with the same type being registered."), *NewParameters.TypeName.ToString(),
					*ParametersMap[NewParameters.TypeName].DisplayName.ToString(),
					*NewParameters.DisplayName.ToString());
			}
			if (ParametersMap[NewParameters.TypeName].Category.IsEqual(NewParameters.Category))
			{
				UE_LOG(LogDataflowFactory, Warning,
					TEXT("Warning : Dataflow node registration mismatch with type (%s). The nodes \
						have inconsistent categories names (%s) vs (%s). There are two different nodes \
						with the same type being registered. "), *NewParameters.TypeName.ToString(),
					*ParametersMap[NewParameters.TypeName].DisplayName.ToString(),
					*NewParameters.DisplayName.ToString());
			}
			if (!ClassMap.Contains(NewParameters.TypeName))
			{
				UE_LOG(LogDataflowFactory, Warning,
					TEXT("Warning: Attempted to register node type(%s) with display name (%s) \
						that conflicts with an existing nodes display name (%s)."),
					*NewParameters.TypeName.ToString(), *NewParameters.DisplayName.ToString(),
					*ParametersMap[NewParameters.TypeName].DisplayName.ToString());
			}
		}
		else
		{
			ClassMap.Add(NewParameters.TypeName, NewFunction);
			ParametersMap.Add(NewParameters.TypeName, NewParameters);

			const FName TypeNameNoVersion = GetTypeNameNoVersion(NewParameters.TypeName);
			VersionMap.FindOrAdd(TypeNameNoVersion).AddUnique(NewParameters.TypeName);
		}
	}

	void FNodeFactory::RegisterGetterNodeForAssetType(FName AssetTypeName, FName NodeTypeName)
	{
		if (ensure(!GetterNodesByAssetType.Contains(AssetTypeName)))
		{
			GetterNodesByAssetType.Emplace(AssetTypeName, NodeTypeName);
		}

	}

	FName FNodeFactory::GetGetterNodeFromAssetClass(const UClass& AssetClass) const
	{
		if (const FName* GetterNodeTypeName = GetterNodesByAssetType.Find(AssetClass.GetFName()))
		{
			return *GetterNodeTypeName;
		}
		// serach for compatible types up in the class hierarchy
		if (const UClass* AssetParentClass = AssetClass.GetSuperClass())
		{
			return GetGetterNodeFromAssetClass(*AssetParentClass);
		}
		return {};
	}

	FName FNodeFactory::GetVersionFromTypeName(const FName& TypeName)
	{
		const FString String = TypeName.ToString();
		const int32 Index = String.Find("_v", ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (Index == INDEX_NONE)
		{
			return FName("v1");
		}
		else
		{
			FString VersionString = String.Mid(Index + 1);
			return FName(*VersionString);
		}
	}

	int32 FNodeFactory::GetNumVersionFromVersion(const FName& Version)
	{
		const FString String = Version.ToString();
		const FString NumVersionString = String.Mid(1);
		return FCString::Atoi(*NumVersionString);
	}

	bool FNodeFactory::IsNodeDeprecated(const FName NodeType)
	{
		// Display node deprecated if InNode is deprecated
		if (FNodeFactory* Factory = GetInstance())
		{
			const FFactoryParameters& Param = Factory->GetParameters(NodeType);
			if (Param.IsValid() && Param.IsDeprecated())
			{
				return true;
			}
		}

		return false;
	}

	bool FNodeFactory::IsNodeExperimental(const FName NodeType)
	{
		// Display node experimental if InNode is experimental
		if (FNodeFactory* Factory = GetInstance())
		{
			const FFactoryParameters& Param = Factory->GetParameters(NodeType);
			if (Param.IsValid() && Param.IsExperimental())
			{
				return true;
			}
		}

		return false;
	}

	bool FNodeFactory::IsNodeDeprecated(const UStruct* Struct)
	{
#if WITH_EDITOR
		return Struct->HasMetaData("Deprecated");
#else
		return false;
#endif // WITH_EDITOR
	}

	bool FNodeFactory::IsNodeExperimental(const UStruct* Struct)
	{
#if WITH_EDITOR
		return Struct->HasMetaData("Experimental");
#else
		return false;
#endif // WITH_EDITOR
	}

	const FFactoryParameters& FNodeFactory::GetParameters(FName InTypeName) const
	{
		static FFactoryParameters EmptyFactoryParameters;

		if (ParametersMap.Contains(InTypeName))
		{
			return ParametersMap[InTypeName];
		}

		return EmptyFactoryParameters;
	}

	FName FNodeFactory::GetTypeNameNoVersion(const FName& TypeName)
	{
		FName TypeNameNoVersion = TypeName;

		FString String = TypeName.ToString();
		int32 Index = String.Find("_v", ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (Index != INDEX_NONE)
		{
			FString NoVersionString = String.Mid(0, Index);
			TypeNameNoVersion = FName(*NoVersionString);

			return TypeNameNoVersion;
		}

		return TypeNameNoVersion;
	}

	FName FNodeFactory::GetDisplayNameNoVersion(const FName& DisplayName)
	{
		FName DisplayNameNoVersion = DisplayName;

		FString String = DisplayName.ToString();
		int32 Index = String.Find("_v", ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (Index != INDEX_NONE)
		{
			FString NoVersionString = String.Mid(0, Index);
			DisplayNameNoVersion = FName(*NoVersionString);

			return DisplayNameNoVersion;
		}

		return DisplayNameNoVersion;
	}

	FString FNodeFactory::GetToolTipFromStruct(UScriptStruct* InStruct, const FName& InTypeName, const FName& InDisplayName)
	{
		FString OutStr, InputsStr, OutputsStr;
#if WITH_EDITOR
		FName NodeVersion = GetVersionFromTypeName(InTypeName);
		bool bIsDeprecated = IsNodeDeprecated(InStruct);
		bool bIsExperimental = IsNodeExperimental(InStruct);

		FName NewDisplayName = GetDisplayNameNoVersion(InDisplayName);

		OutStr.Appendf(TEXT("%s (%s)\n"), *NewDisplayName.ToString(), *NodeVersion.ToString());
		if (bIsDeprecated)
		{
			OutStr.Appendf(TEXT("Deprecated\n"));
		}
		if (bIsExperimental)
		{
			OutStr.Appendf(TEXT("Experimental\n"));
		}

		FText StructText = InStruct->GetToolTipText();

		OutStr.Appendf(TEXT("\n%s\n"), *StructText.ToString());

		// Iterate over the properties
		for (const FField* ChildProperty = InStruct->ChildProperties; ChildProperty; ChildProperty = ChildProperty->Next)
		{
			FName PropertyName = ChildProperty->GetFName();

			if (ChildProperty->HasMetaData(TEXT("Tooltip")))
			{
				FString ToolTipStr = ChildProperty->GetToolTipText(true).ToString();

				if (ToolTipStr.Len() > 0)
				{
					TArray<FString> OutArr;
					ToolTipStr.ParseIntoArray(OutArr, TEXT(":\r\n"));

					if (OutArr.Num() == 0)
					{
						break;
					}

					const FString& MainTooltipText = (OutArr.Num() > 1) ? OutArr[1] : OutArr[0];

					if (ChildProperty->HasMetaData(FDataflowNode::DataflowInput) &&
						ChildProperty->HasMetaData(FDataflowNode::DataflowOutput) &&
						ChildProperty->HasMetaData(FDataflowNode::DataflowPassthrough))
					{
						if (ChildProperty->HasMetaData(FDataflowNode::DataflowIntrinsic))
						{
							InputsStr.Appendf(TEXT("    %s [Intrinsic] - %s\n"), *PropertyName.ToString(), *MainTooltipText);
						}
						else
						{
							InputsStr.Appendf(TEXT("    %s - %s\n"), *PropertyName.ToString(), *MainTooltipText);
						}

						OutputsStr.Appendf(TEXT("    %s [Passthrough] - %s\n"), *PropertyName.ToString(), *MainTooltipText);
					}
					else if (ChildProperty->HasMetaData(FDataflowNode::DataflowInput))
					{
						if (ChildProperty->HasMetaData(FDataflowNode::DataflowIntrinsic))
						{
							InputsStr.Appendf(TEXT("    %s [Intrinsic] - %s\n"), *PropertyName.ToString(), *MainTooltipText);
						}
						else
						{
							InputsStr.Appendf(TEXT("    %s - %s\n"), *PropertyName.ToString(), *MainTooltipText);
						}
					}
					else if (ChildProperty->HasMetaData(FDataflowNode::DataflowOutput))
					{
						OutputsStr.Appendf(TEXT("    %s - %s\n"), *PropertyName.ToString(), *MainTooltipText);
					}
				}
			}
		}

		if (InputsStr.Len() > 0)
		{
			OutStr.Appendf(TEXT("\n Input(s) :\n % s"), *InputsStr);
		}

		if (OutputsStr.Len() > 0)
		{
			OutStr.Appendf(TEXT("\n Output(s):\n%s"), *OutputsStr);
		}
#endif // WITH_EDITOR
		return OutStr;
	}

	//
	// Building Context menu
	//
	// If a node is deprecated -> omit from this list
	// If a node has single version -> don't display version
	//
	TArray<FFactoryParameters> FNodeFactory::RegisteredParameters() const
	{
		TArray<FFactoryParameters> RetVal;

		for (auto Elem : VersionMap)
		{
			// Check how many version(s) the node has
			TArray<FFactoryParameters> ParametersArray;
			for (const FName& VersionedTypeName : Elem.Value)
			{
				const FFactoryParameters& FactoryParameters = GetParameters(VersionedTypeName);
				if (!FactoryParameters.IsDeprecated())
				{
					ParametersArray.Add(FactoryParameters);
				}
			}

			if (ParametersArray.Num() == 1)
			{
				// 
				// There is only one version of the node
				// Do not display version in DisplayName
				//
				ParametersArray[0].DisplayName = GetDisplayNameNoVersion(ParametersArray[0].DisplayName);
				RetVal.Add(ParametersArray[0]);
			}
			else if (ParametersArray.Num() > 1)
			{
				// 
				// There are multiple versions of the node
				// DisplayName: DisplayName (v2)
				// TODO: Should this be removed? 
				//       Adding (v2) to the name instead of the menu text could cause mismatched name issues.
				//       Previous versions should already be deprecated anyway (which seems to be the standard practice).
				for (FFactoryParameters& FactoryParameters : ParametersArray)
				{
					const FString NewDisplayName = GetDisplayNameNoVersion(FactoryParameters.DisplayName).ToString() + " (v" + FString::FromInt(GetNumVersionFromVersion(FactoryParameters.NodeVersion)) + ")";
					FactoryParameters.DisplayName = FName(*NewDisplayName);
					RetVal.Add(FactoryParameters);
				}
			}
		}

		return RetVal;
	}

}

