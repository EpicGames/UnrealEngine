// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowNodeParameters.h"

#include "Dataflow/DataflowArchive.h"
#include "Dataflow/DataflowContextCachingFactory.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowTerminalNode.h"
#include "Serialization/Archive.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Modules/ModuleManager.h"
#include "Misc/ScopeLock.h"

namespace UE::Dataflow
{
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FContextPerfData::Reset()
	{
		DataPerOutput.Reset();
	}

	void FContextPerfData::Accumulate(const FDataflowConnection* Connection, uint64 TotalTime, uint64 ExternalTime)
	{
		if (Connection)
		{
			FData& Data = DataPerOutput.FindOrAdd(Connection->GetGuid());
			Data.ExclusiveTimeMs = FMath::Max(Data.ExclusiveTimeMs, (float)FPlatformTime::ToMilliseconds64(TotalTime - ExternalTime));
			Data.InclusiveTimeMs = FMath::Max(Data.InclusiveTimeMs, (float)FPlatformTime::ToMilliseconds64(TotalTime));
			Data.NumCalls++;
			Data.LastTimestamp = Connection->GetOwningNodeTimestamp();
		}
	}

	void FContextPerfData::Append(const FContextPerfData& InPerfData)
	{
		for (const TPair<FGuid, FData>& InEntry: InPerfData.DataPerOutput)
		{
			const FData& InData = InEntry.Value;
			FData& Data = DataPerOutput.FindOrAdd(InEntry.Key);
			Data.ExclusiveTimeMs = FMath::Max(Data.ExclusiveTimeMs, InData.ExclusiveTimeMs);
			Data.InclusiveTimeMs = FMath::Max(Data.InclusiveTimeMs, InData.InclusiveTimeMs);
			Data.NumCalls = FMath::Max(Data.NumCalls, InData.NumCalls);
			Data.LastTimestamp = FMath::Max(Data.LastTimestamp, InData.LastTimestamp);
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void FContextCallstack::Push(const FDataflowConnection* Connection)
	{
		FEntry Entry
		{
			.Connection = Connection,
			.StartTime = FPlatformTime::Cycles64(),
			.ExternalTime = 0,
		};
		Callstack.Push(Entry);
	}

	void FContextCallstack::Pop(const FDataflowConnection* Connection, uint64& OutTotalTime, uint64& OutExternalTime)
	{
		OutTotalTime = 0;
		if (ensure(!Callstack.IsEmpty()))
		{
			FEntry& Entry = Callstack.Top();
			OutTotalTime = FPlatformTime::Cycles64() - Entry.StartTime;
			OutExternalTime = Entry.ExternalTime;

			Callstack.Pop();

			if (Callstack.Num() > 0)
			{
				FEntry& PreviousEntry = Callstack.Top();
				PreviousEntry.ExternalTime += OutTotalTime;
			}
		}
	}

	const FDataflowConnection* FContextCallstack::Top() const
	{
		if (Callstack.IsEmpty())
		{
			return nullptr;
		}
		return Callstack.Top().Connection;
	}

	int32 FContextCallstack::Num() const
	{
		return Callstack.Num();
	}

	const FDataflowConnection* FContextCallstack::operator[](int32 Index)
	{
		return Callstack[Index].Connection;
	}

	bool FContextCallstack::Contains(const FDataflowConnection* Connection) const
	{
		return (Callstack.FindByKey(Connection) != nullptr);
	}

	uint64 FTimestamp::Invalid = 0;
	uint64 FTimestamp::Current() { return FPlatformTime::Cycles64(); }

	//////////////////////////////////////////////////////////////////////

	void FContext::SetDataFromStructView(FContextCacheKey InKey, const FProperty* InProperty, const FConstStructView& StructView, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp)
	{
		TUniquePtr<FContextCacheElementBase> DataStoreEntry = MakeUnique<FContextCacheElementUStruct>(InNodeGuid, InProperty, StructView, InNodeHash, InTimestamp);
		SetDataImpl(InKey, MoveTemp(DataStoreEntry));
	}

	void FContext::SetDataFromStructArrayView(FContextCacheKey InKey, const FProperty* InProperty, const FConstStructArrayView& StructArrayView, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp)
	{
		TUniquePtr<FContextCacheElementBase> DataStoreEntry = MakeUnique<FContextCacheElementUStructArray>(InNodeGuid, InProperty, StructArrayView, InNodeHash, InTimestamp);
		SetDataImpl(InKey, MoveTemp(DataStoreEntry));
	}

	void FContext::SetDataReference(FContextCacheKey Key, const FProperty* Property, FContextCacheKey ReferenceKey, const FTimestamp& InTimestamp)
	{
		// find the reference key to get 
		if (const TUniquePtr<FContextCacheElementBase>* CacheElement = GetDataImpl(ReferenceKey))
		{
			TUniquePtr<FContextCacheElementBase> CacheReferenceElement = (*CacheElement)->CreateReference(ReferenceKey);
			CacheReferenceElement->SetTimestamp(InTimestamp);

			SetDataImpl(Key, MoveTemp(CacheReferenceElement));
		}
		else
		{
			ensure(false); // could not find the original cache element 
		}
	}

	// this is useful when there's a need to have to have cache entry but  the type is not known and there no connected output
	// ( like reroute nodes with unconnected input for example ) 
	// in that case posting an invalid reference, will allow the evaluatino to go through and the node reading it will get a default value instead
	void FContext::SetNullData(FContextCacheKey InKey, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp)
	{
		TUniquePtr<FContextCacheElementNull> CacheNullElement = MakeUnique<FContextCacheElementNull>(InNodeGuid, InProperty, InNodeHash, InTimestamp);
		SetDataImpl(InKey, MoveTemp(CacheNullElement));
	}

	const void* FContext::GetUntypedData(FContextCacheKey Key, const FProperty* InProperty) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = GetDataImpl(Key))
		{
			return (*Cache)->GetUntypedData(*this, InProperty);
		}
		return nullptr;
	}

	bool FContext::HasData(FContextCacheKey Key, FTimestamp InTimestamp) const
	{
		FContextCacheKey IntKey = (FContextCacheKey)Key;
		return HasDataImpl(Key, InTimestamp);
	}

	bool FContext::IsEmpty() const
	{
		return IsEmptyImpl();
	}

	void FContext::Serialize(FArchive& Ar)
	{
		FTimestamp Timestamp = FTimestamp::Invalid;
		Ar << Timestamp;
		Ar << DataStore;
	}

	const TUniquePtr<FContextCacheElementBase>* FContext::FindCacheElement(FContextCacheKey InKey) const
	{
		return GetDataImpl(InKey);
	}

	bool FContext::HasCacheElement(FContextCacheKey InKey, FTimestamp InTimestamp) const
	{
		return HasDataImpl(InKey, InTimestamp);
	}

	void FContext::SetThreaded(bool bValue)
	{
		if (bValue != IsThreaded())
		{
			if (bValue)
			{
				DataLock = MakeUnique<FCriticalSection>();
			}
			else
			{
				DataLock.Reset();
			}
		}
	}

	bool FContext::IsAsyncEvaluating() const
	{
		return IsThreaded() && (AsyncEvaluator.GetNumRunningTasks() > 0);
	}

	void FContext::CancelAsyncEvaluation()
	{
		AsyncEvaluator.Cancel();
	}

	void FContext::GetAsyncEvaluationStats(int32& OutNumPendingTasks, int32& OutNumRunningTasks, int32& OutNumCompletedTasks) const
	{
		AsyncEvaluator.GetStats(OutNumPendingTasks, OutNumRunningTasks, OutNumCompletedTasks);
	}

	int32 FContext::GetKeys(TSet<FContextCacheKey>& InKeys) const
	{ 
		FScopedOptionalLock Lock(DataLock.Get());
		return DataStore.GetKeys(InKeys);
	}

	void FContext::SetDataImpl(FContextCacheKey Key, TUniquePtr<FContextCacheElementBase>&& DataStoreEntry)
	{
		FScopedOptionalLock Lock(DataLock.Get());
		DataStore.Emplace(Key, MoveTemp(DataStoreEntry));
	}

	const TUniquePtr<FContextCacheElementBase>* FContext::GetDataImpl(FContextCacheKey Key) const
	{
		FScopedOptionalLock Lock(DataLock.Get());
		return DataStore.Find(Key);
	}

	bool FContext::HasDataImpl(FContextCacheKey Key, FTimestamp InTimestamp) const
	{
		FScopedOptionalLock Lock(DataLock.Get());
		return DataStore.Contains(Key) && DataStore[Key]->GetTimestamp() >= InTimestamp;
	}

	bool FContext::IsEmptyImpl() const
	{
		FScopedOptionalLock Lock(DataLock.Get());
		return DataStore.IsEmpty();
	}

	void FContext::ClearAllData()
	{
		FScopedOptionalLock Lock(DataLock.Get());
		DataStore.Reset();
	}

	FTimestamp FContext::GetTimestamp(FContextCacheKey Key) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* const Cache = const_cast<FContext*>(this)->GetDataImpl(Key))
		{
			return (*Cache)->GetTimestamp();
		}
		return FTimestamp::Invalid;
	}

	const FContextPerfData& FContext::GetPerfData() const 
	{ 
#if DATAFLOW_EDITOR_EVALUATION
		return PerfData; 
#else
		static const FContextPerfData Default;
		return Default;
#endif
	}

	FContextPerfData::FData FContext::GetPerfDataForNode(const FDataflowNode& Node) const
	{
		FContextPerfData::FData NodeData;
#if DATAFLOW_EDITOR_EVALUATION
		for (FDataflowOutput* Output : Node.GetOutputs())
		{
			if (Output)
			{
				if (const FContextPerfData::FData* OutputData = PerfData.DataPerOutput.Find(Output->GetGuid()))
				{
					NodeData.ExclusiveTimeMs += OutputData->ExclusiveTimeMs;
					NodeData.InclusiveTimeMs += OutputData->InclusiveTimeMs;
					NodeData.NumCalls = FMath::Max(NodeData.NumCalls, OutputData->NumCalls);
				}
			}
		}
#endif
		NodeData.LastTimestamp = Node.GetTimestamp();
		return NodeData;
	}

	void FContext::ResetPerfDataForNode(const FDataflowNode& Node)
	{
#if DATAFLOW_EDITOR_EVALUATION
		for (FDataflowOutput* Output : Node.GetOutputs())
		{
			if (Output)
			{
				if (const FContextPerfData::FData* OutputData = PerfData.DataPerOutput.Find(Output->GetGuid()))
				{
					if (Node.GetTimestamp().Value > OutputData->LastTimestamp.Value)
					{
						PerfData.DataPerOutput.Remove(Output->GetGuid());
					}
				}
			}
		}
#endif
	}

	void FContext::AddExternalPerfData(const FContextPerfData& InPerfData)
	{
#if DATAFLOW_EDITOR_EVALUATION
		PerfData.Append(InPerfData);
#endif
	}

	void FContext::ClearAllPerfData()
	{
#if DATAFLOW_EDITOR_EVALUATION
		PerfData.Reset();
#endif
	}

	void FContext::EnablePerfData(bool bEnable)
	{
#if DATAFLOW_EDITOR_EVALUATION
		PerfData.bEnabled = bEnable;
#endif
	}

	bool FContext::IsPerfDataEnabled() const
	{
#if DATAFLOW_EDITOR_EVALUATION
		return PerfData.bEnabled;
#else
		return false;
#endif
	}


	void FContext::PushToCallstack(const FDataflowConnection* Connection)
	{
#if DATAFLOW_EDITOR_EVALUATION
		Callstack.Push(Connection);
#endif
	}

	void FContext::PopFromCallstack(const FDataflowConnection* Connection)
	{
#if DATAFLOW_EDITOR_EVALUATION
		ensure(Connection == Callstack.Top());
		uint64 TotalTime = 0;
		uint64 ExternalTime = 0;
		Callstack.Pop(Connection, TotalTime, ExternalTime);
		PerfData.Accumulate(Connection, TotalTime, ExternalTime);
#endif
	}

	bool FContext::IsInCallstack(const FDataflowConnection* Connection) const
	{
#if DATAFLOW_EDITOR_EVALUATION
		return Callstack.Contains(Connection);
#else
		return false;
#endif
	}

	bool FContext::IsCacheEntryAfterTimestamp(FContextCacheKey InKey, const FTimestamp InTimestamp) const
	{
		if (HasData(InKey))
		{
			if (const TUniquePtr<FContextCacheElementBase>* const CacheEntry = GetDataImpl(InKey))
			{
				if (*CacheEntry)
				{
					if ((*CacheEntry)->GetTimestamp() >= InTimestamp)
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	void FContext::Info(const FText& InInfo, const FDataflowNode* InNode, const FDataflowOutput* InOutput)
	{
		BroadCastLog(EMessageSeverity::Info, InInfo, InNode, InOutput);
	}

	void FContext::Info(const FString& InInfo, const FDataflowNode* InNode, const FDataflowOutput* InOutput)
	{
		FText Message = FText::FromString(InInfo);
		Info(Message, InNode, InOutput);
	}

	void FContext::Warning(const FText& InWarning, const FDataflowNode* InNode, const FDataflowOutput* InOutput)
	{
		BroadCastLog(EMessageSeverity::Warning, InWarning, InNode, InOutput);
	}

	void FContext::Warning(const FString& InWarning, const FDataflowNode* InNode, const FDataflowOutput* InOutput)
	{
		FText Message = FText::FromString(InWarning);
		Warning(Message, InNode, InOutput);
	}

	void FContext::Error(const FText& InError, const FDataflowNode* InNode, const FDataflowOutput* InOutput)
	{
		BroadCastLog(EMessageSeverity::Error, InError, InNode, InOutput);
	}

	void FContext::Error(const FString& InError, const FDataflowNode* InNode, const FDataflowOutput* InOutput)
	{
		FText Message = FText::FromString(InError);
		Error(Message, InNode, InOutput);
	}

	void FContext::BroadCastLog(EMessageSeverity::Type Severity, const FText& InMessage, const FDataflowNode* InNode, const FDataflowOutput* InOutput)
	{
		FLogMessage LogMessage
		{
			.Severity = Severity,
			.Node = InNode ? InNode->AsWeak() : TWeakPtr<const FDataflowNode>{},
			.OutputGuid = (InOutput) ? InOutput->GetGuid() : FGuid(),
			.Text = InMessage,
		};

		auto BroadCastLogOnGameThread =
			[WeakDelegates = Delegates.ToWeakPtr(), Message = MoveTemp(LogMessage)]()
			{
				check(IsInGameThread());
				if (TSharedPtr<FDelegates> Delegates = WeakDelegates.Pin())
				{
					if (Delegates->OnContextLog.IsBound())
					{
						Delegates->OnContextLog.Broadcast(Message);
					}
				}
			};

		if (IsInGameThread())
		{
			BroadCastLogOnGameThread();
		}
		else
		{
			Async(EAsyncExecution::TaskGraphMainThread, MoveTemp(BroadCastLogOnGameThread));
		}
	}

	FContextScopedCallstack::FContextScopedCallstack(FContext& InContext, const FDataflowConnection* InConnection)
		: Context(InContext)
		, Connection(InConnection)
	{
		bLoopDetected = Context.IsInCallstack(Connection);
		Context.ResetPerfDataForNode(*Connection->GetOwningNode());
		Context.PushToCallstack(Connection);
	}

	FContextScopedCallstack::~FContextScopedCallstack()
	{
		Context.PopFromCallstack(Connection);
	}

	FContext::FContext() 
		: AsyncEvaluator(*this) 
		, Delegates(MakeShared<FDelegates>())
	{}

	// DEPRECATED 5.7
	int32 FContext::GetArraySizeFromData(const FContextCacheKey InKey) const
	{
		// Query the size without knowning the type ( the cache elements do know it )
		const TUniquePtr<UE::Dataflow::FContextCacheElementBase>* CacheEntryPr = GetDataImpl(InKey);
		if (CacheEntryPr && CacheEntryPr->IsValid())
		{
			return (*CacheEntryPr)->GetNumArrayElements(*this);
		}
		return 0;
	}

	// DEPRECATED 5.7
	void FContext::SetArrayElementFromData(const FContextCacheKey InArrayKey, int32 Index, const FContextCacheKey InElementKey, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp)
	{
		// get the element from the cache 
		const TUniquePtr<UE::Dataflow::FContextCacheElementBase>* CacheEntryPr = GetDataImpl(InArrayKey);
		if (CacheEntryPr && CacheEntryPr->IsValid())
		{
			TUniquePtr<UE::Dataflow::FContextCacheElementBase> ArrayElementCacheEntry = (*CacheEntryPr)->CreateFromArrayElement(*this, Index, InProperty, InNodeGuid, InNodeHash, InTimestamp);
			if (ArrayElementCacheEntry.IsValid())
			{
				SetDataImpl(InElementKey, MoveTemp(ArrayElementCacheEntry));
				return;
			}
		}
		// fallback 
		SetNullData(InElementKey, InProperty, InNodeGuid, InNodeHash, InTimestamp);
	}

	bool FContext::CopyDataToAnotherContext(const FContextCacheKey InSourceKey, FContext& TargetContext, const FContextCacheKey InTargetKey, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const
	{
		const TUniquePtr<UE::Dataflow::FContextCacheElementBase>* CacheEntryToClone = GetDataImpl(InSourceKey);
		if (CacheEntryToClone && CacheEntryToClone->IsValid())
		{
			TUniquePtr<UE::Dataflow::FContextCacheElementBase> ClonedCacheEntry = (*CacheEntryToClone)->Clone(*this);
			if (ClonedCacheEntry)
			{
				ClonedCacheEntry->UpdatePropertyAndNodeData(InProperty, InNodeGuid, InNodeHash, InTimestamp);
				TargetContext.SetDataImpl(InTargetKey, MoveTemp(ClonedCacheEntry));
			}
			return true;
		}
		return false;
	}

	void FContext::BeginContextEvaluation(const FDataflowNode * Node, const FDataflowOutput * Output)
	{
		if (Output)
		{
			Evaluate(*Output);
		}
		else if (Node)
		{
			if (Node->NumOutputs())
			{
				for (const FDataflowOutput* const NodeOutput : Node->GetOutputs())
				{
					Evaluate(*NodeOutput);
				}
			}
			// Note: If the node is deactivated and has an output (like above), then the output might still need to be forwarded.
			//       Therefore the Evaluate method has to be called for whichever value of bActive.
			//       However if the node is deactivated and has no outputs (like below), now is the time to check its bActive state.
			else if (Node->IsActive() && !Node->IsFrozen())
			{
				// TODO: When no outputs are specified, this call to Evaluate should really be removed.
				//       The purpose of the node evaluation function is to evaluate outputs.
				//       Therefore if a node has no outputs, then it shouldn't need any evaluation.

				if (!IsThreaded() && OnNodeBeginEvaluateMulticast.IsBound())
				{
					OnNodeBeginEvaluateMulticast.Broadcast(Node, Output);
				}

				UE_LOGF(LogChaosDataflow, Verbose, "FDataflowNode::Evaluate(): Node [%ls], Output [nullptr], NodeTimestamp [%llu]", *Node->GetName().ToString(), Node->GetTimestamp().Value);
				Node->Evaluate(*this, nullptr);

				if (!IsThreaded() && OnNodeFinishEvaluateMulticast.IsBound())
				{
					OnNodeFinishEvaluateMulticast.Broadcast(Node, Output);
				}
			}
		}
		else
		{
			ensureMsgf(false, TEXT("Invalid arguments, either Node or Output needs to be non null."));
		}
	}

	void FContext::Evaluate(const FDataflowNode* Node, const FDataflowOutput* Output, FOnPostEvaluationFunction PostEvaluationFunction)
	{
		if (IsThreaded())
		{
			if (Output)
			{
				AsyncEvaluator.ScheduleOutputEvaluation(*Output, PostEvaluationFunction);
			}
			else if (Node)
			{
				AsyncEvaluator.ScheduleNodeEvaluation(*Node, PostEvaluationFunction);
			}
		}
		else
		{
			Evaluate(Node, Output);
			if (PostEvaluationFunction.IsSet())
			{
				PostEvaluationFunction(*this);
			}
		}
	}

	void FContext::Evaluate(const FDataflowNode* Node, const FDataflowOutput* Output)
	{
		ensureMsgf(!IsThreaded(), TEXT(
			"WARNING: Trying to use no - asynchronous evaluation method in a threaded context "
			"The node outputs will not be up to date when returning from this method"
			"Please use the EValuation method that takes a PostEvaluation function"
		));
		if (IsThreaded())
		{
			Evaluate(Node, Output, {});
		}
		else
		{
			BeginContextEvaluation(Node, Output);
		}
	}

	bool FContext::Evaluate(const FDataflowOutput& Connection)
	{
		if (IsThreaded())
		{
			if (!Connection.HasCachedValue(*this))
			{
				Connection.SetNullValue(*this);
			}
			return true;
		}
		else
		{
			if (OnNodeBeginEvaluateMulticast.IsBound())
			{
				OnNodeBeginEvaluateMulticast.Broadcast(Connection.GetOwningNode(), &Connection);
			}

			UE_LOGF(LogChaosDataflow, VeryVerbose, "FContext::Evaluate(): Node [%ls], Output [%ls]", *Connection.GetOwningNode()->GetName().ToString(), *Connection.GetName().ToString());
			bool RetValue = false;
			{
				FScopedOptionalLock Lock(IsThreaded() ? Connection.OutputLock.Get() : nullptr);
				RetValue = Connection.EvaluateImpl(*this);
			}
			if (OnNodeFinishEvaluateMulticast.IsBound())
			{
				OnNodeFinishEvaluateMulticast.Broadcast(Connection.GetOwningNode(), &Connection);
			}

			/*const bool bHasError = NodeHasError(Connection.GetOwningNode());

			return !bHasError && RetValue;*/
			return RetValue;
		}
	}

	UObject* FContext::AddAsset(const FString& AssetPath, const UClass* AssetClass)
	{
		return AssetStore.AddAsset(AssetPath, AssetClass);
	}

	UObject* FContext::CommitAsset(const FString& AssetPath)
	{
		return AssetStore.CommitAsset(AssetPath);
	}

	void FContext::ClearAssets()
	{
		AssetStore.ClearAssets();
	}

	void FContextCache::Serialize(FArchive& Ar)
	{
		if (Ar.IsSaving())
		{

			const int64 NumElementsSavedPosition = Ar.Tell();
			int64 NumElementsWritten = 0;
			Ar << NumElementsWritten;

			for (TPair<FContextCacheKey, TUniquePtr<FContextCacheElementBase>>& Elem : Pairs)
			{
				// note : we only serialize typed cache element and ignore the reference ones ( since they don't hold data per say )
				// Also UObject pointers aren't serialized, as there are no ways to differentiate the objects owned by the cache 
				// from the ones own by any other owners for now.
				if (Elem.Value && Elem.Value->Property && Elem.Value->Type == FContextCacheElementBase::EType::CacheElementTyped)
				{
					FProperty* Property = (FProperty*)Elem.Value->Property;
					FName TypeName = FDataflowConnection::GetTypeNameFromProperty(Property);
					FGuid NodeGuid = Elem.Value->NodeGuid;
					uint32 NodeHash = Elem.Value->NodeHash;

					if (FContextCachingFactory::GetInstance()->Contains(TypeName))
					{
						Ar << TypeName << Elem.Key << NodeGuid << NodeHash << Elem.Value->Timestamp;

						DATAFLOW_OPTIONAL_BLOCK_WRITE_BEGIN()
						{
							FContextCachingFactory::GetInstance()->Serialize(Ar, {TypeName, NodeGuid, Elem.Value.Get(), NodeHash, Elem.Value->Timestamp});
						}
						DATAFLOW_OPTIONAL_BLOCK_WRITE_END();

						NumElementsWritten++;
					}
				}
			}


			if (NumElementsWritten)
			{
				const int64 FinalPosition = Ar.Tell();
				Ar.Seek(NumElementsSavedPosition);
				Ar << NumElementsWritten;
				Ar.Seek(FinalPosition);
			}
		}
		else if (Ar.IsLoading())
		{
			int64 NumElementsWritten = 0;
			Ar << NumElementsWritten;
			for (int i = NumElementsWritten; i > 0; i--)
			{
				FName TypeName;
				FGuid NodeGuid;
				uint32 NodeHash;
				FContextCacheKey InKey;
				FTimestamp Timestamp = FTimestamp::Invalid;

				Ar << TypeName << InKey << NodeGuid << NodeHash << Timestamp;

				DATAFLOW_OPTIONAL_BLOCK_READ_BEGIN(FContextCachingFactory::GetInstance()->Contains(TypeName))
				{
					FContextCacheElementBase* NewElement = FContextCachingFactory::GetInstance()->Serialize(Ar, { TypeName, NodeGuid, nullptr, NodeHash, Timestamp });
					check(NewElement);
					NewElement->NodeGuid = NodeGuid;
					NewElement->NodeHash = NodeHash;
					NewElement->Timestamp = Timestamp;
					Add(InKey, TUniquePtr<FContextCacheElementBase>(NewElement));
				}
				DATAFLOW_OPTIONAL_BLOCK_READ_ELSE()
				{
				}
				DATAFLOW_OPTIONAL_BLOCK_READ_END();
			}
		}
	}
};

