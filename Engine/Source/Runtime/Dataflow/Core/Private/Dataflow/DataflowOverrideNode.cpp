// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowOverrideNode.h"

#include "ChaosLog.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowArchive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowOverrideNode)

template<class T>
static const T FindOverrideMapProperty(const UObject* Owner, FName PropertyName, FName ArrayKey, const T& Default = T())
{
	if (Owner && Owner->GetClass())
	{
		if (FMapProperty* MapProperty = FindFProperty<FMapProperty>(Owner->GetClass(), PropertyName))
		{
			if (const TMap<FString, FString>* ContainerData = MapProperty->ContainerPtrToValuePtr<TMap<FString, FString>>(Owner))
			{
				const FString Key = ArrayKey.ToString();
				if (ContainerData->Contains(Key))
				{
					return (FString)(*(ContainerData->Find(Key)));
				}
			}
		}
	}
	return Default;
}

bool FDataflowOverrideNode::ShouldInvalidate(FName InKey) const
{
	if (IsConnected<FName>(&Key))
	{
		return true;
	}

	return InKey == Key;
}

template <>
int32 FDataflowOverrideNode::GetDefaultValue(UE::Dataflow::FContext& Context) const
{
	const FString DefaultValue = GetValue<FString>(Context, &Default, Default);
	int32 DefaultIntValue = 0;

	if (!DefaultValue.IsEmpty() && DefaultValue.IsNumeric())
	{
		DefaultIntValue = FCString::Atoi(*DefaultValue);
	}

	return DefaultIntValue;
}

template <>
float FDataflowOverrideNode::GetDefaultValue(UE::Dataflow::FContext& Context) const
{
	const FString DefaultValue = GetValue<FString>(Context, &Default, Default);
	float DefaultFloatValue = 0.f;

	if (!DefaultValue.IsEmpty() && DefaultValue.IsNumeric())
	{
		DefaultFloatValue = FCString::Atof(*DefaultValue);
	}

	return DefaultFloatValue;
}

FString FDataflowOverrideNode::GetValueFromAsset(UE::Dataflow::FContext& Context, const UObject* InOwner) const
{
	FName InKey = GetValue<FName>(Context, &Key, Key);
	FString EmptyString;

	if (InOwner)
	{
		if (!InKey.IsNone() && InKey.IsValid() && InKey.ToString().Len() > 0)
		{
			return FindOverrideMapProperty<FString>(InOwner, FName("Overrides"), InKey, EmptyString);
		}
	}

	return EmptyString;
}
