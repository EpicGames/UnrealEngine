// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowPath)

FString FDataflowPath::ToString() const
{
	FString OutString;

	if (Node.IsEmpty())
	{
		OutString = FString::Printf(TEXT("Graph=%s"), *Graph);
	}
	else
	{
		if (Input.IsEmpty() && Output.IsEmpty())
		{
			OutString = FString::Printf(TEXT("Graph=%s|Node=%s"), *Graph, *Node);
		}
		else if (Output.IsEmpty())
		{
			OutString = FString::Printf(TEXT("Graph=%s|Node=%s|Input=%s"), *Graph, *Node, *Input);
		}
		else if (Input.IsEmpty())
		{
			OutString = FString::Printf(TEXT("Graph=%s|Node=%s|Output=%s"), *Graph, *Node, *Output);
		}
	}

	return OutString;
}

bool FDataflowPath::PathHasInput() const
{
	return !Input.IsEmpty();
}

bool FDataflowPath::PathHasOutput() const
{
	return !Output.IsEmpty();
}

void FDataflowPath::DecodePath(const FString InPath)
{
	TArray<FString> Tokens;
	int32 NumElems = InPath.ParseIntoArray(Tokens, TEXT("|"));

	FString StringA, StringB;

	Graph = "";
	Node = "";
	Input = "";
	Output = "";

	// Graph
	if (Tokens.Num() > 0)
	{
		Tokens[0].Split(TEXT("="), &StringA, &StringB);
		Graph = StringB;
	}

	// Node
	if (Tokens.Num() > 1)
	{
		Tokens[1].Split(TEXT("="), &StringA, &StringB);
		Node = StringB;
	}

	// Input
	if (Tokens.Num() > 1 && InPath.Contains("Input", ESearchCase::CaseSensitive, ESearchDir::FromStart))
	{
		Tokens[2].Split(TEXT("="), &StringA, &StringB);
		Input = StringB;
	}

	// Output
	if (Tokens.Num() > 1 && InPath.Contains("Output", ESearchCase::CaseSensitive, ESearchDir::FromStart))
	{
		Tokens[2].Split(TEXT("="), &StringA, &StringB);
		Output = StringB;
	}
}

