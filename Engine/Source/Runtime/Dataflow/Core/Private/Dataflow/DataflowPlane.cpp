// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowPlane.h"
#include "Kismet/KismetMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowPlane)

FDataflowPlane::FDataflowPlane(const FPlane& InPlane)
{
	Normal = InPlane.GetNormal();
	Point = InPlane.GetOrigin();
}

FPlane FDataflowPlane::AsPlane() const
{
	return FPlane(Point, Normal.GetSafeNormal());
}

FTransform FDataflowPlane::AsTransform() const
{
	if (!Normal.IsNearlyZero() && Normal.IsNormalized())
	{
		const FQuat Quat = FQuat::FindBetweenVectors(FVector::UpVector, Normal);
		return FTransform(Quat, Point);
	}
	else
	{
		return FTransform::Identity;
	}

}

double FDataflowPlane::DistanceFromPoint(const FVector& InPoint) const
{
	return AsPlane().PlaneDot(InPoint);
}

FDataflowPlane FDataflowPlane::GetTransformed(const FTransform& InTransform) const
{
	const FVector TransformedPoint = InTransform.TransformPosition(Point);

	FMatrix TransformMatrix = InTransform.ToMatrixWithScale();
	const float MulBy = TransformMatrix.Determinant() < 0.f ? -1.f : 1.f;

	FMatrix NormalsTransform = TransformMatrix.TransposeAdjoint() * (double)MulBy;
	const FVector TransformedNormal = NormalsTransform.TransformVector(Normal).GetSafeNormal();

	return FDataflowPlane(TransformedPoint, TransformedNormal);
}











