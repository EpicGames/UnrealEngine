// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowPoints.h"
#include "GeometryCollection/Facades/PointsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowPoints)

const GeometryCollection::Facades::FPointsFacade FDataflowPoints::GetPointsFacade() const
{
	return GeometryCollection::Facades::FPointsFacade(Collection);
}

GeometryCollection::Facades::FPointsFacade FDataflowPoints::GetPointsFacade()
{
	return GeometryCollection::Facades::FPointsFacade(Collection);
}

void FDataflowPoints::Set(FManagedArrayCollection& InCollection) 
{ 
	InCollection.CopyTo(&Collection); 
}

void FDataflowPoints::Set(const FManagedArrayCollection& InCollection)
{
	InCollection.CopyTo(&Collection);
}

