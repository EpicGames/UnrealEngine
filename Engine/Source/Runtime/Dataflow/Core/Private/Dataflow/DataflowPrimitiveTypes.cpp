// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowPrimitiveTypes.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowCategoryRegistry.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"

namespace UE::Dataflow
{
	void RegisterPrimitiveTypes()
	{
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowPrimitiveTypes);
		UE_DATAFLOW_REGISTER_ANYTYPE(FDataflowPrimitiveArrayTypes);

		static const FLinearColor CDefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);
		static const float CDefaultWireThickness = 2.f;

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FDataflowPrimitiveTypes", FLinearColor(0.6f, 0.f, 0.6f), CDefaultWireThickness);
	}
};




