// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowConnection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSelection)

void FDataflowSelection::Initialize(int32 NumBits, bool Value) 
{ 
	SelectionArray.Init(Value, NumBits); 
}

void FDataflowSelection::Initialize(const FDataflowSelection& Other)
{
	SelectionArray = Other.SelectionArray;
}

void FDataflowSelection::Initialize(const TBitArray<>& Other)
{
	SelectionArray = Other;
}

void FDataflowSelection::InitializeFromCollection(const FManagedArrayCollection& InCollection, bool Value)
{
	SelectionArray.Init(Value, InCollection.NumElements(GroupName));
}

void FDataflowSelection::Clear()
{
	SelectionArray.Init(false, SelectionArray.Num());
}

int32 FDataflowSelection::Num() const 
{ 
	return SelectionArray.Num(); 
}

void FDataflowSelection::AsArrayValidated(TArray<int32>& OutSelectionArr, const FManagedArrayCollection& InCollection) const
{
	if (IsValidForCollection(InCollection))
	{
		AsArray(OutSelectionArr);
	}
	else
	{
		int32 NumEl = InCollection.NumElements(GroupName);
		UE_LOGF(LogChaosDataflow, Warning, "Selection had mismatched element count vs array: %d vs %d", Num(), NumEl);
		int32 ValidNum = FMath::Min(NumEl, Num());
		for (int32 Idx = 0; Idx < ValidNum; ++Idx)
		{
			if (SelectionArray[Idx])
			{
				OutSelectionArr.Add(Idx);
			}
		}
	}
}

TArray<int32> FDataflowSelection::AsArrayValidated(const FManagedArrayCollection& InCollection) const
{
	TArray<int32> SelectionArr;
	AsArrayValidated(SelectionArr, InCollection);
	return SelectionArr;
}

void FDataflowSelection::AsArray(TArray<int32>& SelectionArr) const
{
	SelectionArr.Reset(SelectionArray.Num());

	TBitArray<>::FConstIterator It(SelectionArray);
	while (It)
	{
		int32 Index = It.GetIndex();

		if (SelectionArray[Index])
		{
			SelectionArr.Add(Index);
		}

		++It;
	}
}

TArray<int32> FDataflowSelection::AsArray() const
{
	TArray<int32> SelectionArr;

	TBitArray<>::FConstIterator It(SelectionArray);
	while (It)
	{
		int32 Index = It.GetIndex();

		if (SelectionArray[Index])
		{
			SelectionArr.Add(Index);
		}

		++It;
	}

	return SelectionArr;
}

bool FDataflowSelection::SetFromArray(const TArray<int32>& SelectionArr)
{
	SelectionArray.Init(false, SelectionArray.Num());
	bool bSuccess = true;
	for (int32 Elem : SelectionArr)
	{
		if (SelectionArray.IsValidIndex(Elem))
		{
			SelectionArray[Elem] = true;
		}
		else
		{
			bSuccess = false;
		}
	}
	if (!bSuccess)
	{
		UE_LOGF(LogChaosDataflow, Warning, "FDataflowSelection::SetFromArray failed to set invalid indices.");
	}
	return bSuccess;
}

void FDataflowSelection::SetFromBitArray(TBitArray<>&& Other)
{
	SelectionArray = MoveTemp(Other);
}

void FDataflowSelection::SetFromArray(const TArray<bool>& SelectionArr)
{
	SelectionArray.Init(false, SelectionArray.Num());

	for (int32 Idx = 0; Idx < SelectionArr.Num(); ++Idx)
	{
		if (SelectionArr[Idx])
		{
			SetSelected(Idx);
		}
	}
}

void FDataflowSelection::AND(const FDataflowSelection& Other, FDataflowSelection& Result) const
{ 
	Result.SelectionArray = TBitArray<>::BitwiseAND(SelectionArray, Other.SelectionArray, EBitwiseOperatorFlags::MaxSize);
}

void FDataflowSelection::OR(const FDataflowSelection& Other, FDataflowSelection& Result) const
{
	Result.SelectionArray = TBitArray<>::BitwiseOR(SelectionArray, Other.SelectionArray, EBitwiseOperatorFlags::MaxSize);
}

void FDataflowSelection::XOR(const FDataflowSelection& Other, FDataflowSelection& Result) const
{
	Result.SelectionArray = TBitArray<>::BitwiseXOR(SelectionArray, Other.SelectionArray, EBitwiseOperatorFlags::MaxSize);
}

void FDataflowSelection::Subtract(const FDataflowSelection& Other, FDataflowSelection& Result) const
{
	TBitArray<> InvSelectionArray = Other.SelectionArray;
	InvSelectionArray.BitwiseNOT();
	Result.SelectionArray = TBitArray<>::BitwiseAND(SelectionArray, InvSelectionArray, EBitwiseOperatorFlags::MaxSize);
}

bool FDataflowSelection::IsValid() const
{
	return (!GroupName.IsNone() && SelectionArray.Num() > 0);
}

int32 FDataflowSelection::NumSelected() const
{
	return SelectionArray.CountSetBits(0, SelectionArray.Num());
}

bool FDataflowSelection::AnySelected() const
{
	TBitArray<>::FConstIterator It(SelectionArray);
	while (It)
	{
		if (It.GetValue())
		{
			return true;
		}

		++It;
	}

	return false;
}

void FDataflowSelection::SetWithMask(const bool Value, const FDataflowSelection& Mask)
{
	if (SelectionArray.Num() == Mask.Num())
	{
		for (int32 Idx = 0; Idx < SelectionArray.Num(); ++Idx)
		{
			if (Mask.IsSelected(Idx))
			{
				SelectionArray[Idx] = Value;
			}
		}
	}
}

void FDataflowSelection::SetSelected(const TArray<int32>& Indices)
{
	for (int32 Idx = 0; Idx < Indices.Num(); ++Idx)
	{
		if (SelectionArray.IsValidIndex(Indices[Idx]))
		{
			SetSelected(Indices[Idx]);
		}
	}
}

bool FDataflowSelection::SetSelectedWithCheck(const TArray<int32>& Indices)
{
	for (int32 Idx = 0; Idx < Indices.Num(); ++Idx)
	{
		if (SelectionArray.IsValidIndex(Indices[Idx]))
		{
			SetSelected(Indices[Idx]);
		}
		else
		{
			return false;
		}
	}

	return true;
}

bool FDataflowSelection::IsValidForCollection(const FManagedArrayCollection& InCollection) const
{
	return !GroupName.IsNone() && (Num() != 0) && Num() == InCollection.NumElements(GroupName);
}

bool FDataflowSelection::InitFromArray(const FManagedArrayCollection& InCollection, const TArray<int32>& InSelectionArr)
{
	InitializeFromCollection(InCollection, false);
	return SetFromArray(InSelectionArr);
}

FString FDataflowSelection::ToString()
{
	const int32 NumSelectedElems = NumSelected();
	const int32 NumElems = SelectionArray.Num();
	FString OutString;

	OutString = FString::Printf(TEXT("Selected %s: %s of %s"), *GroupName.ToString(), *FString::FromInt(NumSelected()), *FString::FromInt(NumElems));

	return OutString;
}

FName FDataflowSelection::GetSelectionTypeFromCollection(const FDataflowConnection& InConnection)
{
	if (InConnection.IsType<FDataflowTransformSelection>())
	{
		return 	FDataflowTransformSelection::TransformGroupName;
	}
	else if (InConnection.IsType<FDataflowVertexSelection>())
	{
		return 	FDataflowVertexSelection::VerticesGroupName;
	}
	else if (InConnection.IsType<FDataflowFaceSelection>())
	{
		return 	FDataflowFaceSelection::FacesGroupName;
	}
	else if (InConnection.IsType<FDataflowGeometrySelection>())
	{
		return 	FDataflowGeometrySelection::GeometryGroupName;
	}
	else if (InConnection.IsType<FDataflowCurveSelection>())
	{
		return 	FDataflowCurveSelection::CurveGroupName;
	}
	else if (InConnection.IsType<FDataflowPointsSelection>())
	{
		return 	FDataflowPointsSelection::PointsGroupName;
	}
	else if (InConnection.IsType<FDataflowMaterialSelection>())
	{
		return 	FDataflowMaterialSelection::MaterialGroupName;
	}

	return NAME_None;
}
