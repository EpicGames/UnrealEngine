// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSettings)

UDataflowSettings::UDataflowSettings(const FObjectInitializer& ObjectInitlaizer)
	: Super(ObjectInitlaizer)
{
	TransformLevelColors.LevelColors.AddUninitialized(10);
	TransformLevelColors.LevelColors[0] = FLinearColor(FColor(0, 255, 255));
	TransformLevelColors.LevelColors[1] = FLinearColor(FColor(243, 156, 18));
	TransformLevelColors.LevelColors[2] = FLinearColor(FColor(46, 204, 113));
	TransformLevelColors.LevelColors[3] = FLinearColor(FColor(255, 255, 0));
	TransformLevelColors.LevelColors[4] = FLinearColor(FColor(169, 7, 228));
	TransformLevelColors.LevelColors[5] = FLinearColor(FColor(255, 0, 255));
	TransformLevelColors.LevelColors[6] = FLinearColor(FColor(26, 188, 156));
	TransformLevelColors.LevelColors[7] = FLinearColor(FColor(189, 195, 199));
	TransformLevelColors.LevelColors[8] = FLinearColor(FColor(0, 0, 255));
	TransformLevelColors.LevelColors[9] = FLinearColor(FColor(0, 255, 0));

	TransformLevelColors.BlankColor = FLinearColor(255, 255, 255, 255);
}

FName UDataflowSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR

FText UDataflowSettings::GetSectionText() const
{
	return NSLOCTEXT("DataflowPlugin", "DataflowSettingsSection", "Dataflow");
}

void UDataflowSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		OnDataflowSettingsChangedDelegate.Broadcast(NodeColorsMap);
		OnDataflowSettingsChangedPinSettingsDelegate.Broadcast(PinSettingsMap);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

FNodeColors UDataflowSettings::RegisterColors(const FName& Category, const FNodeColors& Colors)
{
	if (!NodeColorsMap.Contains(Category))
	{
		NodeColorsMap.Add(Category, Colors);
	}
	return NodeColorsMap[Category];
}

FPinSettings UDataflowSettings::RegisterPinSettings(const FName& InPinType, const FPinSettings& InSettings)
{
	if (!PinSettingsMap.Contains(InPinType))
	{
		PinSettingsMap.Add(InPinType, InSettings);
	}
	return PinSettingsMap[InPinType];
}



