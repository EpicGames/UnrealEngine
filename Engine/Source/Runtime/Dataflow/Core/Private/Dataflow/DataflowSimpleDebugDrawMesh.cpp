// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimpleDebugDrawMesh.h"
#include "VectorTypes.h"

int32 FSimpleDebugDrawMesh::GetMaxVertexIndex() const
{
	return Vertices.Num();
}

bool FSimpleDebugDrawMesh::IsValidVertex(int32 VertexIndex) const
{
	return VertexIndex >= 0 && VertexIndex < Vertices.Num();
}

FVector FSimpleDebugDrawMesh::GetVertexPosition(int32 VertexIndex) const
{
	return Vertices[VertexIndex];
}

FVector FSimpleDebugDrawMesh::GetVertexNormal(int32 VertexIndex) const
{
	return VertexNormals[VertexIndex];
}

void FSimpleDebugDrawMesh::SetVertex(const int32 VertexIndex, const FVector& VertexPosition)
{
	if (IsValidVertex(VertexIndex))
	{
		Vertices[VertexIndex] = VertexPosition;
	}
}

int32 FSimpleDebugDrawMesh::GetMaxTriangleIndex() const
{
	return Triangles.Num();
}

bool FSimpleDebugDrawMesh::IsValidTriangle(int32 TriangleIndex) const
{
	return TriangleIndex >= 0 && TriangleIndex < Triangles.Num();
}

FIntVector3 FSimpleDebugDrawMesh::GetTriangle(int32 TriangleIndex) const
{
	return Triangles[TriangleIndex];
}

void FSimpleDebugDrawMesh::SetTriangle(const int32 TriangleIndex, const int32 VertexIndexA, const int32 VertexIndexB, const int32 VertexIndexC)
{
	if (IsValidTriangle(TriangleIndex))
	{
		Triangles[TriangleIndex] = FIntVector(VertexIndexA, VertexIndexB, VertexIndexC);
	}
}

void FSimpleDebugDrawMesh::TransformVertices(const FTransform& Transform) 
{
	for (FVector& Vertex : Vertices)
	{
		Vertex = Transform.TransformPosition(Vertex);
	}

	for (FVector& Normal : VertexNormals)
	{
		Normal = Transform.TransformVector(Normal);
	}
}

static FVector BilinearInterp(const FVector& V00, const FVector& V10, const FVector& V11, const FVector& V01, double Tx, double Ty)
{
	FVector A = UE::Geometry::Lerp(V00, V01, Ty);
	FVector B = UE::Geometry::Lerp(V10, V11, Ty);
	return UE::Geometry::Lerp(A, B, Tx);
}

void FSimpleDebugDrawMesh::MakeRectangleMesh(const FVector& Origin, const float Width, const float Height, const int32 WidthVertexCount, const int32 HeightVertexCount)
{
	int32 WidthNV = (WidthVertexCount > 1) ? WidthVertexCount : 2;
	int32 HeightNV = (HeightVertexCount > 1) ? HeightVertexCount : 2;

	int32 TotalNumVertices = WidthNV * HeightNV;
	int32 TotalNumTriangles = 2 * (WidthNV - 1) * (HeightNV - 1);
	
	Vertices.Empty();
	Triangles.Empty();

	if (TotalNumVertices > 0)
	{
		Vertices.AddUninitialized(TotalNumVertices);
		VertexNormals.AddUninitialized(TotalNumVertices);
	}
	if (TotalNumTriangles > 0)
	{
		Triangles.AddUninitialized(TotalNumTriangles);
	}

	// corner vertices
	FVector V00 = Origin + FVector(-Width / 2.0f, -Height / 2.0f, 0.f); 
	FVector V01 = Origin + FVector( Width / 2.0f, -Height / 2.0f, 0.f);
	FVector V11 = Origin + FVector( Width / 2.0f,  Height / 2.0f, 0.f);
	FVector V10 = Origin + FVector(-Width / 2.0f,  Height / 2.0f, 0.f);

	// Compute normal vector for the plane
	const FVector Edge1 = V01 - V00;
	const FVector Edge2 = V10 - V00;
	FVector Normal = FVector::CrossProduct(Edge1, Edge2);
	Normal.Normalize();

	int32 Vi = 0;
	int32 Ti = 0;

	// add vertex rows
	int32 Start_vi = Vi;
	for (int32 Yi = 0; Yi < HeightNV; ++Yi)
	{
		double Ty = (double)Yi / (double)(HeightNV - 1);
		for (int32 Xi = 0; Xi < WidthNV; ++Xi)
		{
			double Tx = (double)Xi / (double)(WidthNV - 1);
			SetVertex(Vi, BilinearInterp(V00, V01, V11, V10, Tx, Ty));

			Vi++;
		}
	}

	// add triangulated quads
	for (int32 Y0 = 0; Y0 < HeightNV - 1; ++Y0)
	{
		for (int32 X0 = 0; X0 < WidthNV - 1; ++X0)
		{
			int32 I00 = Start_vi + Y0 * WidthNV + X0;
			int32 I10 = Start_vi + (Y0 + 1) * WidthNV + X0;
			int32 I01 = I00 + 1;
			int32 I11 = I10 + 1;

			SetTriangle(Ti, I00, I11, I01);

			Ti++;

			SetTriangle(Ti, I00, I10, I11);

			Ti++;
		}
	}

	// Add normal vectors
	for (int32 Idx = 0; Idx < TotalNumVertices; ++Idx)
	{
		VertexNormals[Idx] = Normal;
	}
}


