// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSystem.h"

namespace UE::Dataflow
{
	/*static*/ Dataflow::FSystem& FSystem::Get()
	{
		static Dataflow::FSystem Instance;
		return Instance;
	}
}
