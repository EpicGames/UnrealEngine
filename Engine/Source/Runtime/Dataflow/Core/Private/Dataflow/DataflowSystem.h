// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowTaskGraph.h"

namespace UE::Dataflow
{
	struct FSystem
	{
	public:
		static Dataflow::FSystem& Get();

		/* TaskGraph to schedule Dtaaflow specific tasks */
		Dataflow::FTaskGraph TaskGraph;
	};

}