// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowTaskGraph.h"
#include "Algo/Transform.h"

#define LOCTEXT_NAMESPACE "DataflowTaskGraph"

DEFINE_LOG_CATEGORY_STATIC(LogDataflowTaskGraph, Log, All);

namespace UE::Dataflow
{
	bool FTask::IsCompleted() const
	{
		return Task.IsCompleted();
	}

	bool FTask::IsValid() const
	{
		return Task.IsValid();
	}

	bool FTaskGraph::FGameThreadTask::CanExecute() const
	{
		for (const FTask& Prerequisite : Prerequisites)
		{
			if (!Prerequisite.Task.IsCompleted())
			{
				return false;
			}
		}
		return true;
	}

	bool FTaskGraph::FGameThreadTask::IsCancelled() const
	{
		return CancellationToken && CancellationToken->IsCanceled();
	}

	bool FTaskGraph::FGameThreadTask::TryExecute()
	{
		if (CanExecute())
		{
			const bool bIsCancelled = IsCancelled();
			if (TaskBody && !bIsCancelled)
			{
				TaskBody();
			}
			CompletionTaskEvent.Trigger();
			return true;
		}
		return false;
	}

	///////////////////////////////////////////////////////////////////////////////////////////

	FTaskGraph::FTaskGraph()
	{
		// TODO : launch ticker only when first gamethread task is requested
		TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FTaskGraph::OnTick));
	}

	FTaskGraph::~FTaskGraph()
	{
		FTSTicker::GetCoreTicker().RemoveTicker(TickDelegateHandle);
		TickDelegateHandle.Reset();
	}

	FTask FTaskGraph::LaunchGameThreadTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<FTask>& Prerequisites, FCancellationTokenPtr CancellationToken)
	{
		GameThreadTasks.Emplace(
			MakeShared<FGameThreadTask>(FGameThreadTask
			{
				.DebugName = DebugName,
				.TaskBody = MoveTemp(TaskBody),
				.Prerequisites = Prerequisites,
				.CancellationToken = CancellationToken,
				.CompletionTaskEvent = UE::Tasks::FTaskEvent{ DebugName },
			}));
		FTask ReturnTask;
		ReturnTask.Task = GameThreadTasks.Last()->CompletionTaskEvent;
		return ReturnTask;
	}

	FTask FTaskGraph::LaunchAnyThreadTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<FTask>& Prerequisites, FCancellationTokenPtr CancellationToken)
	{
		TArray<UE::Tasks::FTask> UETasksPrerequisites;
		Algo::Transform(Prerequisites, UETasksPrerequisites, [](const FTask& Task) { return Task.Task; });
		FTask ReturnTask;
		ReturnTask.Task = UE::Tasks::Launch(DebugName, MoveTemp(TaskBody), UETasksPrerequisites);
		return ReturnTask;
	}

	bool FTaskGraph::OnTick(float DeltaTime)
	{
		UE_LOGF(LogDataflowTaskGraph, Verbose, "FTaskGraph::Tick -- START");

		// TODO : should this be a parameter of the taskgraph ?
		constexpr double TickTimeBudget = 5.0;

		const double StartTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
		double CurrentTimeMs = StartTimeMs;

		TArray<FGameThreadTaskRef> NonExecutedTasks;

		const int32 NumTasks = GameThreadTasks.Num();
		NonExecutedTasks.Reserve(NumTasks);

		bool bOverBudget = false;
		for (int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			if (bOverBudget)
			{
				NonExecutedTasks.Add(GameThreadTasks[TaskIndex]);
			}
			else if (!GameThreadTasks[TaskIndex]->TryExecute())
			{
				NonExecutedTasks.Add(GameThreadTasks[TaskIndex]);
			}
			else
			{
				// we only check time if the task has executed to make sure we don't run out of time just checking that tasks can run 
				// and be stuck at always evelauting the same one with no progress
				CurrentTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
				if ((CurrentTimeMs - StartTimeMs) > TickTimeBudget)
				{
					bOverBudget = true;
				}

			}
		}

		// flip the arrays
		GameThreadTasks = MoveTemp(NonExecutedTasks);

		UE_LOGF(LogDataflowTaskGraph, Verbose, "FTaskGraph::Tick -- END : time = [%.3f ms]", (float)(CurrentTimeMs - StartTimeMs));
		return true;
	}
};

#undef LOCTEXT_NAMESPACE
