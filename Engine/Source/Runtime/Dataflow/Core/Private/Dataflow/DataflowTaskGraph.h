// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/Task.h"
#include "Containers/Ticker.h"
#include "Templates/SharedPointer.h"

namespace UE::Dataflow
{
	// encapsulation of a task so that we cannot for example wait on them 
	struct FTask
	{
	public:
		bool IsCompleted() const;
		bool IsValid() const;

		// Explicitly delete the wait method to avoid future implementer to add it 
		// Dtaaflow Tasks should not be awaited and should be listen to using a completion callback/task instead
		void Wait() = delete;

	private:
		friend struct FTaskGraph;
		UE::Tasks::FTask Task;
	};

	struct FTaskGraph
	{
	public:
		FTaskGraph();
		~FTaskGraph();

		using FCancellationTokenPtr = TSharedPtr<UE::Tasks::FCancellationToken>;

		/** 
		* Launch a task to run on the game thread 
		* Execution will happen during the next execution of tick method of the taskgraph when all dependent tasks have be completed
		*/
		FTask LaunchGameThreadTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<FTask>& Prerequisites, FCancellationTokenPtr CancellationToken = {});

		/**
		* Launch a task to run on any thread
		*/
		FTask LaunchAnyThreadTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<FTask>& Prerequisites, FCancellationTokenPtr CancellationToken = {});

	private:
		struct FGameThreadTask
		{
			/** Debug name of the task */
			const TCHAR* DebugName = TEXT("DataflowGameThreadTask");

			/** Function to execute when running the task */
			TUniqueFunction<void()> TaskBody;

			/** List of pPrerequisites */
			TArray<FTask> Prerequisites;

			/** optional cancellation token */
			FCancellationTokenPtr CancellationToken;

			/** Task event that completes when the body has executed */
			UE::Tasks::FTaskEvent CompletionTaskEvent;

			bool CanExecute() const;
			bool IsCancelled() const;
			bool TryExecute();
		};
		using FGameThreadTaskRef = TSharedRef<FGameThreadTask>;

		FTSTicker::FDelegateHandle TickDelegateHandle;
		bool OnTick(float DeltaTime);

		TArray<FGameThreadTaskRef> GameThreadTasks;


	};
};
