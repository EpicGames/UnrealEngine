// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowTerminalNode.h"

#include "ChaosLog.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowArchive.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowTerminalNode)

#define LOCTEXT_NAMESPACE "DataflowTerminalNode"

FString FDataflowTerminalNode::GetAssetPath(const FString& InputPath, UObject* BoundAsset)
{
	static FString EmptyString;

	FString OutPath = InputPath;
	if (!FPackageName::IsValidObjectPath(OutPath))
	{
		// no valid path , is there a bound asset ?
		if (BoundAsset)
		{
			OutPath = BoundAsset->GetPathName();
		}
		if (!FPackageName::IsValidObjectPath(OutPath))
		{
			OutPath = EmptyString;
		}
	}
	return OutPath;
}

UObject* FDataflowTerminalNode::GetOrCreateAsset(UE::Dataflow::FContext& Context, const FString& InAssetPath, const UClass* AssetClass) const
{
#if WITH_EDITOR
	if (!FPackageName::IsValidObjectPath(InAssetPath))
	{
		Context.Error(FText::Format(
			LOCTEXT("InvalidAssetPath", "invalid asset path: Path = {0}"), FText::FromString(InAssetPath)),
			this);
		return nullptr;
	}

	// Try to create the package
	const FString PackageName = FPackageName::ObjectPathToPackageName(InAssetPath);
	UPackage* Package = Cast<UPackage>(FindPackage(nullptr, *PackageName));
	if (Package == nullptr)
	{
		Package = CreatePackage(*PackageName);
	}
	if (Package == nullptr)
	{
		Context.Error(FText::Format(
			LOCTEXT("InvalidPackage", "Failed to find or create package {0}"), FText::FromString(PackageName)),
			this);
		return nullptr;
	}

	// Check if there's an existing object matching this path and compatible with this type
	const FName AssetName = FName(FPackageName::GetLongPackageAssetName(PackageName));
	UObject* ExistingObject = StaticFindObjectFastInternal( /*Class=*/ NULL, Package, AssetName, EFindObjectFlags::ExactClass);
	if (ExistingObject && !ExistingObject->GetClass()->IsChildOf(AssetClass))
	{
		Context.Error(FText::Format(
			LOCTEXT("IncompatibleType", "Asset {0} already exists but is not a {1} compatible type"),
			FText::FromString(InAssetPath), FText::FromName(AssetClass->GetFName())),
			this);
		return nullptr;
	}

	UObject* OutObject = ExistingObject;
	if (!OutObject)
	{
		OutObject = NewObject<UObject>(Package, AssetClass, AssetName, RF_Public | RF_Standalone | RF_Transactional);
		if (!OutObject)
		{
			Context.Error(FText::Format(
				LOCTEXT("AssetCreationFailed", "Failed to create asset {0}"), FText::FromString(InAssetPath)),
				this);
			return nullptr;
		}
	}
	// make sure the asset is set properly
	OutObject->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(OutObject);

	return OutObject;
#else
	Context.Error(LOCTEXT("AssetCreationOnlyEditor", "Creation of assets only supported in Editor"), this);
	return nullptr;
#endif
}

#undef LOCTEXT_NAMESPACE