// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowUtils.h"

#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Internationalization/Regex.h"

namespace UE::Dataflow
{
	namespace Utils
	{
		/* e.g. "0, 2, 5-10, 12-15" */
		EErrorCode ParseIndicesStr(const FString& InFramesString, TArray<int32>& OutIndices)
		{
			OutIndices.Empty();

			static const FRegexPattern AllowedCharsPattern(TEXT("^[-,0-9\\s]+$"));

			if (!FRegexMatcher(AllowedCharsPattern, InFramesString).FindNext())
			{
				// Input contains invalid characters
				return EErrorCode::InvalidChars;
			}

			static const FRegexPattern SingleNumberPattern(TEXT("^\\s*(\\d+)\\s*$"));
			static const FRegexPattern RangePattern(TEXT("^\\s*(\\d+)\\s*-\\s*(\\d+)\\s*$"));

			TArray<FString> Segments;
			InFramesString.ParseIntoArray(Segments, TEXT(","), true);
			for (const FString& Segment : Segments)
			{
				bool bSegmentValid = false;

				FRegexMatcher SingleNumberMatcher(SingleNumberPattern, Segment);
				if (SingleNumberMatcher.FindNext())
				{
					const int32 SingleNumber = FCString::Atoi(*SingleNumberMatcher.GetCaptureGroup(1));

					if (SingleNumber < 0 || SingleNumber > 100000)
					{
						return EErrorCode::NumberInRangeTooBig;
					}

					OutIndices.Add(SingleNumber);
					bSegmentValid = true;
				}
				else
				{
					FRegexMatcher RangeMatcher(RangePattern, Segment);
					if (RangeMatcher.FindNext())
					{
						const int32 RangeStart = FCString::Atoi(*RangeMatcher.GetCaptureGroup(1));
						const int32 RangeEnd = FCString::Atoi(*RangeMatcher.GetCaptureGroup(2));

						if (RangeStart > RangeEnd)
						{
							return EErrorCode::InvalidRangeStartEnd;
						}

						if (RangeStart < 0 || RangeStart > 100000 || RangeEnd < 0 || RangeEnd > 100000)
						{
							return EErrorCode::NumberInRangeTooBig;
						}

						for (int32 i = RangeStart; i <= RangeEnd; ++i)
						{
							OutIndices.Add(i);
						}
						bSegmentValid = true;
					}
				}

				if (!bSegmentValid)
				{
					// Invalid format in segment
					return EErrorCode::InvalidFormatInSegment;
				}
			}

			return EErrorCode::None;
		}
	
	
		void GetMinMaxInArray(const TArray<float>& InArray, float& OutMin, float& OutMax)
		{
			OutMin = FLT_MAX;
			OutMax = -FLT_MAX;

			const int32 NumElems = InArray.Num();
			if (NumElems > 0)
			{
				for (int32 Idx = 0; Idx < NumElems; ++Idx)
				{
					if (InArray[Idx] < OutMin)
					{
						OutMin = InArray[Idx];
					}

					if (InArray[Idx] > OutMax)
					{
						OutMax = InArray[Idx];
					}
				}

				return;
			}

			OutMin = 0.0;
			OutMax = 0.0;
		}
	}
}
