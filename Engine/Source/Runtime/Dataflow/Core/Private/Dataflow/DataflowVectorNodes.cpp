// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVectorNodes.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowVectorNodes)

namespace UE::Dataflow
{
	void RegisterDataflowVectorNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorMakeVec2Node);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorMakeVec3Node);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorMakeVec4Node);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorBreakNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorAddNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorSubtractNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorMultiplyNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorDivideNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorDotProductNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorLengthNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorSquaredLengthNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorDistanceNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorCrossProductNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorScaleNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowVectorNormalize);

		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowVectorTypes, FDataflowNumericTypes, FDataflowVectorBreakNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FDataflowNumericTypes, FDataflowVectorTypes, FDataflowVectorMakeVec3Node);
	}
}

//-----------------------------------------------------------

FDataflowVectorMakeVec2Node::FDataflowVectorMakeVec2Node(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&X);
	RegisterInputConnection(&Y);
	RegisterOutputConnection(&Vector2D);
	// set default to 2D vector
	SetOutputConcreteType<FVector2D>(&Vector2D);
}

void FDataflowVectorMakeVec2Node::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Vector2D))
	{
		const double InX = GetValue(Context, &X);
		const double InY = GetValue(Context, &Y);
		SetValue(Context, FVector4(InX, InY, 0, 0), &Vector2D);
	}
}

//-----------------------------------------------------------

FDataflowVectorMakeVec3Node::FDataflowVectorMakeVec3Node(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&X);
	RegisterInputConnection(&Y);
	RegisterInputConnection(&Z);
	RegisterOutputConnection(&Vector3D);
	// set default to 3D vector
	SetOutputConcreteType<FVector>(&Vector3D);
}

void FDataflowVectorMakeVec3Node::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Vector3D))
	{
		const double InX = GetValue(Context, &X);
		const double InY = GetValue(Context, &Y);
		const double InZ = GetValue(Context, &Z);
		SetValue(Context, FVector4(InX, InY, InZ, 0), &Vector3D);
	}
}

//-----------------------------------------------------------

FDataflowVectorMakeVec4Node::FDataflowVectorMakeVec4Node(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&X);
	RegisterInputConnection(&Y);
	RegisterInputConnection(&Z);
	RegisterInputConnection(&W);
	RegisterOutputConnection(&Vector4D);
	// set default to 4D vector
	SetOutputConcreteType<FVector4>(&Vector4D);
}

void FDataflowVectorMakeVec4Node::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Vector4D))
	{
		const double InX = GetValue(Context, &X);
		const double InY = GetValue(Context, &Y);
		const double InZ = GetValue(Context, &Z);
		const double InW = GetValue(Context, &W);
		SetValue(Context, FVector4(InX, InY, InZ, InW), &Vector4D);
	}
}

//-----------------------------------------------------------

FDataflowVectorBreakNode::FDataflowVectorBreakNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&V);
	RegisterOutputConnection(&X);
	RegisterOutputConnection(&Y);
	RegisterOutputConnection(&Z);
	RegisterOutputConnection(&W);

	// set default out types to double
	SetOutputConcreteType<double>(&X);
	SetOutputConcreteType<double>(&Y);
	SetOutputConcreteType<double>(&Z);
	SetOutputConcreteType<double>(&W);
}

void FDataflowVectorBreakNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&X))
	{
		SetValue(Context, GetValue(Context, &V).X, &X);
	}
	else if (Out->IsA(&Y))
	{
		SetValue(Context, GetValue(Context, &V).Y, &Y);
	}
	else if (Out->IsA(&Z))
	{
		SetValue(Context, GetValue(Context, &V).Z, &Z);
	}
	else if (Out->IsA(&W))
	{
		SetValue(Context, GetValue(Context, &V).W, &W);
	}
}

//-----------------------------------------------------------

FDataflowVectorAddNode::FDataflowVectorAddNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterOutputConnection(&V);
	// set default to 4D vector
	SetOutputConcreteType<FVector4>(&V);

}

void FDataflowVectorAddNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&V))
	{
		const FVector4 InA = GetValue(Context, &A);
		const FVector4 InB = GetValue(Context, &B);
		SetValue(Context, (InA + InB), &V);
	}
}

//-----------------------------------------------------------

FDataflowVectorSubtractNode::FDataflowVectorSubtractNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterOutputConnection(&V);
	// set default to 4D vector
	SetOutputConcreteType<FVector4>(&V);

}

void FDataflowVectorSubtractNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&V))
	{
		const FVector4 InA = GetValue(Context, &A);
		const FVector4 InB = GetValue(Context, &B);
		SetValue(Context, (InA - InB), &V);
	}
}

//-----------------------------------------------------------

FDataflowVectorMultiplyNode::FDataflowVectorMultiplyNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterOutputConnection(&V);
	// set default to 4D vector
	SetOutputConcreteType<FVector4>(&V);

}

void FDataflowVectorMultiplyNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&V))
	{
		const FVector4 InA = GetValue(Context, &A);
		const FVector4 InB = GetValue(Context, &B);
		SetValue(Context, (InA * InB), &V);
	}
}

//-----------------------------------------------------------

FDataflowVectorDivideNode::FDataflowVectorDivideNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterInputConnection(&Fallback);
	RegisterOutputConnection(&V);
	// set default to 4D vector
	SetOutputConcreteType<FVector4>(&V);

}

void FDataflowVectorDivideNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&V))
	{
		const FVector4 InA = GetValue(Context, &A);
		const FVector4 InB = GetValue(Context, &B);
		const FVector4 InFallBack = GetValue(Context, &Fallback);
		FVector4 OutV(0);
		OutV.X = (InB.X == 0) ? InFallBack.X : (InA.X / InB.X);
		OutV.Y = (InB.Y == 0) ? InFallBack.Y : (InA.Y / InB.Y);
		OutV.Z = (InB.Z == 0) ? InFallBack.Z : (InA.Z / InB.Z);
		OutV.W = (InB.W == 0) ? InFallBack.W : (InA.W / InB.W);
		SetValue(Context, OutV, &V);
	}
}

//-----------------------------------------------------------

FDataflowVectorDotProductNode::FDataflowVectorDotProductNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterOutputConnection(&DotProduct);
	// set default type to be double
	SetOutputConcreteType<double>(&DotProduct);
}

void FDataflowVectorDotProductNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&DotProduct))
	{
		const FVector4 InA = GetValue(Context, &A);
		const FVector4 InB = GetValue(Context, &B);
		const double OutDotProduct = ((InA.X * InB.X) + (InA.Y * InB.Y) + (InA.Z * InB.Z) + (InA.W * InB.W));
		SetValue(Context, OutDotProduct, &DotProduct);
	}
}

//-----------------------------------------------------------

FDataflowVectorLengthNode::FDataflowVectorLengthNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&V);
	RegisterOutputConnection(&Length);
	// set default type to be double
	SetOutputConcreteType<double>(&Length);
}

void FDataflowVectorLengthNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Length))
	{
		const FVector4 InV = GetValue(Context, &V);
		const double OutLength = FMath::Sqrt((InV.X * InV.X) + (InV.Y * InV.Y) + (InV.Z * InV.Z) + (InV.W * InV.W));
		SetValue(Context, OutLength, &Length);
	}
}

//-----------------------------------------------------------

FDataflowVectorSquaredLengthNode::FDataflowVectorSquaredLengthNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&V);
	RegisterOutputConnection(&SquaredLength);
	// set default type to be double
	SetOutputConcreteType<double>(&SquaredLength);
}

void FDataflowVectorSquaredLengthNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SquaredLength))
	{
		const FVector4 InV = GetValue(Context, &V);
		const double OutSquaredLength = ((InV.X * InV.X) + (InV.Y * InV.Y) + (InV.Z * InV.Z) + (InV.W * InV.W));
		SetValue(Context, OutSquaredLength, &SquaredLength);
	}
}

//-----------------------------------------------------------

FDataflowVectorDistanceNode::FDataflowVectorDistanceNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterOutputConnection(&Distance);
	// set default type to be double
	SetOutputConcreteType<double>(&Distance);
}

void FDataflowVectorDistanceNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Distance))
	{
		const FVector4 InA = GetValue(Context, &A);
		const FVector4 InB = GetValue(Context, &B);
		const FVector4 Delta = (InB - InA);
		const double OutDistance = FMath::Sqrt((Delta.X * Delta.X) + (Delta.Y * Delta.Y) + (Delta.Z * Delta.Z) + (Delta.W * Delta.W));
		SetValue(Context, OutDistance, &Distance);
	}
}

//-----------------------------------------------------------

FDataflowVectorCrossProductNode::FDataflowVectorCrossProductNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterOutputConnection(&CrossProduct);
	// set default type to be vector 3
	SetOutputConcreteType<FVector>(&CrossProduct);
}

void FDataflowVectorCrossProductNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&CrossProduct))
	{
		// convert to FVector from FVector4
		const FVector InA(GetValue(Context, &A));
		const FVector InB(GetValue(Context, &B));
		const FVector OutCrossProduct = FVector::CrossProduct(InA, InB);
		SetValue(Context, OutCrossProduct, &CrossProduct);
	}
}

//-----------------------------------------------------------

FDataflowVectorScaleNode::FDataflowVectorScaleNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	Scale.Value = 1.0; // set default
	RegisterInputConnection(&V);
	RegisterInputConnection(&Scale);
	RegisterOutputConnection(&Scaled);
	// set default type to be Vector4 for now
	SetOutputConcreteType<FVector4>(&Scaled);
}

void FDataflowVectorScaleNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Scaled))
	{
		const FVector4 InV = GetValue(Context, &V);
		const double InScale = GetValue(Context, &Scale);
		const FVector4 OutResult = (InV * InScale);
		SetValue(Context, OutResult, &Scaled);
	}
}

//-----------------------------------------------------------

FDataflowVectorNormalize::FDataflowVectorNormalize(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&V);
	RegisterOutputConnection(&Normalized);
	// set default type to be Vector4 for now
	SetOutputConcreteType<FVector4>(&Normalized);

}

void FDataflowVectorNormalize::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Normalized))
	{
		const FVector4 InV = GetValue(Context, &V);
		const FVector4 OutResult = InV.GetSafeNormal();
		SetValue(Context, OutResult, &Normalized);
	}
}
