// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"

#include "DataflowVectorNodes.generated.h"

#define DATAFLOW_MATH_VECTOR_NODES_CATEGORY "Math|Vectors"

/** Make a 2D Vector */
USTRUCT()
struct FDataflowVectorMakeVec2Node : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorMakeVec2Node, "MakeVector2", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "Create Add 2D XY")

public:
	FDataflowVectorMakeVec2Node(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** X component */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes X;

	/** Y component */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes Y;

	/** 2D Vector {X, Y} */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes Vector2D;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Make a 3D Vector */
USTRUCT()
struct FDataflowVectorMakeVec3Node : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorMakeVec3Node, "MakeVector3", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "Create Add 3D XYZ")

public:
	FDataflowVectorMakeVec3Node(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** X component */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes X;

	/** Y component */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes Y;

	/** Z component */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes Z;

	/** 3D Vector {X, Y, Z} */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes Vector3D;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/** Make a 4D Vector */
USTRUCT()
struct FDataflowVectorMakeVec4Node : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorMakeVec4Node, "MakeVector4", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "Create Add 4D XYZW")

public:
	FDataflowVectorMakeVec4Node(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** X component */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes X;

	/** Y component */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes Y;

	/** Z component */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes Z;

	/** W component */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes W;

	/** 4D Vector {X, Y, Z, W} */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes Vector4D;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/** 
* Break a vector in 4 components
* if the input vector is of a lower dimension than 4, the remaining components will be set to zero
*/
USTRUCT()
struct FDataflowVectorBreakNode: public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorBreakNode, "BreakVector", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "Expand Split X Y Z W Components")

public:
	FDataflowVectorBreakNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Vector to break into components */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes V;

	/** X component */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowNumericTypes X;

	/** Y component */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowNumericTypes Y;

	/** Z component */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowNumericTypes Z;

	/** W component */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowNumericTypes W;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Add two vectors component wise : V = (A + B)*/
USTRUCT()
struct FDataflowVectorAddNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorAddNode, "AddVector", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "+ Plus Addition")

public:
	FDataflowVectorAddNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** A Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes A;

	/** B Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes B;

	/** Add result V=A+B */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes V;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Subtract two vectors component wise: V = (A - B) */
USTRUCT()
struct FDataflowVectorSubtractNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorSubtractNode, "SubtractVector", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "- Minus Subtraction")

public:
	FDataflowVectorSubtractNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** A Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes A;

	/** B Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes B;

	/** Add result V=A-B */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes V;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Multiply two vectors component wise: V = (A * B) */
USTRUCT()
struct FDataflowVectorMultiplyNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorMultiplyNode, "MultiplyVector", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "* Multiplication Times")

public:
	FDataflowVectorMultiplyNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** A Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes A;

	/** B Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes B;

	/** Add result V=A*B */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes V;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** 
* Multiply two vectors component wise: V = (A / B) 
* When a component of B is zero, use the falback value as a return value for the specific component 
*/
USTRUCT()
struct FDataflowVectorDivideNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorDivideNode, "DivideVector", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "/ Division Over")

public:
	FDataflowVectorDivideNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** A Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes A;

	/** B Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes B;

	/** Fallback Vector used when components of B are zero */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes Fallback;

	/** Add result V=A*B */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes V;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Compute the dot product between two vectors : DotProduct = A.B */
USTRUCT()
struct FDataflowVectorDotProductNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorDotProductNode, "VectorDotProduct", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "Project")

public:
	FDataflowVectorDotProductNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** A Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes A;

	/** B Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes B;

	/** Resulting dot product : DotProduct=A.B  */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes DotProduct;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Compute the Length of a vector : Length = |V| */
USTRUCT()
struct FDataflowVectorLengthNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorLengthNode, "VectorLength", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "Size Magnitude")

public:
	FDataflowVectorLengthNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Vector to get length from */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes V;

	/** Length of the input vector : Length=|V| */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes Length;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Compute the Squared length of a vector : Length = |V||V| */
USTRUCT()
struct FDataflowVectorSquaredLengthNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorSquaredLengthNode, "VectorSquaredLength", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "Size Magnitude")

public:
	FDataflowVectorSquaredLengthNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Vector to get squared length from */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes V;

	/** Length of the input vector : SquaredLength = |V||V| */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes SquaredLength;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Compute the distance between two vectors : Distance = |B-A| */
USTRUCT()
struct FDataflowVectorDistanceNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorDistanceNode, "VectorDistance", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "")

public:
	FDataflowVectorDistanceNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** A Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes A;

	/** B Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes B;

	/** Distance between A and B : Distance=|B-A| */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes Distance;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** 
* Compute the cross product of two vectors :  CrossProduct = B^A 
* This node only operates in 3 dimensions, inputs will be converted to a 3D vector internally and result will be a vector with a zero W component
*/
USTRUCT()
struct FDataflowVectorCrossProductNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorCrossProductNode, "VectorCrossProduct", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "")

public:
	FDataflowVectorCrossProductNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** A Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes A;

	/** B Vector operand */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes B;

	/** Resulting cross product of A and B : CrossProduct=B^A */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes CrossProduct;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Scale a vector by a scalar : Scaled = (V * Scale) */
USTRUCT()
struct FDataflowVectorScaleNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorScaleNode, "ScaleVector", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "Scalar Float Double")

public:
	FDataflowVectorScaleNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Vector to scale */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes V;

	/** Scale factor */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowNumericTypes Scale;

	/** Scaled vector : Scaled=(V * Scale) */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes Scaled;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Normalize a vector : Normalized = V/|V| */
USTRUCT()
struct FDataflowVectorNormalize : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowVectorNormalize, "NormalizeVector", DATAFLOW_MATH_VECTOR_NODES_CATEGORY, "")

public:
	FDataflowVectorNormalize(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Vector to normalize */
	UPROPERTY(EditAnywhere, Category = Operands, meta = (DataflowInput));
	FDataflowVectorTypes V;

	/** Normalized vector : Normalized=V/|V| */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVectorTypes Normalized;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterDataflowVectorNodes();
}
