// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "Misc/TVariant.h"

#include "DataflowAnyType.generated.h"

/** Any supported type */
USTRUCT()
struct FDataflowAnyType
{
	using FPolicyType = FDataflowAllTypesPolicy;
	using FStorageType = void;

	GENERATED_BODY()
	DATAFLOWCORE_API static const FName TypeName;
};

/** Any supported type */
USTRUCT()
struct FDataflowAllTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowAllTypesPolicy;
	using FStorageType = void;
	GENERATED_BODY()
};


/** Generic Array types */
USTRUCT()
struct FDataflowArrayTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowArrayTypePolicy;
	using FStorageType = void;
	GENERATED_BODY()
};

/** UStruct based types */
USTRUCT()
struct FDataflowUStructTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowUStructTypePolicy;
	using FStorageType = void;
	GENERATED_BODY()
};

UENUM()
enum class EDataflowNumericType : uint8
{
	Double UMETA(DisplayName = "double"),
	Float UMETA(DisplayName = "float"),
	Int64 UMETA(DisplayName = "int64"),
	UInt64 UMETA(DisplayName = "uint64"),
	Int32 UMETA(DisplayName = "int32"),
	UInt32 UMETA(DisplayName = "uint32"),
	Int16 UMETA(DisplayName = "int16"),
	UInt16 UMETA(DisplayName = "uint16"),
	Int8 UMETA(DisplayName = "int8"),
	UInt8 UMETA(DisplayName = "uint8")
};

/**
* Numeric types
* (double, float, int64, uint64, int32, uint32, int16, uint16, int8, uint8)"
*/
USTRUCT()
struct FDataflowNumericTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowNumericTypePolicy;
	using FStorageType = double;

	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Value)
	double Value = 0.0;
};

/**
* Vector types
* (2D, 3D and 4D vector, single and double precision)
*/
USTRUCT()
struct FDataflowVectorTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowVectorTypePolicy;
	using FStorageType = FVector4;

	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Value)
	FVector4 Value = FVector4(0, 0, 0, 0);
};

/** String types (FString or FName or FText) */
USTRUCT()
struct FDataflowStringTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowStringTypePolicy;
	using FStorageType = FString;

	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Value)
	FString Value;
};

/** Bool types  */
USTRUCT()
struct FDataflowBoolTypes : public FDataflowAnyType
{
	using FPolicyType = TDataflowSingleTypePolicy<bool>;
	using FStorageType = bool;

	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Value)
	bool Value = true;
};

/** Transform types  */
USTRUCT()
struct FDataflowTransformTypes : public FDataflowAnyType
{
	using FPolicyType = TDataflowSingleTypePolicy<FTransform>;
	using FStorageType = FTransform;

	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Value)
	FTransform Value = FTransform::Identity;
};

/**
* String convertible types
* (String types, Numeric types, Vector types and Booleans)
*/
USTRUCT()
struct FDataflowStringConvertibleTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowStringConvertibleTypePolicy;
	using FStorageType = FString;

	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Value)
	FString Value;
};

/** UObject types */
USTRUCT()
struct FDataflowUObjectConvertibleTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowUObjectConvertibleTypePolicy;
	using FStorageType = TObjectPtr<UObject>;

	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Value)
	TObjectPtr<UObject> Value = nullptr;
};

/** Selection types */
USTRUCT()
struct FDataflowSelectionTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowSelectionTypePolicy;
	using FStorageType = FDataflowSelection;

	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FDataflowSelection Value;
};

/** Vector array types */
USTRUCT()
struct FDataflowVectorArrayTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowVectorArrayPolicy;
	using FStorageType = TArray<FVector4>;
	
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Value)
	TArray<FVector4> Value;
};

/** Numeric array types */
USTRUCT()
struct FDataflowNumericArrayTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowNumericArrayPolicy;
	using FStorageType = TArray<double>;
	
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Value)
	TArray<double> Value;
};

/** String array types */
USTRUCT()
struct FDataflowStringArrayTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowStringArrayPolicy;
	using FStorageType = TArray<FString>;
	
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Value)
	TArray<FString> Value;
};

/** Bool array types */
USTRUCT()
struct FDataflowBoolArrayTypes : public FDataflowAnyType
{
	using FPolicyType = TDataflowSingleTypePolicy<TArray<bool>>;
	using FStorageType = TArray<bool>;
	
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Value)
	TArray<bool> Value;
};

/** Transform array types */
USTRUCT()
struct FDataflowTransformArrayTypes : public FDataflowAnyType
{
	using FPolicyType = TDataflowSingleTypePolicy<TArray<FTransform>>;
	using FStorageType = TArray<FTransform>;
	
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Value)
	TArray<FTransform> Value;
};

/** Rotation types */
USTRUCT()
struct FDataflowRotationTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowRotationTypePolicy;
	using FStorageType = FRotator;

	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FRotator Value = FRotator(EForceInit::ForceInitToZero);
};

/** Mesh Selection types */
USTRUCT()
struct FDataflowMeshSelectionTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowMeshSelectionTypePolicy;
	using FStorageType = FDataflowSelection;

	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FDataflowSelection Value;
};

namespace UE::Dataflow
{
	DATAFLOWCORE_API bool AreTypesCompatible(FName TypeA, FName TypeB);

	DATAFLOWCORE_API void RegisterAnyTypes();
}