// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Containers/Map.h"
#include "Dataflow/DataflowTypePolicy.h"

namespace UE::Dataflow
{
	struct FAnyTypesRegistry
	{
	public:
		using FTypeFilterFunction = TFunction<bool(FName, FName)>;

		template<typename T>
		static void RegisterTypeStatic(FName TypeName)
		{
			GetInstance().RegisterType<T>(TypeName);
		}
		DATAFLOWCORE_API static bool AreTypesCompatibleStatic(FName TypeA, FName TypeB);
		DATAFLOWCORE_API static FName GetStorageTypeStatic(FName Type);

		DATAFLOWCORE_API static void RegisterAutoConvertNodeTypeStatic(FName FromType, FName ToType, FName AutoConvertNode, FTypeFilterFunction TypeFilterFunction = {});
		DATAFLOWCORE_API static FName GetAutoConvertNodeTypeStatic(FName FromType, FName ToType);

		DATAFLOWCORE_API static bool IsAnyTypeStatic(FName Type);
		DATAFLOWCORE_API static FAnyTypesRegistry& GetInstance();

	private:
		typedef bool(*FSupportTypeFunction)(FName);

		struct FTypeInfo
		{
			FSupportTypeFunction SupportTypeFunction = nullptr;
			FName StorageType;
		};

		struct FAutoConvertNodeInfo
		{
			FName FromType;
			FName ToType;
			FName AutoConvertNode;
			FTypeFilterFunction TypeFilterFunction;
		};

		FAnyTypesRegistry() {};

		template<typename T>
		void RegisterType(FName TypeName)
		{
			FTypeInfo TypeInfo
			{
				.SupportTypeFunction = &T::FPolicyType::SupportsTypeStatic,
				.StorageType = FName(TDataflowPolicyTypeName<typename T::FStorageType>::GetName()),
			};
			TypeInfosByName.Emplace(TypeName, TypeInfo);
		}

		void RegisterAutoConvertNodeType(FName FromType, FName ToType, FName AutoConvertNode, FTypeFilterFunction TypeFilterFunction);

		bool AreTypesCompatible(FName TypeA, FName TypeB) const;
		FName GetStorageType(FName Type) const;
		FName GetAutoConvertNodeType(FName FromType, FName ToType) const;
		bool IsAnyType(FName Type) const;

		TMap<FName, FTypeInfo> TypeInfosByName;
		TArray<FAutoConvertNodeInfo> AutoConvertNodes;
	};
};

#define UE_DATAFLOW_REGISTER_ANYTYPE(Type) UE::Dataflow::FAnyTypesRegistry::RegisterTypeStatic<Type>(#Type)
#define UE_DATAFLOW_REGISTER_AUTOCONVERT(FromType, ToType, NodeType) UE::Dataflow::FAnyTypesRegistry::RegisterAutoConvertNodeTypeStatic(#FromType, #ToType, #NodeType)
#define UE_DATAFLOW_REGISTER_AUTOCONVERT_WITH_FILTER(FromType, ToType, NodeType, FilterFunc) UE::Dataflow::FAnyTypesRegistry::RegisterAutoConvertNodeTypeStatic(#FromType, #ToType, #NodeType, FilterFunc)
