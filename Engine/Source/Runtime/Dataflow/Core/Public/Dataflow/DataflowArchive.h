// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// The following code serializes the size of the PropertyData to serialize so that when loading,
// we can skip over it if the function could not be loaded.
// Serialize a temporary value for the delta in order to end up with an archive of the right size.
// Then serialize the PropertyData in order to get its size.
#define DATAFLOW_OPTIONAL_BLOCK_WRITE_BEGIN(){\
	const int64 NodeDataSizePos = Ar.Tell();\
	int64 NodeDataSize = 0;\
	Ar << NodeDataSize;\
	const int64 NodeBegin = Ar.Tell();


// Only go back and serialize the number of argument bytes if there is actually an underlying buffer to seek to.
// Come back to the temporary value we wrote and overwrite it with the PropertyData size we just calculated.
// And finally seek back to the end.
#define DATAFLOW_OPTIONAL_BLOCK_WRITE_END()\
	if (NodeDataSizePos != INDEX_NONE)\
	{\
		const int64 NodeEnd = Ar.Tell();\
		NodeDataSize = (NodeEnd - NodeBegin);\
		Ar.Seek(NodeDataSizePos);\
		Ar << NodeDataSize;\
		Ar.Seek(NodeEnd);\
	}\
	}


#define DATAFLOW_OPTIONAL_BLOCK_READ_BEGIN(COND) \
	{\
	int64 NodeDataSize = 0;\
	Ar << NodeDataSize;\
	if (COND) 

#define DATAFLOW_OPTIONAL_BLOCK_READ_ELSE() \
	else{\
	Ar.Seek(Ar.Tell() + NodeDataSize);\

#define DATAFLOW_OPTIONAL_BLOCK_READ_END() }}

