// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FLazySingleton;

namespace UE::Dataflow
{
	// registry for categories
	// This is used for node in editor filtering
	struct FCategoryRegistry
	{
	public:
		static DATAFLOWCORE_API FCategoryRegistry& Get();
		static DATAFLOWCORE_API void TearDown();

		using FCategoryName = FName;
		using FAssetType = FName;

		/** register a category name for a specific type of asset */
		DATAFLOWCORE_API void RegisterCategoryForAssetType(const FCategoryName Category, const FAssetType AssetType);
		DATAFLOWCORE_API bool IsCategoryForAssetType(const FCategoryName Category, const FAssetType AssetType) const;

	private:
		TMap<FCategoryName, TArray<FAssetType>> AssetTypesByCategory;

		friend FLazySingleton;
	};
}

#define UE_DATAFLOW_REGISTER_CATEGORY_FORASSET_TYPE(CATEGORY_NAME, ASSET_TYPE) \
	UE::Dataflow::FCategoryRegistry::Get().RegisterCategoryForAssetType(CATEGORY_NAME, ASSET_TYPE::StaticClass()->GetFName());
