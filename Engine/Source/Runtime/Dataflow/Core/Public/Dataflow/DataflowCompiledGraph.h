// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

namespace UE::Dataflow
{
	class FGraph;

	/** 
	* This structure stores a compiled graph
	* A compiled graph allow a dataflow evaluator to evaluate a graph efficiently without having to parse the graph again
	* Reference to nodes are sorted by tasks in order of execution and dependency
	* Making it easy to spawn actual async task from for example
	* The data is also organized in arrays of fixed sized structure for faster cache access and limited allocation
	* 
	* Note that a compiled graph is associated to a specific main or sub graph
	* 
	* Todo(dataflow) : A compiled graph do not yet provide any information about branching and conditional, this will be added in the near future 
	*/
	struct FCompiledGraph
	{
	public:
		DATAFLOWCORE_API FCompiledGraph(TSharedRef<const FGraph> InSourceGraph);

		DATAFLOWCORE_API int32 StartTask();

		/**
		* Add a node to the current task.
		* UpstreamNodeIds : nodes whose outputs feed this node's inputs. Must already have been
		*                  added to the compiled graph (topological order is preserved by the compiler).
		* bIsTerminal    : true if Node->IsTerminal() — cached here so evaluators can enumerate
		*                  terminals without walking the source FGraph.
		*/
		DATAFLOWCORE_API void AddNode(FGuid NodeId, TConstArrayView<FGuid> UpstreamNodeIds = {}, bool bIsTerminal = false);

		DATAFLOWCORE_API void AddDependency(int32 TaskId);
		DATAFLOWCORE_API void EndTask();

		DATAFLOWCORE_API int32 GetNumNodes() const;
		DATAFLOWCORE_API FGuid GetNextNodeId(int32 NodeIndex) const;

		/** O(1) lookup of a node's id by its index in the compiled graph. Returns invalid FGuid for out-of-range. */
		DATAFLOWCORE_API FGuid GetNodeId(int32 NodeIndex) const;

		/**
		* O(1) lookup of a node's index in the compiled graph by its FGuid.
		* Returns INDEX_NONE if the node is not part of this compiled graph.
		*/
		DATAFLOWCORE_API int32 FindNodeIndex(const FGuid& NodeId) const;

		/** Indices (into Nodes) of nodes whose outputs feed the given node. Empty for source nodes. */
		DATAFLOWCORE_API TConstArrayView<int32> GetUpstreamNodeIndices(int32 NodeIndex) const;

		/** Indices of all terminal nodes (nodes with IsTerminal() == true) in the compiled graph. */
		DATAFLOWCORE_API TConstArrayView<int32> GetTerminalNodeIndices() const;

		DATAFLOWCORE_API int32 GetNumTasks() const;
		DATAFLOWCORE_API void GetTasksNodesAndDependencies(int32 TaskId, TArray<FGuid>& OutNodeIds, TArray<int32>& OutParentTaskIds) const;

		DATAFLOWCORE_API TSharedRef<const FGraph> GetSourceGraph() const;
		
	private:
		enum class ECommandType
		{
			AddNode,		// Parameter is an index into a node array 
			AddDependency,	// Parameter is an index into a task array
			LaunchTask		// Parameter is an index into a task array 
		};

		struct FTask
		{
			/** Index into the command buffer ( Start of the task ) */
			int32 CommandIndex = INDEX_NONE; 

			/** Number of commands for this task */
			int32 NumCommands = 0;	
 		};

		struct FNode
		{
			/** Guid uniquely identifying a node in the graph */
			FGuid NodeId;

			/** index of the task in the task array */
			int32 TaskIndex = INDEX_NONE; // index into the task buffer

			/** CSR offset into UpstreamEdgesFlat for this node's upstream edges. */
			int32 UpstreamEdgeStart = 0;

			/** Number of upstream edges for this node. */
			int32 UpstreamEdgeCount = 0;
		};

		struct FCommand
		{
			/** Type of the command (see ECommandType) */
			ECommandType Type = ECommandType::LaunchTask;

			/** Parameter of the command (an index if the task or node array) */
			int32 Parameter = 0;
		};

		/** List of tasks (index into the command buffer array) */
		TArray<FTask> Tasks;

		/** List of nodes (with back reference to the task array) */
		TArray<FNode> Nodes;

		/** O(1) lookup table mapping a node's FGuid to its index in Nodes. */
		TMap<FGuid, int32> NodeIdToIndex;

		/** CSR storage of upstream-node-index edges for every node. Indexed via FNode::UpstreamEdgeStart/Count. */
		TArray<int32> UpstreamEdgesFlat;

		/** Pre-baked list of terminal node indices, populated at compile time from Node->IsTerminal(). */
		TArray<int32> TerminalNodeIndices;

		/** List of commands (with reference to the task and node arrays) */
		TArray<FCommand> Commands;

		TSharedRef<const FGraph> SourceGraph;
	};
}