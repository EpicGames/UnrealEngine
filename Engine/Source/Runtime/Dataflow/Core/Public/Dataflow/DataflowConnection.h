// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "DataflowConnection.generated.h"

class FProperty;
struct FDataflowNode;
struct FDataflowInput;
struct FDataflowOutput;

namespace UE::Dataflow
{

	template<class T> inline DATAFLOWCORE_API FName GraphConnectionTypeName();
	template<class T> inline DATAFLOWCORE_API T DeepCopy(const T&);

	struct FPin
	{
		enum class EDirection : uint8 {
			NONE = 0,
			INPUT,
			OUTPUT
		};
		EDirection Direction;
		FName Type;
		FName Name;
		bool bHidden = false;

		bool operator==(const FPin& Other) const
		{
			return Direction == Other.Direction && Type == Other.Type && Name == Other.Name && bHidden == Other.bHidden;
		}
		
		DATAFLOWCORE_API static const FPin InvalidPin;
	};

	struct FConnectionParameters
	{
		FConnectionParameters(FName InType = FName(), FName InName = FName(), FDataflowNode* InOwner = nullptr, const FProperty* InProperty = nullptr, uint32 InOffset = INDEX_NONE, FGuid InGuid = FGuid::NewGuid())
			: Type(InType)
			, Name(InName)
			, Owner(InOwner)
			, Property(InProperty)
			, Offset(InOffset)
			, Guid(InGuid)
		{}

		FName Type;
		FName Name;
		FDataflowNode* Owner = nullptr;
		const FProperty* Property = nullptr;
		uint32 Offset = INDEX_NONE;
		FGuid Guid;
	};


	// Do not hold onto FConnectionReference when Reference is dynamically allocated (e.g., when using array inputs).
	// Use FConnectionKey instead.
	struct FConnectionReference
	{
		const void* Reference;
		int32 Index = INDEX_NONE;
		const void* ContainerReference = nullptr;

		FConnectionReference(const void* InReference, int32 InIndex = INDEX_NONE, const void* InContainerReference = nullptr)
			: Reference(InReference)
			, Index(InIndex)
			, ContainerReference(InContainerReference)
		{}
	};

	template<typename T>
	struct TConnectionReference : public FConnectionReference
	{
		TConnectionReference(const T* InReference, int32 InIndex = INDEX_NONE, const void* InContainerReference = nullptr)
			: FConnectionReference(InReference, InIndex, InContainerReference)
		{}
	};

	class FConnectionKey
	{
	public:
		FConnectionKey() = default;

		bool operator==(const FConnectionKey& Other) const
		{
			return Offset == Other.Offset && ContainerIndex == Other.ContainerIndex && ContainerElementOffset == Other.ContainerElementOffset;
		}

		friend uint32 GetTypeHash(const FConnectionKey& Key)
		{
			return HashCombineFast(HashCombineFast(GetTypeHash(Key.Offset), GetTypeHash(Key.ContainerIndex)), GetTypeHash(Key.ContainerElementOffset));
		}

		static const FConnectionKey Invalid;

	private:
		friend struct ::FDataflowConnection;
		friend struct ::FDataflowInput;
		friend struct ::FDataflowOutput;
		friend struct ::FDataflowNode;
		FConnectionKey(uint32 InOffset, int32 InContainerIndex, uint32 InContainerElementOffset)
			: Offset(InOffset)
			, ContainerIndex(InContainerIndex)
			, ContainerElementOffset(InContainerElementOffset)
		{}

		uint32 Offset = INDEX_NONE;
		int32 ContainerIndex = INDEX_NONE;
		uint32 ContainerElementOffset = INDEX_NONE;
	};

	class FGraph;
}

//
// Input Output Base
//
USTRUCT()
struct FDataflowConnection
{
	GENERATED_USTRUCT_BODY()

protected:
	FDataflowNode* OwningNode = nullptr;
	const FProperty* Property = nullptr;
	FGuid  Guid;
	IDataflowTypePolicy* TypePolicy = nullptr;
	uint32 Offset;
	UE::Dataflow::FPin::EDirection Direction;
	bool bIsAnyType:1 = false;
	bool bHasConcreteType : 1 = false;
	bool bCanHidePin:1 = false;
	bool bPinIsHidden:1 = false;

private:
	FName OriginalType;
	FName Type;
	FName Name;

	/** Lock the type from being changed */
	bool bLockType:1 = false;

	// Name of the group dependecy the connection is part of 
	// This applies to anytype connections
	// All other connection with the same group will inherit type from each other
	FName TypeDependencyGroup;

	friend struct FDataflowNode;
	friend class UE::Dataflow::FGraph;

protected:
	DATAFLOWCORE_API void SetTypeInternal(FName NewType);

	DATAFLOWCORE_API bool IsOwningNodeEnabled() const;

	/** this should only be used for serialization */
	DATAFLOWCORE_API void SetAsAnyType(bool bAnyType, const FName& ConcreteType);

	/** this should only be used for serialization - for support of old simple TArray types*/
	DATAFLOWCORE_API void ForceSimpleType(FName InType);
	DATAFLOWCORE_API void FixAndPropagateType();
	virtual void FixAndPropagateType(FName InType) { ensure(false); }

	/** 
	* returns true is the parameter type is an extension of the current type 
	* TArray<int> from TArray for example 
	*/
	DATAFLOWCORE_API bool IsExtendedType(FName InType) const;

	/**
	* reset the output to its original type 
	* no-op for non enytypes types
	* Anytypes connection will fail if it is already connected or has connected type dependencies
	* @return true if the change was made 
	*/
	DATAFLOWCORE_API bool ResetToOriginalType();

	/* Rename the connection */
	DATAFLOWCORE_API void Rename(FName NewName);

	DATAFLOWCORE_API void ForceTypeDependencyGroup(FName InTypeDependencyGroup);

public:
	FDataflowConnection() {};
	DATAFLOWCORE_API FDataflowConnection(UE::Dataflow::FPin::EDirection Direction, const UE::Dataflow::FConnectionParameters& Params);
	virtual ~FDataflowConnection() = default;

	FDataflowNode* GetOwningNode() { return OwningNode; }
	const FDataflowNode* GetOwningNode() const { return OwningNode; }

	DATAFLOWCORE_API FGuid GetOwningNodeGuid() const;
	DATAFLOWCORE_API uint32 GetOwningNodeValueHash() const;
	DATAFLOWCORE_API UE::Dataflow::FTimestamp GetOwningNodeTimestamp() const;

	const FProperty* GetProperty() const { return Property; }

	DATAFLOWCORE_API FString GetPropertyTooltip() const;
	DATAFLOWCORE_API FString GetPropertyTypeNameTooltip() const;

	UE::Dataflow::FPin::EDirection GetDirection() const { return Direction; }
	uint32 GetOffset() const { return Offset; }
	virtual int32 GetContainerIndex() const { return INDEX_NONE; }
	virtual uint32 GetContainerElementOffset() const { return INDEX_NONE; }
	UE::Dataflow::FConnectionKey GetConnectionKey() const 
	{		
		return UE::Dataflow::FConnectionKey(GetOffset(), GetContainerIndex(), GetContainerElementOffset());
	}

	/**
	* Set the concrete type of an anytype connection
	* Changing the type may fail or assert if :
	*	- the connection is connected to another one, 
	*	- the type is incompatible with the original type 
	*	- the type has any type dependencies connected
	*	- the type is locked ( see LockType() )
	* (see IsSafeToTryChangingType)
	* @return true if the type was effectivelly changed
	*/
	DATAFLOWCORE_API bool SetConcreteType(FName InType);

	void LockType() { bLockType = true; }
	void UnlockType() { bLockType = false; }
	bool IsTypeLocked() const { return bLockType; }

	FName GetType() const { return Type; }
	FName GetOriginalType() const { return OriginalType; }

	template <typename T>
	bool IsType() const
	{
		return Type == UE::Dataflow::GetTypeName<T>();
	}

	FGuid GetGuid() const { return Guid; }
	void SetGuid(FGuid InGuid) { Guid = InGuid; }
	
	FName GetName() const { return Name; }
	void SetName(FName InName) { Name = InName; }

	virtual void* RealAddress() const { ensure(OwningNode);  return (void*)((size_t)OwningNode + (size_t)GetOffset()); };
	UE::Dataflow::FContextCacheKey CacheKey() const { return GetTypeHash(Guid); };

	/* 
	* check is this connection can chage type 
	* case where it cannot are : 
	*	it is already connected
	*	it has copnnected type dependencies
	*/
	DATAFLOWCORE_API bool IsSafeToTryChangingType() const;

	virtual bool IsConnected() const { return false; }
	virtual bool AddConnection(FDataflowConnection* In) { return false; };
	virtual bool RemoveConnection(FDataflowConnection* In) { return false; }
	virtual void GetConnections(TArray<FDataflowConnection*>& OutConnections) const {};

	bool IsAnyType() const { return bIsAnyType; }
	DATAFLOWCORE_API static bool IsAnyType(const FName& InType);
	bool HasConcreteType() const { return bHasConcreteType; }
	DATAFLOWCORE_API void SetTypePolicy(IDataflowTypePolicy* InTypePolicy);
	DATAFLOWCORE_API bool SupportsType(FName InType) const;

	/**
	* return the short list of supported types
	* if the type of the connection is a broad anytype and that the list of supported type is not finite, then nothing is returned
	*/
	DATAFLOWCORE_API void GetSupportedTypes(TArray<FName>& OutSupportedTypes) const;


	DATAFLOWCORE_API FDataflowConnection& SetTypeDependencyGroup(FName DependencyGroupName);
	FName GetTypeDependencyGroup() const { return TypeDependencyGroup;	}

	DATAFLOWCORE_API bool IsAnytypeDependencyConnected() const;

	template<class T>
	bool IsA(const T* InVar) const
	{
		return (size_t)RealAddress() == (size_t)InVar;
	}

	virtual void Invalidate(const UE::Dataflow::FTimestamp& ModifiedTimestamp = UE::Dataflow::FTimestamp::Current()) {};

	bool GetCanHidePin() const { return bCanHidePin; }
	bool GetPinIsHidden() const { return bCanHidePin && bPinIsHidden; }
	FDataflowConnection& SetCanHidePin(bool bInCanHidePin) 
	{
		bCanHidePin = bInCanHidePin; 
		return *this; 
	}
	FDataflowConnection& SetPinIsHidden(bool bInPinIsHidden)
	{
		bPinIsHidden = bInPinIsHidden;
		return *this;
	}

	/**
	* extract the type name from a FProperty 
	* this method make sure the resulting typename is consistent with Dataflow standard where there's no space in the name
	* for example FProperty returns : "TArray<TObjectPtr<UObject> >" but dataflow compatible typename is "TArray<TObjectPtr<UObject>>"
	*/
	DATAFLOWCORE_API static FName GetTypeNameFromProperty(const FProperty* Property);

private:
	void InitFromType();
};
