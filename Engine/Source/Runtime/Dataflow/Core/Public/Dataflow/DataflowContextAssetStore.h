// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"
#include "Containers/StringFwd.h"
#include "UObject/GCObject.h"

class UObject;

namespace UE::Dataflow
{
	/** interface to be implemented by the context that expose the asset store API */
	struct IContextAssetStoreInterface
	{
	public:
		/**
		* Typed version of the AddAsset virtual method
		* @see AddAsset
		*/
		template <typename T>
		inline T* AddAssetTyped(const FString& AssetPath)
		{
			return Cast<T>(AddAsset(AssetPath, T::StaticClass()));
		}

		/**
		* Create an new asset to be commited later ( when the terminal nodes are evaluated )
		* When commited, the asset will be duplicated to a final package matching PersistentAssetPath
		* @return an UObject owned by the transient package
		*/
		virtual UObject* AddAsset(const FString& PersistentAssetPath, const UClass* AssetClass) = 0;

		/**
		* Commit the asset matching the TransientAssetPath
		* If such asset was added previously this will duplicate it and save it to a persistent package
		* @return the newly created persistent UObject asset
		*/
		virtual UObject* CommitAsset(const FString& TransientAssetPath) = 0;

		/**
		* Clear all assets from the store
		*/
		virtual void ClearAssets() = 0;

	};

	/** asset store to managed creation and storage of assets during the evaluation of the Dataflow graph */
	struct FContextAssetStore: public FGCObject
	{
	public:
		/**
		* Create an new asset to be commited later ( when the terminal nodes are evaluated )
		* When commited, the asset will be duplicated to a final package matching PersistentAssetPath
		* @return an UObject owned by the transient package
		*/
		DATAFLOWCORE_API UObject* AddAsset(const FString& PersistentAssetPath, const UClass* AssetClass);

		/**
		* Commit the asset matching the TransientAssetPath 
		* If such asset was added previously this will duplicate it and save it to a persistent package
		* @return the newly created persistent UObject asset
		*/
		DATAFLOWCORE_API UObject* CommitAsset(const FString& TransientAssetPath);

		/**
		* Commit the asset matching the TransientAsset
		* If such asset was added previously this will duplicate it and save it to a package matching AssetPath
		* @return the newly created persistent UObject asset
		*/
		DATAFLOWCORE_API UObject* CommitAsset(const UObject* TransientAsset);
		
		/**
		* Clear all assets from the store 
		*/
		DATAFLOWCORE_API void ClearAssets();

		//~ Begin FGCObject interface
		DATAFLOWCORE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override { return TEXT("FContextAssetStore"); }
		//~ End FGCObject interface

	private:
		struct FAssetData
		{
			FString				PersistentPath;
			FString				TransientPath;
			TObjectPtr<UObject> TransientAsset = nullptr;
		};

		const FAssetData* FindByTransientPath(const FString& TransientAssetPath) const;
		const FAssetData* FindByTransientAsset(const UObject* TransientAsset) const;

		UObject* CommitAsset(const FAssetData& AssetData);

		TArray<FAssetData> AssetData;
	};

}


