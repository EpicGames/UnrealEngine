// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructArrayView.h"
#include "Templates/Decay.h"
#include "Templates/UniquePtr.h"
#include "UObject/GCObject.h"
#include "UObject/UnrealType.h"
#include "Dataflow/DataflowTypePolicy.h"

namespace UE::Dataflow
{
	class FContext;
	struct FContextCacheElementBase;
	struct FContextCacheElementNull;

	template<class T>
	struct TContextCacheElementUObjectArray;

	typedef uint32 FContextCacheKey;

	/** Trait used to select the UObject* or TObjectPtr cache element path code. */
	template <typename T>
	struct TIsUObjectPtrElement
	{
		typedef typename TDecay<T>::Type Type;
		static constexpr bool Value = (std::is_pointer_v<Type> && std::is_convertible_v<Type, const UObjectBase*>) || TIsTObjectPtr_V<Type>;
	};

	namespace Private
	{
		// True if TBaseStructure<T> provides a Get() (USTRUCT or explicit TBaseStructure specialization).
		template <typename T, typename = void>
		struct THasTBaseStructureGet { static constexpr bool Value = false; };
		template <typename T>
		struct THasTBaseStructureGet<T, std::void_t<decltype(&TBaseStructure<T>::Get)>> { static constexpr bool Value = true; };

		// True if T is reflected via a TVariantStructure specialization (LWC float/double core variant types).
		// we need to explicitly list them ( like in many other place in the engine ) because TVariantStructure will static assert in its default implementation 
		// IMPORTANT : Keep in sync with UE_DECLARE_CORE_VARIANT_TYPE invocations in CoreUObject/Public/UObject/Class.h.
		template <typename T> struct TIsReflectedCoreVariant { static constexpr bool Value = false; };
		template <> struct TIsReflectedCoreVariant<FVector2f>    { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FVector2d>    { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FVector3f>    { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FVector3d>    { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FVector4f>    { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FVector4d>    { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FPlane4f>     { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FPlane4d>     { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FQuat4f>      { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FQuat4d>      { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FRotator3f>   { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FRotator3d>   { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FTransform3f> { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FTransform3d> { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FMatrix44f>   { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FMatrix44d>   { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FBox2f>       { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FBox2d>       { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FRay3f>       { static constexpr bool Value = true; };
		template <> struct TIsReflectedCoreVariant<FRay3d>       { static constexpr bool Value = true; };

		// Resolves to TBaseStructure<T>::Get() when available, otherwise to TVariantStructure<T>::Get()
		// (covers LWC float/double variants like FVector3f, FQuat4f, FTransform3d, ...).
		template <typename T>
		UScriptStruct* GetReflectedScriptStruct()
		{
			using FStruct = typename TDecay<T>::Type;
			if constexpr (THasTBaseStructureGet<FStruct>::Value)
			{
				return TBaseStructure<FStruct>::Get();
			}
			else if constexpr (TIsReflectedCoreVariant<FStruct>::Value)
			{
				return TVariantStructure<FStruct>::Get();
			}
			else
			{
				static_assert(sizeof(FStruct) == 0, "T is not reflected via TBaseStructure or TVariantStructure");
				return nullptr;
			}
		}
	}

	/** Trait used to select the UStruct cache element path code. */
	template <typename T>
	struct TIsReflectedStruct
	{
		static constexpr bool Value =
			Private::THasTBaseStructureGet<typename TDecay<T>::Type>::Value ||
			Private::TIsReflectedCoreVariant<typename TDecay<T>::Type>::Value;
	};

	struct FTimestamp
	{
		typedef uint64 Type;
		Type Value = Type(0);

		FTimestamp(Type InValue) : Value(InValue) {}
		bool operator>=(const FTimestamp& InTimestamp) const { return Value >= InTimestamp.Value; }
		bool operator<(const FTimestamp& InTimestamp) const { return Value < InTimestamp.Value; }
		bool operator==(const FTimestamp& InTimestamp) const { return Value == InTimestamp.Value; }
		bool IsInvalid() { return Value == Invalid; }
		bool IsInvalid() const { return Value == Invalid; }

		static DATAFLOWCORE_API Type Current();
		static DATAFLOWCORE_API Type Invalid; // 0
	};

	struct IContextCacheStore
	{
		virtual const TUniquePtr<FContextCacheElementBase>* FindCacheElement(FContextCacheKey Key) const = 0;
		virtual bool HasCacheElement(FContextCacheKey Key, FTimestamp InTimestamp = FTimestamp::Invalid) const = 0;
	};

	//--------------------------------------------------------------------
	// base class for all context cache entries 
	//--------------------------------------------------------------------
	struct FContextCacheElementBase 
	{
		enum EType
		{
			CacheElementTyped,
			CacheElementReference,
			CacheElementNull,
			CacheElementUObject, // UObjectPtr
			CacheElementUObjectArray, // TArray of UObjectPtr
			CacheElementUStruct,
			CacheElementUStructArray
		};

		FContextCacheElementBase(EType CacheElementType, FGuid InNodeGuid = FGuid(), const FProperty* InProperty = nullptr, uint32 InNodeHash = 0, FTimestamp InTimestamp = FTimestamp::Invalid)
			: Type(CacheElementType)
			, NodeGuid(InNodeGuid)
			, Property(InProperty)
			, NodeHash(InNodeHash)
			, Timestamp(InTimestamp)
		{}
		virtual ~FContextCacheElementBase() = default;

		// InReferenceDataKey is the key of the cache element this function is called on 
		inline TUniquePtr<FContextCacheElementBase> CreateReference(FContextCacheKey InReferenceDataKey) const;

		// clone the cache entry
		virtual TUniquePtr<FContextCacheElementBase> Clone(const IContextCacheStore& Context) const = 0;

		template<typename T>
		inline const T& GetTypedData(const IContextCacheStore& Context, const FProperty* PropertyIn, const T& Default) const;

		virtual const void* GetUntypedData(const IContextCacheStore& Context, const FProperty* PropertyIn) const = 0;

		virtual bool IsArray(const IContextCacheStore& Context) const = 0;
		virtual int32 GetNumArrayElements(const IContextCacheStore& Context) const = 0;
		virtual TUniquePtr<FContextCacheElementBase> CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const = 0;
		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const = 0;
		// DereferencedElements will not be FContextCacheElementReference and instead be the final element any reference stored.
		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const = 0;
		// DereferencedElements will not be FContextCacheElementReference and instead be the final element any reference stored.
		virtual TUniquePtr<FContextCacheElementBase> AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const = 0;

		// Get the matching const struct view if available, otherwise return an invalid one 
		virtual FConstStructView GetConstStructView(const IContextCacheStore& Context) const { return {}; }

		EType GetType() const {	return Type; }

		const FProperty* GetProperty() const { return Property; }
		const FTimestamp& GetTimestamp() const { return Timestamp; }
		void SetTimestamp(const FTimestamp& InTimestamp) { Timestamp = InTimestamp; }

		const FGuid& GetNodeGuid() const { return NodeGuid; }
		const uint32 GetNodeHash() const { return NodeHash; }

		// use this with caution: setting the property of a wrong type may cause problems
		void SetProperty(const FProperty* NewProperty) { Property = NewProperty; }

		// use this with caution: setting the property of a wrong type may cause problems
		void UpdatePropertyAndNodeData(const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp)
		{ 
			Property = InProperty;
			NodeGuid = InNodeGuid;
			NodeHash = InNodeHash;
			Timestamp = InTimestamp;
		}

	private:
		friend struct FContextCache;

		EType Type;
		FGuid NodeGuid;
		const FProperty* Property = nullptr;
		uint32 NodeHash = 0;
		FTimestamp Timestamp = FTimestamp::Invalid;
	};

	//--------------------------------------------------------------------
	// Value storing context cache entry - strongly typed
	//--------------------------------------------------------------------
	template<class T>
	struct TContextCacheElement : public FContextCacheElementBase 
	{
		TContextCacheElement(FGuid InNodeGuid, const FProperty* InProperty, T&& InData, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementTyped, InNodeGuid, InProperty, InNodeHash, Timestamp)
			, Data(Forward<T>(InData))
		{}

		TContextCacheElement(const TContextCacheElement<T>& Other)
			: FContextCacheElementBase(EType::CacheElementTyped, Other.GetNodeGuid(), Other.GetProperty(), Other.GetNodeHash(), Other.GetTimestamp())
			, Data(Other.Data)
		{}

		inline const T& GetData(const IContextCacheStore& Context, const FProperty* PropertyIn, const T& Default) const;

		virtual const void* GetUntypedData(const IContextCacheStore& Context, const FProperty* PropertyIn) const
		{
			return &Data;
		}

		virtual bool IsArray(const IContextCacheStore& Context) const override
		{
			return (TIsTArray<FDataType>::Value);
		}

		virtual int32 GetNumArrayElements(const IContextCacheStore& Context) const override
		{
			if constexpr (TIsTArray<FDataType>::Value)
			{
				return Data.Num();
			}
			else
			{
				return 0;
			}
		}

		virtual TUniquePtr<FContextCacheElementBase> CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			if constexpr (TIsTArray<FDataType>::Value)
			{
				if (Data.IsValidIndex(Index))
				{
					return MakeUnique<TContextCacheElement<decltype(Data[Index])>>(InNodeGuid, InProperty, Data[Index], InNodeHash, InTimestamp);
				}
				return {};
			}
			else
			{
				return {};
			}
		}

		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			if constexpr (TIsTArray<FDataType>::Value)
			{
				return {}; // already an Array
			}
			else 
			{
				TArray<FDataType> Array{ Data };
				return MakeUnique<TContextCacheElement<TArray<FDataType>>>(InNodeGuid, InProperty, MoveTemp(Array), InNodeHash, InTimestamp);
			}
		}

		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			if constexpr (TIsTArray<FDataType>::Value)
			{
				return {}; // already an Array
			}
			else
			{
				TArray<FDataType> Result{ Data };
				Result.Reserve(DereferencedElements.Num() + 1);
				for (const FContextCacheElementBase* Element : DereferencedElements)
				{
					check(Element);
					check(Element->GetType() == EType::CacheElementTyped);
					const void* UntypedElementData = Element->GetUntypedData(Context, Element->GetProperty());
					check(UntypedElementData);
					Result.Emplace(*static_cast<const FDataType*>(UntypedElementData));
				}

				return MakeUnique<TContextCacheElement<TArray<FDataType>>>(InNodeGuid, InProperty, MoveTemp(Result), InNodeHash, InTimestamp);
			}
		}

		virtual TUniquePtr<FContextCacheElementBase> AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			if constexpr (TIsTArray<FDataType>::Value)
			{
				FDataType Result(Data);
				for (const FContextCacheElementBase* Element : DereferencedElements)
				{
					check(Element);
					check(Element->GetType() == EType::CacheElementTyped);
					const void* UntypedElementData = Element->GetUntypedData(Context, Element->GetProperty());
					check(UntypedElementData);
					Result.Append(*static_cast<const FDataType*>(UntypedElementData));
				}
				return MakeUnique<TContextCacheElement<FDataType>>(InNodeGuid, InProperty, MoveTemp(Result), InNodeHash, InTimestamp);
			}
			else
			{
				return {};
			}
		}

		const T& GetDataDirect() const { return Data; }

		inline virtual TUniquePtr<FContextCacheElementBase> Clone(const IContextCacheStore& Context) const override;

	private:
		typedef typename TDecay<T>::Type FDataType;  // Using universal references here means T could be either const& or an rvalue reference
		const FDataType Data;                        // Decaying T removes any reference and gets the correct underlying storage data type
	};

	//--------------------------------------------------------------------
	// Reference to another context cache entry 
	//--------------------------------------------------------------------
	struct FContextCacheElementReference : public FContextCacheElementBase
	{
		FContextCacheElementReference(FGuid InNodeGuid, const FProperty* InProperty, FContextCacheKey InDataKey, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementReference, InNodeGuid, InProperty, InNodeHash, Timestamp)
			, DataKey(InDataKey)
		{}


		template<class T>
		inline const T& GetData(const IContextCacheStore& Context, const FProperty* PropertyIn, const T& Default) const;

		virtual const void* GetUntypedData(const IContextCacheStore& Context, const FProperty* PropertyIn) const override;

		virtual bool IsArray(const IContextCacheStore& Context) const override;

		virtual int32 GetNumArrayElements(const IContextCacheStore& Context) const override;

		virtual TUniquePtr<FContextCacheElementBase> CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;

		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;

		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;

		virtual TUniquePtr<FContextCacheElementBase> AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;

		virtual FConstStructView GetConstStructView(const IContextCacheStore& Context) const;

		virtual TUniquePtr<FContextCacheElementBase> Clone(const IContextCacheStore& Context) const override;

		// Traverse references until the non-reference element is found.
		DATAFLOWCORE_API const FContextCacheElementBase* GetReferencedElement(const IContextCacheStore& Context) const;

	private:
		const FContextCacheKey DataKey; // this is a key to another cache element
	};

	//--------------------------------------------------------------------
	// Null entry, this will always return a default value 
	//--------------------------------------------------------------------
	struct FContextCacheElementNull : public FContextCacheElementBase
	{
		//
		// IMPORTANT: 
		// Timestamp must be set to (Timestamp.Value - 1) to make sure that this type of entry is always invalid
		//
		UE_DEPRECATED(5.6, "Use the other constructor that does not pass a DataKey (the key is not needed)")
		FContextCacheElementNull(FGuid InNodeGuid, const FProperty* InProperty, FContextCacheKey InDataKey, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementNull, InNodeGuid, InProperty, InNodeHash, FTimestamp((Timestamp.Value == 0)? 0: (Timestamp.Value - 1)))
		{}

		FContextCacheElementNull(FGuid InNodeGuid, const FProperty* InProperty, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementNull, InNodeGuid, InProperty, InNodeHash, FTimestamp((Timestamp.Value == 0) ? 0 : (Timestamp.Value - 1)))
		{}

		virtual const void* GetUntypedData(const IContextCacheStore& Context, const FProperty* PropertyIn) const override;

		virtual bool IsArray(const IContextCacheStore& Context) const override;
		virtual int32 GetNumArrayElements(const IContextCacheStore& Context) const override;
		virtual TUniquePtr<FContextCacheElementBase> CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;
		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;
		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;
		virtual TUniquePtr<FContextCacheElementBase> AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;

		virtual TUniquePtr<FContextCacheElementBase> Clone(const IContextCacheStore& Context) const override;
	};

	//--------------------------------------------------------------------
	// UObject cache element, prevents the object from being garbage collected while in the cache
	//--------------------------------------------------------------------
	template<class T>
	struct TContextCacheElementUObject : public FContextCacheElementBase, public FGCObject
	{
		TContextCacheElementUObject(FGuid InNodeGuid, const FProperty* InProperty, T&& InObjectPtr, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementUObject, InNodeGuid, InProperty, InNodeHash, Timestamp)
			, ObjectPtr(InObjectPtr)
		{}

		TContextCacheElementUObject(const TContextCacheElementUObject<T>& Other)
			: FContextCacheElementBase(EType::CacheElementUObject, Other.GetNodeGuid(), Other.GetProperty(), Other.GetNodeHash(), Other.GetTimestamp())
			, ObjectPtr(Other.ObjectPtr)
		{}

		const T& GetData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/, const T& /*Default*/) const
		{
			return ObjectPtr;
		}

		virtual const void* GetUntypedData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/) const override
		{
			return (void*)ObjectPtr;
		}

		virtual bool IsArray(const IContextCacheStore& Context) const override
		{
			return false;
		}

		virtual int32 GetNumArrayElements(const IContextCacheStore& Context) const override
		{
			return 0;
		}

		virtual TUniquePtr<FContextCacheElementBase> CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			return {};
		}

		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			TArray<FDataType> Array{ ObjectPtr };
			return MakeUnique<TContextCacheElementUObjectArray<FDataType>>(InNodeGuid, InProperty, MoveTemp(Array), InNodeHash, InTimestamp);
		}

		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			TArray<FDataType> Result{ ObjectPtr };
			Result.Reserve(DereferencedElements.Num() + 1);
			for (const FContextCacheElementBase* Element : DereferencedElements)
			{
				check(Element);
				check(Element->GetType() == EType::CacheElementUObject);
				const TContextCacheElementUObject<T>* TypedElement = static_cast<const TContextCacheElementUObject<T>*>(Element);
				Result.Emplace(TypedElement->ObjectPtr);
			}

			return MakeUnique<TContextCacheElementUObjectArray<FDataType>>(InNodeGuid, InProperty, MoveTemp(Result), InNodeHash, InTimestamp);
		}

		virtual TUniquePtr<FContextCacheElementBase> AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			return {};
		}

		inline virtual TUniquePtr<FContextCacheElementBase> Clone(const IContextCacheStore& Context) const override;

		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override { Collector.AddReferencedObject(ObjectPtr); }
		virtual FString GetReferencerName() const override { return TEXT("TContextCacheElementUObject"); }
		//~ End FGCObject interface

	private:
		typedef typename TDecay<T>::Type FDataType;  // Using universal references here means T could be either const& or an rvalue reference
		FDataType ObjectPtr;                         // Decaying T removes any reference and gets the correct underlying storage data type
	};

	//--------------------------------------------------------------------
	// TArray<UObjectPtr> cache element, prevents the object from being garbage collected while in the cache
	//--------------------------------------------------------------------
	template<class T>
	struct TContextCacheElementUObjectArray : public FContextCacheElementBase, public FGCObject
	{
		template<typename ArrayT>
		TContextCacheElementUObjectArray(FGuid InNodeGuid, const FProperty* InProperty, ArrayT&& InObjectPtrArray, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementUObjectArray, InNodeGuid, InProperty, InNodeHash, Timestamp)
			, ObjectPtrArray(Forward<ArrayT>(InObjectPtrArray))
		{}


		TContextCacheElementUObjectArray(const TContextCacheElementUObjectArray<T>& Other)
			: FContextCacheElementBase(EType::CacheElementUObjectArray, Other.GetNodeGuid(), Other.GetProperty(), Other.GetNodeHash(), Other.GetTimestamp())
			, ObjectPtrArray(Other.ObjectPtrArray)
		{}

		virtual bool IsArray(const IContextCacheStore& Context) const override
		{
			return true;
		}

		const TArray<T>& GetData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/, const TArray<T>& /*Default*/) const
		{
			return ObjectPtrArray;
		}

		virtual const void* GetUntypedData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/) const override
		{
			return &ObjectPtrArray;
		}

		virtual int32 GetNumArrayElements(const IContextCacheStore& Context) const override
		{
			return ObjectPtrArray.Num();
		}

		virtual TUniquePtr<FContextCacheElementBase> CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			if (ObjectPtrArray.IsValidIndex(Index))
			{
				FDataType Element = ObjectPtrArray[Index];
				return MakeUnique<TContextCacheElementUObject<FDataType>>(InNodeGuid, InProperty, MoveTemp(Element), InNodeHash, InTimestamp);
			}
			return {};
		}

		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			return {}; // no array of array 
		}

		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			return {}; // no array of array 
		}

		virtual TUniquePtr<FContextCacheElementBase> AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override
		{
			TArray<FDataType> Result(ObjectPtrArray);
			for (const FContextCacheElementBase* Element : DereferencedElements)
			{
				check(Element);
				check(Element->GetType() == EType::CacheElementUObjectArray);
				const void* UntypedElementData = Element->GetUntypedData(Context, Element->GetProperty());
				check(UntypedElementData);
				Result.Append(*static_cast<const TArray<FDataType>*>(UntypedElementData));
			}
			return MakeUnique< TContextCacheElementUObjectArray<FDataType>>(InNodeGuid, InProperty, MoveTemp(Result), InNodeHash, InTimestamp);
		}

		inline virtual TUniquePtr<FContextCacheElementBase> Clone(const IContextCacheStore& Context) const override;

		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override { Collector.AddReferencedObjects(ObjectPtrArray); }
		virtual FString GetReferencerName() const override { return TEXT("TContextCacheElementUObjectArray"); }
		//~ End FGCObject interface

	private:
		typedef typename TDecay<T>::Type FDataType;     // Using universal references here means T could be either const& or an rvalue reference
		TArray<FDataType> ObjectPtrArray;               // Decaying T removes any reference and gets the correct underlying storage data type
	};

	//--------------------------------------------------------------------
	// UStruct cache element
	//--------------------------------------------------------------------
	struct FContextCacheElementUStruct : public FContextCacheElementBase, public FGCObject
	{
		explicit FContextCacheElementUStruct(FGuid InNodeGuid, const FProperty* InProperty, const FConstStructView& StructView, uint32 InNodeHash, FTimestamp Timestamp);

		template<typename T>
		explicit FContextCacheElementUStruct(FGuid InNodeGuid, const FProperty* InProperty, T&& InStruct, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementUStruct, InNodeGuid, InProperty, InNodeHash, Timestamp)
		{
			using FStruct = typename TDecay<T>::Type;
			if constexpr (Private::THasTBaseStructureGet<FStruct>::Value)
			{
				InstancedStruct.InitializeAs<FStruct>(Forward<T>(InStruct));
			}
			else
			{
				// LWC variant types (FVector3f, FQuat4f, ...) are specialized in TVariantStructure, not TBaseStructure,
				// so FInstancedStruct::InitializeAs<T> can't be used; fall back to the raw (UScriptStruct*, memory) overload.
				static_assert(Private::TIsReflectedCoreVariant<FStruct>::Value, "T is not reflected via TBaseStructure or TVariantStructure");
				InstancedStruct.InitializeAs(TVariantStructure<FStruct>::Get(), reinterpret_cast<const uint8*>(&InStruct));
			}
		}

		explicit FContextCacheElementUStruct(const FContextCacheElementUStruct& Other);

		template<typename T>
		inline const T& GetData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/, const T& Default) const
		{
			using FStruct = typename TDecay<T>::Type;
			if constexpr (Private::THasTBaseStructureGet<FStruct>::Value)
			{
				return InstancedStruct.Get<T>();
			}
			else 
			{
				if (ensure(InstancedStruct.GetScriptStruct() == TVariantStructure<FStruct>::Get()))
				{
					return *reinterpret_cast<const T*>(InstancedStruct.GetMemory());
				}
				return Default;
			}
		}

		DATAFLOWCORE_API virtual const void* GetUntypedData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/) const override;

		DATAFLOWCORE_API virtual bool IsArray(const IContextCacheStore& Context) const override;
		DATAFLOWCORE_API virtual int32 GetNumArrayElements(const IContextCacheStore& Context) const override;
		DATAFLOWCORE_API virtual TUniquePtr<FContextCacheElementBase> CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;
		DATAFLOWCORE_API virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;
		DATAFLOWCORE_API virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;
		DATAFLOWCORE_API virtual TUniquePtr<FContextCacheElementBase> AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;

		// Get the matching const struct view if available, otherwise return an invalid one 
		DATAFLOWCORE_API virtual FConstStructView GetConstStructView(const IContextCacheStore& Context) const;

		DATAFLOWCORE_API virtual TUniquePtr<FContextCacheElementBase> Clone(const IContextCacheStore& Context) const override;

		//~ Begin FGCObject interface
		DATAFLOWCORE_API void AddReferencedObjects(FReferenceCollector& Collector) override;
		DATAFLOWCORE_API FString GetReferencerName() const override;
		//~ End FGCObject interface

	private:
		FInstancedStruct InstancedStruct;
	};

	//--------------------------------------------------------------------
	// UStruct array cache element
	//--------------------------------------------------------------------
	struct FContextCacheElementUStructArray : public FContextCacheElementBase, public FGCObject
	{
		explicit FContextCacheElementUStructArray(FGuid InNodeGuid, const FProperty* InProperty, const FConstStructArrayView& InStructArrayView, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementUStructArray, InNodeGuid, InProperty, InNodeHash, Timestamp)
			, InstancedStructArray(InStructArrayView)
		{}

		// construct from a single struct view into an array 
		explicit FContextCacheElementUStructArray(FGuid InNodeGuid, const FProperty* InProperty, const FConstStructView& InStructView, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementUStructArray, InNodeGuid, InProperty, InNodeHash, Timestamp)
			, InstancedStructArray(InStructView)
		{}
		explicit FContextCacheElementUStructArray(FGuid InNodeGuid, const FProperty* InProperty, const TArray<FConstStructView>& InStructViews, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementUStructArray, InNodeGuid, InProperty, InNodeHash, Timestamp)
			, InstancedStructArray(InStructViews)
		{}

		template<typename T>
		explicit FContextCacheElementUStructArray(FGuid InNodeGuid, const FProperty* InProperty, TArray<T>&& InStructArray, uint32 InNodeHash, FTimestamp Timestamp)
			: FContextCacheElementBase(EType::CacheElementUStructArray, InNodeGuid, InProperty, InNodeHash, Timestamp)
			, InstancedStructArray(Private::GetReflectedScriptStruct<T>())
		{
			InstancedStructArray.Get<T>() = Forward<TArray<T>>(InStructArray);
		}

		explicit FContextCacheElementUStructArray(const FContextCacheElementUStructArray& Other)
			: FContextCacheElementUStructArray(Other.GetNodeGuid(), Other.GetProperty(), Other.GetStructArrayView(), Other.GetNodeHash(), Other.GetTimestamp())
		{}

		virtual ~FContextCacheElementUStructArray() override = default;

		template<typename T>
		const TArray<T>& GetData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/, const TArray<T>& /*Default*/) const
		{
			return InstancedStructArray.Get<T>();
		}

		virtual const void* GetUntypedData(const IContextCacheStore& /*Context*/, const FProperty* /*PropertyIn*/) const override;

		virtual bool IsArray(const IContextCacheStore& Context) const override;
		virtual int32 GetNumArrayElements(const IContextCacheStore& Context) const override;
		virtual TUniquePtr<FContextCacheElementBase> CreateFromArrayElement(const IContextCacheStore& Context, int32 Index, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;
		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElement(const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;
		virtual TUniquePtr<FContextCacheElementBase> CreateArrayFromElementAndAppend(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;
		virtual TUniquePtr<FContextCacheElementBase> AppendArrayElements(const TArray<const FContextCacheElementBase*>& DereferencedElements, const IContextCacheStore& Context, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const override;

		virtual TUniquePtr<FContextCacheElementBase> Clone(const IContextCacheStore& Context) const override;

		//~ Begin FGCObject interface
		DATAFLOWCORE_API void AddReferencedObjects(FReferenceCollector& Collector) override;
		DATAFLOWCORE_API FString GetReferencerName() const override;
		//~ End FGCObject interface

	private:
		FConstStructArrayView GetStructArrayView() const;

		// Implements a FInstancedStruct for arrays (FInstancedStructContainer cannot be cast to a TArray)
		struct FInstancedStructArray final : private TArray<uint8>
		{
			explicit FInstancedStructArray(const UScriptStruct* const InScriptStruct);
			explicit FInstancedStructArray(const FConstStructView& StructView);
			explicit FInstancedStructArray(const TArray<FConstStructView>& StructViews);
			explicit FInstancedStructArray(const FConstStructArrayView& StructArrayView);
			~FInstancedStructArray();

			// expose parts of the TArray interface that are safe to use directly
			// Note: otherwise use Get()
			using TArray<uint8>::IsValidIndex;
			using TArray<uint8>::Num;
			using TArray<uint8>::GetData;

			const int32 GetStructureSize() const;

			const UScriptStruct* GetScriptStruct() const;

			FConstStructView GetScriptViewAt(int32 Index) const;

			const void* GetMemory() const;

			void AddReferencedObjects(FReferenceCollector& Collector);
			
			// Add elements from other struct views.
			void AddElements(const TArray<FConstStructView>& StructViews);

			// Append arrays from other FInstancedStructArrays
			void AppendElements(const TArray<const FInstancedStructArray*>& OtherArrays);

			template<typename T>
			const TArray<T>& Get() const
			{
				check(!ScriptStruct || Private::GetReflectedScriptStruct<T>() == ScriptStruct);
				return reinterpret_cast<const TArray<T>&>(*this);
			}

			template<typename T>
			TArray<T>& Get()
			{
				check(!ScriptStruct || Private::GetReflectedScriptStruct<T>() == ScriptStruct);
				return reinterpret_cast<TArray<T>&>(*this);
			}
		private:
			void InitFromRawData(const void* Data, const int32 Num);

			const UScriptStruct* const ScriptStruct;
		};

		FInstancedStructArray InstancedStructArray;
	};

	// cache element method implementation 
	template<class T>
	const T& FContextCacheElementBase::GetTypedData(const IContextCacheStore& Context, const FProperty* PropertyIn, const T& Default) const
	{
		// check(PropertyIn->IsA<T>()); // @todo(dataflow) compile error for non-class T; find alternatives
		if (Type == EType::CacheElementTyped)
		{
			if constexpr (!TIsUObjectPtrElement<T>::Value)
			{
				return static_cast<const TContextCacheElement<T>&>(*this).GetData(Context, PropertyIn, Default);
			}
		}
		if (Type == EType::CacheElementReference)
		{
			return static_cast<const FContextCacheElementReference&>(*this).GetData(Context, PropertyIn, Default);
		}
		if (Type == EType::CacheElementNull)
		{
			return Default; 
		}
		if (Type == EType::CacheElementUObjectArray)
		{
			if constexpr (TIsTArray<T>::Value)
			{
				if constexpr (TIsUObjectPtrElement<typename T::ElementType>::Value)
				{
					return static_cast<const TContextCacheElementUObjectArray<typename T::ElementType>&>(*this).GetData(Context, PropertyIn, Default);
				}
			}
		}
		if (Type == EType::CacheElementUObject)
		{
			if constexpr (TIsUObjectPtrElement<T>::Value)
			{
				return static_cast<const TContextCacheElementUObject<T>&>(*this).GetData(Context, PropertyIn, Default);
			}
		}
		if (Type == EType::CacheElementUStructArray)
		{
			if constexpr (TIsTArray<T>::Value)
			{
				if constexpr (TIsReflectedStruct<typename T::ElementType>::Value)
				{
					return static_cast<const FContextCacheElementUStructArray&>(*this).GetData<typename T::ElementType>(Context, PropertyIn, Default);
				}
			}
		}
		if (Type == EType::CacheElementUStruct)
		{
			if constexpr (TIsReflectedStruct<T>::Value)
			{
				return static_cast<const FContextCacheElementUStruct&>(*this).GetData<T>(Context, PropertyIn, Default);
			}
		}
		check(false); // should never happen
		return Default;
	}

	struct FContextCache : public TMap<FContextCacheKey, TUniquePtr<FContextCacheElementBase>>
	{
		DATAFLOWCORE_API void Serialize(FArchive& Ar);
	};

	// cache classes implemetation 
	// this needs to be after the FContext definition because they access its methods

	TUniquePtr<FContextCacheElementBase> FContextCacheElementBase::CreateReference(FContextCacheKey InReferenceDataKey) const
	{
		return MakeUnique<FContextCacheElementReference>(GetNodeGuid(), GetProperty(), InReferenceDataKey, GetNodeHash(), GetTimestamp());
	}

	template<class T>
	const T& TContextCacheElement<T>::GetData(const IContextCacheStore& Context, const FProperty* PropertyIn, const T& Default) const
	{
		return Data;
	}

	template<class T>
	TUniquePtr<FContextCacheElementBase> TContextCacheElement<T>::Clone(const IContextCacheStore& Context) const
	{
		return MakeUnique<TContextCacheElement<T>>(*this);
	}

	template<class T>
	const T& FContextCacheElementReference::GetData(const IContextCacheStore& Context, const FProperty* InProperty, const T& Default) const
	{
		if (const TUniquePtr<FContextCacheElementBase>* Cache = Context.FindCacheElement(DataKey))
		{
			return (*Cache)->GetTypedData<T>(Context, InProperty, Default);
		}
		return Default;
	}

	template<class T>
	TUniquePtr<FContextCacheElementBase> TContextCacheElementUObject<T>::Clone(const IContextCacheStore& Context) const
	{
		return MakeUnique<TContextCacheElementUObject<T>>(*this);
	}

	template<class T>
	TUniquePtr<FContextCacheElementBase> TContextCacheElementUObjectArray<T>::Clone(const IContextCacheStore& Context) const
	{
		return MakeUnique<TContextCacheElementUObjectArray<T>>(*this);
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////////

	struct FContextValue
	{
	public:
		DATAFLOWCORE_API FContextValue(IContextCacheStore& InContext, FContextCacheKey InCacheKey);
		DATAFLOWCORE_API FContextValue(IContextCacheStore& InContext, TUniquePtr<FContextCacheElementBase>&& InCacheElement);

		DATAFLOWCORE_API int32 Num() const;
		DATAFLOWCORE_API bool IsArray() const;
		DATAFLOWCORE_API FContextValue GetAt(int32 Index) const;
		DATAFLOWCORE_API FContextValue ToArray() const;
		// Warning: no checking will be done to verify that Elements can be appended to the array for strongly typed TContextCacheElement arrays and elements. It is assumed they can be cast to the correct type.
		DATAFLOWCORE_API static FContextValue MakeArray(IContextCacheStore& InContext, const TArray<const FContextValue>& Elements);
		// Warning: no checking will be done to verify that Elements can be appended to the array for strongly typed TContextCacheElement arrays and elements. It is assumed they can be cast to the correct type.
		DATAFLOWCORE_API static FContextValue AppendArrays(IContextCacheStore& InContext, const TArray<const FContextValue>& ArrayElements);

		DATAFLOWCORE_API FConstStructView GetConstStructView() const;
		DATAFLOWCORE_API FInstancedStruct GetInstancedStruct() const;

		template<typename T>
		const T* GetTypedStructPtr() const
		{
			return GetConstStructView().GetPtr<T>();
		}

		template<typename T>
		const TInstancedStruct<T> GetTypedInstancedStruct() const
		{
			TInstancedStruct<T> OutInstancedStruct;
			FConstStructView StructView = GetConstStructView();
			if (StructView.IsValid())
			{
				OutInstancedStruct.InitializeAsScriptStruct(StructView.GetScriptStruct(), StructView.GetMemory());
			}
			return OutInstancedStruct;
		}

		UE_DEPRECATED(5.8, "From release 5.10, this method is going to be private, direct access to the cache element is not authorized and the rest of the API of this class should be used instead to get data from the cache")
		DATAFLOWCORE_API const FContextCacheElementBase* GetCacheEntry() const;

	private:
		friend class FContext; // to access GetCacheEntry when it's private

		DATAFLOWCORE_API const FContextCacheElementBase* GetCacheEntryPrivate() const;

		IContextCacheStore& Context;
		FContextCacheKey CacheKey; // only set when CacheElement is null
		TSharedPtr<FContextCacheElementBase> CacheElement; // when CacheKey is define this should be null
	};
};

DATAFLOWCORE_API FArchive& operator<<(FArchive& Ar, UE::Dataflow::FTimestamp& ValueIn);
DATAFLOWCORE_API FArchive& operator<<(FArchive& Ar, UE::Dataflow::FContextCache& ValueIn);




