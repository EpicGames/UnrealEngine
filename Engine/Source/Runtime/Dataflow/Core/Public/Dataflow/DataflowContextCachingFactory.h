// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowNodeParameters.h"

#define UE_API DATAFLOWCORE_API

class FArchive;

namespace UE::Dataflow
{
	struct FContextCacheData {

		FContextCacheData(FName InType, FGuid InNodeGuid, FContextCacheElementBase* InData, uint32 InNodeHash, FTimestamp InTimestamp)
			: Type(InType), NodeGuid(InNodeGuid), Data(InData), NodeHash(InNodeHash), Timestamp(InTimestamp) {}

		FName Type;
		FGuid NodeGuid;
		FContextCacheElementBase* Data = nullptr;
		uint32 NodeHash;
		FTimestamp Timestamp = FTimestamp::Invalid;
	};

	//
	//
	//
	class FContextCachingFactory
	{
		typedef TFunction<FContextCacheElementBase*(FArchive& Ar, FContextCacheElementBase* InData)> FSerializeFunction;

		// All Maps indexed by TypeName
		TMap<FName, FSerializeFunction > CachingMap;		// [TypeName] -> Caching Funcitons
		static UE_API FContextCachingFactory* Instance;
		FContextCachingFactory() {}

	public:
		~FContextCachingFactory() { delete Instance; }

		static FContextCachingFactory* GetInstance()
		{
			if (!Instance)
			{
				Instance = new FContextCachingFactory();
			}
			return Instance;
		}

		UE_API void RegisterSerializeFunction(const FName& Type, FSerializeFunction InSerializeFunc);

		template<class T>
		static const T& GetTypedElement(const FContextCacheElementBase* InElement, const T& Default)
		{
			// we only support typed cache element and not cache reference
			if (InElement && InElement->GetType() == FContextCacheElementBase::EType::CacheElementTyped)
			{
				// it is assumed the type requested matches the cache entry 
				const TContextCacheElement<T>* TypedElement = static_cast<const TContextCacheElement<T>*>(InElement);
				return TypedElement->GetDataDirect();
			}
			return Default;
		}

		template<class T>
		static FContextCacheElementBase* NewTypedElement(T&& Data)
		{
			return new TContextCacheElement<T>(FGuid(), (FProperty*)nullptr, MoveTemp(Data), (uint32)0, FTimestamp::Invalid);
		}


		UE_API FContextCacheElementBase* Serialize(FArchive& Ar, FContextCacheData&& Data);

		bool Contains(FName InType) const { return CachingMap.Contains(InType); }

	};

}

#undef UE_API
