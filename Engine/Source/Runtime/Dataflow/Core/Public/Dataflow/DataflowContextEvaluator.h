// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/Ticker.h"
#include "Misc/Guid.h"
#include "Tasks/Task.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

struct FDataflowNode;
struct FDataflowOutput;

#define UE_API DATAFLOWCORE_API

namespace UE::Dataflow
{
	struct FCompiledGraph;
	class FGraph;

	class FContext;

	using FOnPostEvaluationFunction = TFunction<void(FContext&)>;
	using FOnTerminalNodeEvaluatedFunction = TFunction<void(FContext&, const FDataflowNode& Node)>;

	/*
	* Asynchronously evaluate a dataflow nodes
	* This may be slower than executing the graph in one go synchronously but this offers the following advantage:
	* - this can be cancelled at anytime ( only the in progress node will have finish evaluating while all the pending ones will be discarded )
	* - nodes that need to run on the game thread will do so ( see FDataflowNode EvaluateOnGameThreadOnly method )
	*/
	struct FContextEvaluator
	{
	public:
		using FNodeId = FGuid;

		struct FEvaluationEntry
		{
			FNodeId Id;
			TWeakPtr<const FDataflowNode> WeakNode;
			FOnPostEvaluationFunction OnPostEvaluation;

			bool operator == (const FEvaluationEntry& Other) const
			{
				return Id == Other.Id;
			}
			bool operator == (const FNodeId& Other) const
			{
				return (Id == Other);
			}

			FString ToString() const;
		};

		FContextEvaluator(FContext& InOwningContext)
			: OwningContext(InOwningContext)
		{
		}

		void ScheduleNodeEvaluation(const FDataflowNode& Node, FOnPostEvaluationFunction OnPostEvaluation);
		void ScheduleOutputEvaluation(const FDataflowOutput& Output, FOnPostEvaluationFunction OnPostEvaluation);

		void Process();
		void Cancel();
		void GetStats(int32& OutNumPendingTasks, int32& OutNumRunningTasks, int32& OutNumCompletedTasks) const;

		int32 GetNumPendingTasks() const { return PendingEvaluationEntries.Num(); }
		int32 GetNumRunningTasks() const { return RunningTasks.Num(); }
		int32 GetNumCompletedTasks() const { return CompletedTasks.Num(); }

	private:
		bool IsScheduledOrRunning(const FNodeId& Id) const;

		void FindInvalidUpstreamNodes(const FDataflowNode& Node, TArray<const FDataflowNode*>& OutInvalidUpstreamNodes);
		bool ShouldRunOnGameThread(const FDataflowNode& Node);
		void ScheduleEvaluation(const FEvaluationEntry& Entry);

		bool TryScheduleTask(const FEvaluationEntry& Entry);
		void ScheduleTask(const FEvaluationEntry& Entry);
		void ClearCompletedTasks();

		FContext& OwningContext;
		TMap<FNodeId, FEvaluationEntry> PendingEvaluationEntries;
		TMap<FNodeId, FGraphEventRef> RunningTasks;
		TSet<FNodeId> CompletedTasks;
	};

	struct FDataflowEvaluatorParameters
	{
		FDataflowEvaluatorParameters(TSharedRef<const FCompiledGraph> InCompiledGraph)
			: CompiledGraph(InCompiledGraph)
		{}

		/** Compiled graph to evaluate */
		TSharedRef<const FCompiledGraph> CompiledGraph;

		/** 
		* Optional Id of the node to evaluate in particular
		* Evaluator will try to its best to evaluate only the relevant nodes that leads to this one 
		* If not specified the entire graph will be evaluated
		*/
		FGuid NodeId;

		/** 
		* Optional function to call when evaluation is done 
		*/
		FOnPostEvaluationFunction OnPostEvaluation;

		/** 
		* Optional function to call each time after a "terminal" node has been evaluated
		* A terminal node is a type of node that has its IsTerminal() method to return true 
		* (see FDataflowTerminalNode, FDataflowSubGraphOutputNode, FDataflowExecutionNode)
		*/
		FOnTerminalNodeEvaluatedFunction OnTerminalNodeEvaluated;
	};

	struct IDataflowEvaluator
	{
		virtual ~IDataflowEvaluator() = default;
		virtual void Start(const FDataflowEvaluatorParameters& InParams) = 0;
		virtual void Cancel() = 0;

		virtual bool IsRunning() const = 0;
		virtual float GetProgress() const = 0;
		virtual FText GetProgressMessage() const = 0;
	};


	/**
	* Evaluator progress tracking
	* Easy drop-in structure to set and get progress in a thread safe way 
	*/
	struct FThreadSafeProgressTracker
	{
	public:
		float GetProgress() const;
		FText GetProgressMessage() const;

		void SetProgress(float InCurrent, float InTotal, FName InNodeName);
		void SetProgress(float InProgress, FName InNodeName);
		void AddProgress(float InProgressDelta, FName InNodeName);
	
	private:
		mutable FCriticalSection ProgressLock;
		FName NodeName;
		float Progress = 0.f;
	};

	/**
	* This evaluator will evaluate the graph in one go when Start is called
	* This is useful for running simulation graph for example that needs to execute in exactly one frame
	* or evaluating graph from a blueprint 
	*/
	struct FDataflowDirectEvaluator : public IDataflowEvaluator
	{
	public:
		UE_API FDataflowDirectEvaluator(FContext& InOwningContext);
		virtual ~FDataflowDirectEvaluator() = default;

		//~ Begin IDataflowEvaluator interface
		UE_API virtual void Start(const FDataflowEvaluatorParameters& StartParams) override;
		UE_API virtual void Cancel() override;
		UE_API virtual bool IsRunning() const override;
		UE_API virtual float GetProgress() const override;
		UE_API virtual FText GetProgressMessage() const override;
		//~ End IDataflowEvaluator interface

	private:
		FContext& OwningContext;

		std::atomic<bool> bIsCanceled = false;
		std::atomic<bool> bIsRunning = false;

		FThreadSafeProgressTracker ProgressTracker;
	};

	/**
	* This evaluator will try evaluate the graph on the game thread tick 
	* It will run as many nodes it can based on a time budget 
	* Note that a node may take more than the budget and create hitches
	*/
	struct FDataflowTickEvaluator: public IDataflowEvaluator
	{
		UE_API FDataflowTickEvaluator(FContext& InOwningContext);
		UE_API virtual ~FDataflowTickEvaluator();

		//~ Begin IDataflowEvaluator interface
		UE_API virtual void Start(const FDataflowEvaluatorParameters& InParams) override;
		UE_API virtual void Cancel() override;
		UE_API virtual bool IsRunning() const override;
		UE_API virtual float GetProgress() const override;
		UE_API virtual FText GetProgressMessage() const override;
		//~ End IDataflowEvaluator interface

	private:
		void Reset();
		bool Tick(float DeltaTime);
		void OnEvaluationCompleted();
		void OnEvaluationCanceled();

		FContext& OwningContext;
		TSharedPtr<const FCompiledGraph> CompiledGraph;
		TSharedPtr<const FGraph> DataflowGraph;
		FOnPostEvaluationFunction PostEvaluationFunction;
		FOnTerminalNodeEvaluatedFunction TerminalNodeEvaluatedFunction;
		FGuid NodeId;

		FTSTicker::FDelegateHandle TickDelegateHandle;

		/** Node indices (into CompiledGraph->Nodes) to evaluate this run, in topological order.
		* When p.Dataflow.UseDemandPullTick != 0 this is the demand-pull slice; otherwise it is
		* the full [0, NumNodes) sweep matching the legacy behaviour. */
		TArray<int32> NodeIndicesToEvaluate;

		/** Position within NodeIndicesToEvaluate of the next node to evaluate. */
		int32 NextNodeCursor = 0;

		std::atomic<bool> bIsCanceled = false;
		std::atomic<bool> bIsRunning = false;

		FThreadSafeProgressTracker ProgressTracker;
	};

	struct FTask;

	/**
	* This evaluator will evaluate the graph in one go when Start is called
	* This is useful for running simulation graph for example that needs to execute in exactly one frame
	* it is required to run from a worker thread and will assert if a node requires gamethread evaluation
	* Note that it is safe to cancel or query if its running or its progress from another thread
	*/
	struct FDataflowTaskGraphEvaluator : public IDataflowEvaluator
	{
	public:
		UE_API FDataflowTaskGraphEvaluator(FContext& InOwningContext);
		virtual ~FDataflowTaskGraphEvaluator() = default;

		//~ Begin IDataflowEvaluator interface
		UE_API virtual void Start(const FDataflowEvaluatorParameters& InParams) override;
		UE_API virtual void Cancel() override;
		UE_API virtual bool IsRunning() const override;
		UE_API virtual float GetProgress() const override;
		UE_API virtual FText GetProgressMessage() const override;
		//~ End IDataflowEvaluator interface

	private:
		FContext& OwningContext;

		TSharedPtr<FTask> CompletionTask;
		TSharedRef<UE::Tasks::FCancellationToken> CancellationToken;

		TSharedRef<FThreadSafeProgressTracker> ProgressTracker;
	};
}

#undef UE_API
