// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosLog.h"
#include "CoreMinimal.h"
#include "Dataflow/DataflowConnection.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNode.h"

#include "DataflowCoreNodes.generated.h"

struct FDataflowOutput;

USTRUCT()
struct FDataflowReRouteNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowReRouteNode, "ReRouteNode", "Core", "")

public:
	FDataflowReRouteNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;


public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Value", DisplayName = "Value"))
	FDataflowAnyType Value;
};

USTRUCT(meta=(Icon="GraphEditor.Branch_16x"))
struct FDataflowBranchNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowBranchNode, "Branch", "FlowControl", "")

public:
	FDataflowBranchNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;

public:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DisplayName = "TrueValue"))
	FDataflowAnyType TrueValue;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DisplayName = "FalseValue"))
	FDataflowAnyType FalseValue;

	UPROPERTY(EditAnywhere, Category="Condition", meta = (DataflowInput, DisplayName = "Condition"))
	bool bCondition = true;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Result"))
	FDataflowAnyType Result;
};


USTRUCT(meta = (Icon = "GraphEditor.Branch_16x"))
struct FDataflowSelectNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSelectNode, "Select", "FlowControl", "")

public:
	FDataflowSelectNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;

public:
	UPROPERTY()
	TArray<FDataflowAnyType> Inputs;

	UPROPERTY(EditAnywhere, Category = "Condition", meta = (DataflowInput))
	int32 SelectedIndex = 0;

	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "Inputs[0]"))
	FDataflowAnyType Result;

private:
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return true; }
	virtual bool CanRemovePin() const override { return Inputs.Num() > NumInitialInputs; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;

	UE::Dataflow::TConnectionReference<FDataflowAnyType> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredDataflowInputs = 1;
	static constexpr int32 NumInitialInputs = 2;
	static const FName MainTypeGroup;
};

/** 
* Print value in the log
* Supports any type comnvertible to a string 
*/
USTRUCT()
struct FDataflowPrintNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowPrintNode, "Print", "Core", "")

public:
	FDataflowPrintNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Value", DisplayName = "Value"))
	FDataflowStringConvertibleTypes Value;
};

/**
* Force an evaluation dependency between two values
*/
USTRUCT()
struct FDataflowForceDependencyNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowForceDependencyNode, "ForceDependency", "FlowControl", "")

public:
	FDataflowForceDependencyNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;


public:

	/* Evaluating Value will force an evaluation of DependentValue*/
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Value"))
	FDataflowAnyType Value;

	/* Evaluating Value will force an evaluation of DependentValue*/
	UPROPERTY(meta = (DataflowInput))
	FDataflowAnyType DependentValue;
};

namespace UE::Dataflow
{
	void RegisterCoreNodes();
}
