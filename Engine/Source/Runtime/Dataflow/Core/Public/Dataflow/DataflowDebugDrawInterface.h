// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Math/MathFwd.h"
#include "Containers/Array.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Chaos/ChaosDebugDrawInterface.h"

struct FDataflowBaseElement;

/** Dataflow object debug draw interface */
struct IDataflowDebugDrawObject : public FRefCountedObject
{
	virtual ~IDataflowDebugDrawObject() = default;

	static FName StaticType() { return FName("IDataflowDebugDrawObject"); }

	/** Check of the object type */
	virtual bool IsA(FName InType) const  {return false;}
};

class IDataflowDebugDrawInterface: public Chaos::IDebugDrawInterface
{
public:
	using FDataflowElementsType = TArray<TSharedPtr<FDataflowBaseElement>>;

	virtual ~IDataflowDebugDrawInterface() = default;

	/** Draw a Dataflow debug draw object ( for example skeleton bones )*/
	virtual void DrawObject(const TRefCountPtr<IDataflowDebugDrawObject>& Object) = 0;

	/** Render a string a part of the Dataflow construction view overlay */
	virtual void DrawOverlayText(const FString& InString) = 0;

	/** Get the text overlay string ( concatenated string from DrawOverlayText calls )*/
	virtual FString GetOverlayText() const = 0;

	/** Dataflow elements non const accessor */
	virtual FDataflowElementsType& ModifyDataflowElements() = 0;

	/** Dataflow elements const accessor */
	virtual const FDataflowElementsType& GetDataflowElements() const = 0;
};


