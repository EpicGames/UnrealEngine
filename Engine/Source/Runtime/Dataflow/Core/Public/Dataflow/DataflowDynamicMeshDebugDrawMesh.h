// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowDebugDrawInterface.h"

struct FDynamicMeshDebugDrawMesh : public IDataflowDebugDrawInterface::IDebugDrawMesh
{
	DATAFLOWCORE_API FDynamicMeshDebugDrawMesh(const class UE::Geometry::FDynamicMesh3* DynamicMesh);

	virtual ~FDynamicMeshDebugDrawMesh() = default;

	DATAFLOWCORE_API virtual int32 GetMaxVertexIndex() const override;
	DATAFLOWCORE_API virtual bool IsValidVertex(int32 VertexIndex) const override;
	DATAFLOWCORE_API virtual FVector GetVertexPosition(int32 VertexIndex) const override;
	DATAFLOWCORE_API virtual FVector GetVertexNormal(int32 VertexIndex) const override;

	DATAFLOWCORE_API virtual int32 GetMaxTriangleIndex() const override;
	DATAFLOWCORE_API virtual bool IsValidTriangle(int32 VertexIndex) const override;
	DATAFLOWCORE_API virtual FIntVector3 GetTriangle(int32 VertexIndex) const override;

private:
	const class UE::Geometry::FDynamicMesh3* DynamicMesh;
};
