// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataflowElement.generated.h"

/** Base dataflow scene element that could be used in outliner and for rendering */
USTRUCT()
struct FDataflowBaseElement
{
	GENERATED_BODY()

	FDataflowBaseElement() {}

	FDataflowBaseElement(const FString& InElementName, FDataflowBaseElement* InParentElement, const FBox& InBoundingBox, const bool bInIsConstruction) :
		ElementName(InElementName), ParentElement(InParentElement), BoundingBox(InBoundingBox), bIsConstruction(bInIsConstruction) {}

	virtual ~FDataflowBaseElement() = default;

	static FName StaticType() { return FName("FDataflowBaseElement"); }

	/** Check of the element type */
	virtual bool IsA(FName InType) const { return InType.ToString().Equals(StaticType().ToString());}
	
	/** Element name to be used to retrieve */
	FString ElementName = "None";

	/** Parent element to build the hierarchy if necessary */
	FDataflowBaseElement* ParentElement = nullptr;

	/** Bounding box to focus */
	FBox BoundingBox = FBox();
	
	/** Construction flag */
	bool bIsConstruction = true;
	
	/** Visible flag to enable/disable the rendering */
	bool bIsVisible = true;

	/** Selection flag */
	bool bIsSelected = false;
};
