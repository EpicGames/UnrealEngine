// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointerFwd.h"
#include "DataflowFunctionProperty.generated.h"

namespace UE::Dataflow
{
	class FContext;
}

/**
 * Function property for all Dataflow nodes.
 * The structure is also used in DataFlow::FFunctionPropertyCustomization to appear as text and/or image buttons.
 * This helps with the equivalent UCLASS UFUNCTION CallInEditor functionality that is missing from the USTRUCT implementation.
 *
 * By default the text of the button is the name of the structure property.
 * The tooltip is the property source documentation.
 * Further (but optional) customizations can be achieved by using the following Meta tags where declaring the property:
 *   DisplayName
 *   ButtonImage
 *
 * Specifying an empty DisplayName string will only display the icon and no text.
 *
 * For example:
 *   UPROPERTY(EditAnywhere, Category = "Functions")
 *   FDataflowFunctionProperty ReimportAssetTextOnly;
 *
 *   UPROPERTY(EditAnywhere, Category = "Functions", Meta = (ButtonImage = "Persona.ReimportAsset"))
 *   FDataflowFunctionProperty ReimportAssetTextAndIcon;
 * 
 *   UPROPERTY(EditAnywhere, Category = "Functions", Meta = (DisplayName = "", ButtonImage = "Persona.ReimportAsset"))
 *   FDataflowFunctionProperty ReimportAssetIconOnly;
 *
 *   UPROPERTY(EditAnywhere, Category = "Functions", Meta = (DisplayName = "Reimport Asset"))
 *   FDataflowFunctionProperty ReimportAssetOverriddenText;
 *
 *   UPROPERTY(EditAnywhere, Category = "Functions", Meta = (DisplayName = "Reimport Asset", ButtonImage = "Persona.ReimportAsset"))
 *   FDataflowFunctionProperty ReimportAssetOverriddenTextAndIcon;
 *
 */
USTRUCT()
struct FDataflowFunctionProperty
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE_OneParam(FDelegate, UE::Dataflow::FContext&);

	FDataflowFunctionProperty() = default;

	explicit FDataflowFunctionProperty(FDelegate&& InDelegate) : Delegate(MoveTemp(InDelegate)) {}

	void Execute(UE::Dataflow::FContext& Context) const { Delegate.ExecuteIfBound(Context); }

private:
	FDelegate Delegate;
};
