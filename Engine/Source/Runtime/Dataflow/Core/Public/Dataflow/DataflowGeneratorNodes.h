// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowAnyType.h"

#include "DataflowGeneratorNodes.generated.h"

struct IDataflowValueGenerator
{
	virtual void GenerateValues(TArray<double>& InOutValues) const = 0;
};

USTRUCT()
struct FDataflowValueGenerator
{
	GENERATED_USTRUCT_BODY()

public:
	FDataflowValueGenerator(TSharedPtr<IDataflowValueGenerator> InGenerator = {}) :
		Generator(InGenerator)
	{}

	DATAFLOWCORE_API void GenerateValues(TArray<double>& InOutValues) const;

private:
	TSharedPtr<IDataflowValueGenerator> Generator;
};

// --------------------------------------------------------------------------------------------------------------------

/**
* Generator to generate a random sequence
*/
USTRUCT()
struct FDataflowRandomGeneratorNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowRandomGeneratorNode, "RandomGenerator", "Core|Generators", "seed")

public:
	FDataflowRandomGeneratorNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;

public:
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	int32 Seed = 0;

	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	FDataflowNumericTypes RandomMin;

	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	FDataflowNumericTypes RandomMax;

	UPROPERTY(EditAnywhere, Category = "Random")
	bool bReverseOrder = false;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowValueGenerator Generator;
};

// --------------------------------------------------------------------------------------------------------------------

/**
* Generator to generate a linear sequence
*/
USTRUCT()
struct FDataflowLinearSequenceGeneratorNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowLinearSequenceGeneratorNode, "LinearSequenceGenerator", "Core|Generators", "")

public:
	FDataflowLinearSequenceGeneratorNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;

public:
	UPROPERTY(EditAnywhere, Category = "Sequence", meta = (DataflowInput))
	FDataflowNumericTypes Start;

	UPROPERTY(EditAnywhere, Category = "Sequence", meta = (DataflowInput))
	FDataflowNumericTypes Increment;

	UPROPERTY(EditAnywhere, Category = "Sequence")
	bool bReverseOrder = false;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowValueGenerator Generator;
};

// --------------------------------------------------------------------------------------------------------------------

/**
* Generator to generate a sequence from Perlin noise
*/
USTRUCT()
struct FDataflowPerlinNoiseGeneratorNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowPerlinNoiseGeneratorNode, "PerlinNoiseGenerator", "Core|Generators", "")

public:
	FDataflowPerlinNoiseGeneratorNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;

public:
	UPROPERTY(EditAnywhere, Category = "Perlin", meta = (DataflowInput, UIMin = 0.01, ClampMin = 0.01))
	float Frequency = 1.0;

	UPROPERTY(EditAnywhere, Category = "Perlin", meta = (DataflowInput))
	float Offset = 0.0;

	UPROPERTY(EditAnywhere, Category = "Perlin", meta = (DataflowInput))
	float NoiseMin = -1.0;

	UPROPERTY(EditAnywhere, Category = "Perlin", meta = (DataflowInput))
	float NoiseMax = 1.0;

	UPROPERTY(EditAnywhere, Category = "Perlin")
	bool bClamp = false;

	UPROPERTY(EditAnywhere, Category = "Perlin")
	bool bReverseOrder = false;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowValueGenerator Generator;
};

// --------------------------------------------------------------------------------------------------------------------

/**
* Generator to generate a sequence from a simple regex
* Example: "0, 2, 5-10, 12-15"
*/
USTRUCT()
struct FDataflowSimpleRegexGeneratorNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSimpleRegexGeneratorNode, "Simple Regex Generator", "Core|Generators", "")

public:
	FDataflowSimpleRegexGeneratorNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;

public:
	UPROPERTY(EditAnywhere, Category = "Regex")
	FString List;

	UPROPERTY(EditAnywhere, Category = "Regex")
	bool bReverseOrder = false;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowValueGenerator Generator;
};

// --------------------------------------------------------------------------------------------------------------------

namespace UE::Dataflow
{
	void RegisterGeneratorNodes();
}

