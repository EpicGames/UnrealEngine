// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "Chaos/ChaosArchive.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowTerminalNode.h"
#include "Serialization/Archive.h"

#include "DataflowGraph.generated.h"

#define UE_API DATAFLOWCORE_API

struct FDataflowConnection;

namespace UE::Dataflow
{
	class FGraph;
}

/** Interface for objects that owns a dataflow graph */
UINTERFACE(MinimalAPI)
class UDataflowGraphInterface : public UInterface
{
	GENERATED_BODY()
};

class IDataflowGraphInterface
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<UE::Dataflow::FGraph> GetDataflowGraph() const = 0;
};

namespace UE::Dataflow
{
	/**
	* A link structure defines a connection between one output and one input of two separate Dataflow nodes
	* The Guids are the unique identifiers of the node and inputs/outputs in the dataflow graph (see FGraph class below)
	*/
	struct FLink
	{
		FGuid InputNode;
		FGuid Input;
		FGuid OutputNode;
		FGuid Output;

		FLink() = default;

		FLink(FGuid InOutputNode, FGuid InOutput, FGuid InInputNode, FGuid InInput)
			: InputNode(InInputNode), Input(InInput)
			, OutputNode(InOutputNode), Output(InOutput) {}

		FLink(const FLink& Other)
			: InputNode(Other.InputNode), Input(Other.Input)
			, OutputNode(Other.OutputNode), Output(Other.Output) {}

		bool operator==(const FLink& Other) const
		{
			return Equals(Other);
		}

		bool Equals(const FLink& Other) const
		{
			return Input == Other.Input && InputNode == Other.InputNode
				&& Output == Other.Output && OutputNode == Other.OutputNode;
		}
	};


	/**
	* A Dataflow graph is a collection of interconnected nodes
	* Each node has aunique Guid and name and a list of inputs and outputs ( see FDataflowNode )
	*/
	class FGraph
	{
	public:
		using FNodeSharedPtr = TSharedPtr<FDataflowNode>;
		using FConstNodeSharedPtr = TSharedPtr<const FDataflowNode>;

	private:
		FGuid  Guid;
		TArray<FNodeSharedPtr> Nodes;
		TMap<FName, TArray<FNodeSharedPtr>> FilteredNodes;
		TArray<FLink> Connections;
		TSet<FName> DisabledNodes;

		/** 
		* This guid changes whenever the topology of the graph changes
		* (Add/Remove nodes, Connect/Disconnect input and/or outputs)
		*/
		FGuid GraphTopologyGuid;
		
		/** Node filter type that could be used for fast access*/
		static TSet<FName> RegisteredFilters;

		/** Friend register function */
		friend DATAFLOWCORE_API void RegisterNodeFilter(const FName& NodeFilter);
		
	public:
		UE_API FGraph(FGuid InGuid = FGuid::NewGuid());
		virtual ~FGraph() = default;
		
		/** Get the list of node matching a specific named filter (@see RegisterNodeFilter) */
		UE_API const TArray<FNodeSharedPtr>& GetFilteredNodes(const FName& NodeFilter) const;

		/** 
		* Get all the nodes of the graph ( const method )
		* this includes the node in all subgraphs
		*/
		const TArray<FNodeSharedPtr>& GetNodes() const { return Nodes; }

		/**
		* Get all the nodes of the graph ( non-const method )
		* this includes the node in all subgraphs
		*/
		TArray<FNodeSharedPtr>& GetNodes() { return Nodes; }

		/** Get the number of nodes in the graph including the ones in the subgraphs */
		int32 NumNodes() { return Nodes.Num(); }

		template<class T> 
		UE_DEPRECATED(5.8, "Use the Shared pointer parameter version of AddNode instead")
		TSharedPtr<T> AddNode(T* InNode)
		{
			TSharedPtr<T> NewNode(InNode);
			AddNode(NewNode);
			return NewNode;
		}

		template<class T>
		UE_DEPRECATED(5.8, "Use the Shared pointer parameter version of AddNode instead")
		TSharedPtr<T> AddNode(TUniquePtr<T> &&InNode)
		{
			TSharedPtr<T> NewNode(InNode.Release());
			AddNode(NewNode);
			return NewNode;
		}

		/** Add a new Dataflow node to the graph */
		UE_API FNodeSharedPtr AddNode(FNodeSharedPtr NewNode);

		/** Find a node by its unique identifier (non-const method) */
		UE_API FNodeSharedPtr FindBaseNode(FGuid InGuid);

		/** Find a node by its unique identifier (const method) */
		UE_API FConstNodeSharedPtr FindBaseNode(FGuid InGuid) const;

		/** Find a node by its name (non-const method) */
		UE_API FNodeSharedPtr FindBaseNode(FName InName);

		/** Find a node by its name (const method) */
		UE_API FConstNodeSharedPtr FindBaseNode(FName InName) const;

		/** Find a node by name from a named filter (@see RegisterNodeFilter method) */
		UE_API FNodeSharedPtr FindFilteredNode(const FName& NodeFilter, FName InName) const;

		/** 
		* Remove a specific node from the graph
		* Removal of the node will automatically disconnect the inputs and outputs 
		* this will also remove the node from any filters it may be in 
		*/
		UE_API void RemoveNode(FNodeSharedPtr Node);

		/**
		* Get all the graph connections
		* This includes all the subgraph connections as well
		*/
		const TArray<FLink>& GetConnections() const { return Connections; }

		UE_API void ClearConnections(FDataflowConnection* ConnectionBase);
		UE_API void ClearConnections(FDataflowInput* Input);
		UE_API void ClearConnections(FDataflowOutput* Output);

		enum class EConnectType : uint8
		{
			REJECTED = 0,
			DIRECT, // both are already compatible
			INPUT_PROMOTION, // input can be changed to adapt the output type
			OUTPUT_PROMOTION, // output can be changed to adapt the input type
		};

		UE_API bool CanConnect(const FDataflowOutput& Output, const FDataflowInput& Input) const;
		UE_API EConnectType GetConnectType(const FDataflowOutput& Output, const FDataflowInput& Input) const;
		UE_API bool Connect(FDataflowOutput& Output, FDataflowInput& Input);

		UE_API bool Connect(FDataflowConnection* ConnectionA, FDataflowConnection* ConnectionB);
		UE_API void Connect(FDataflowOutput* OutputConnection, FDataflowInput* InputConnection);
		UE_API void Disconnect(FDataflowOutput* OutputConnection, FDataflowInput* InputConnection);

		UE_API void AddReferencedObjects(FReferenceCollector& Collector);

		UE_API virtual void Serialize(FArchive& Ar, UObject* OwningObject);
		const TSet<FName>& GetDisabledNodes() const { return DisabledNodes; }

		UE_API static void SerializeForSaving(FArchive& Ar, FGraph* InGraph, TArray<TSharedPtr<FDataflowNode>>& InNodes, TArray<FLink>& InConnections);
		UE_API static void SerializeForLoading(FArchive& Ar, FGraph* InGraph, UObject* OwningObject);

		const FGuid& GetGraphTopologyGuid() const { return GraphTopologyGuid; }

	private:
		void Reset();
		void OnChangeGraphTopology();
	};

	UE_API void RegisterNodeFilter(const FName& NodeFilter);
}


inline FArchive& operator<<(FArchive& Ar, UE::Dataflow::FLink& Value)
{
	Ar << Value.InputNode << Value.OutputNode << Value.Input << Value.Output;
	return Ar;
}

inline FArchive& operator<<(Chaos::FChaosArchive& Ar, UE::Dataflow::FLink& Value)
{
	Ar << Value.InputNode << Value.OutputNode << Value.Input << Value.Output;
	return Ar;
}

#undef UE_API



