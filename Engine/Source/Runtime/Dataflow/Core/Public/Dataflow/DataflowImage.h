// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageCore.h"

#include "DataflowImage.generated.h"

UENUM()
enum class EDataflowImageResolution : int32
{
	Resolution16 = 16 UMETA(DisplayName = "16 x 16"),
	Resolution32 = 32 UMETA(DisplayName = "32 x 32"),
	Resolution64 = 64 UMETA(DisplayName = "64 x 64"),
	Resolution128 = 128 UMETA(DisplayName = "128 x 128"),
	Resolution256 = 256 UMETA(DisplayName = "256 x 256"),
	Resolution512 = 512 UMETA(DisplayName = "512 x 512"),
	Resolution1024 = 1024 UMETA(DisplayName = "1024 x 1024"),
	Resolution2048 = 2048 UMETA(DisplayName = "2048 x 2048"),
	Resolution4096 = 4096 UMETA(DisplayName = "4096 x 4096"),
	Resolution8192 = 8192 UMETA(DisplayName = "8192 x 8192")
};

UENUM()
enum class EDataflowImageChannel : int32
{
	Red = 0 UMETA(DisplayName = "Red Channel"),
	Green = 1 UMETA(DisplayName = "Green Channel"),
	Blue = 2 UMETA(DisplayName = "Blue Channel"),
	Alpha = 3 UMETA(DisplayName = "Alpha Channel"),
};

/** 
* Represents image for dataflow
* type is constrained to Float32 with 1 or 4 channels
*/
USTRUCT()
struct FDataflowImage
{
	GENERATED_USTRUCT_BODY()

	DATAFLOWCORE_API int32 GetWidth() const;
	DATAFLOWCORE_API int32 GetHeight() const;

	/** get the readonly underlying image object */
	DATAFLOWCORE_API const FImage& GetImage() const;

	/** Create a single channel float format image */
	DATAFLOWCORE_API void CreateR32F(EDataflowImageResolution Resolution, TArrayView64<float> InPixels = {});

	/** Create a single channel float format image */
	DATAFLOWCORE_API void CreateR32F(int32 Width, int32 Height, TArrayView64<float> InPixels = {});

	/** Create a four channels float format image */
	DATAFLOWCORE_API void CreateRGBA32F(EDataflowImageResolution Resolution, TArrayView64<FVector4f> InPixels = {});

	/** Create a four channels float format image */
	DATAFLOWCORE_API void CreateRGBA32F(int32 Width, int32 Height, TArrayView64<FVector4f> InPixels = {});

	/** Create a four channels float format image filled with a specific color */
	DATAFLOWCORE_API void CreateFromColor(EDataflowImageResolution Resolution, FLinearColor Color);

	/** Create a four channels float format image filled with a specific color */
	DATAFLOWCORE_API void CreateFromColor(int32 Width, int32 Height, FLinearColor Color);

	/**
	* Copy R32F pixels to the image
	* Number of pixels must match and format must be R32F
	* return false if the copy could not be done
	*/
	DATAFLOWCORE_API bool CopyPixels(TArrayView64<float> Pixels);

	/**
	* Copy RGBA32F pixels to the image
	* Number of pixels must match and format must be RGBA32F
	* return false if the copy could not be done
	*/
	DATAFLOWCORE_API bool CopyPixels(TArrayView64<FVector4f> Pixels);
	DATAFLOWCORE_API bool CopyRGBAPixels(TArrayView64<FVector4f> Pixels);

	/** 
	* Convert the current image to a 4 channel float pixel format 
	* previous data is kept 
	*/
	DATAFLOWCORE_API void ConvertToRGBA32F();

	DATAFLOWCORE_API bool Serialize(FArchive& Ar);

	/**
	* Get a specific color channel and copy it to an image
	* Warning : the outImage will be resized to the size of the current image and any previously store data will be lost 
	*/
	DATAFLOWCORE_API void ReadChannel(EDataflowImageChannel Channel, FDataflowImage& OutImage) const;

	/**
	* write to a sepcific channel from an existing image
	* if the source image is not the same size it will be resized to adapt the size of the current image
	* if the source image is not a greyscale image it will be converted to greyscale before copy the data to the channel
	*/
	DATAFLOWCORE_API void WriteChannel(EDataflowImageChannel Channel, const FDataflowImage& SrcImage);

private:
	FImage Image;
};

template<> struct TStructOpsTypeTraits<FDataflowImage> : public TStructOpsTypeTraitsBase2<FDataflowImage>
{
	enum
	{
		WithSerializer = true,
	};
};