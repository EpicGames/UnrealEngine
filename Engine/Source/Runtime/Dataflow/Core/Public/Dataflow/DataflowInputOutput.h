// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowConnection.h"
#include "Dataflow/DataflowAnyType.h"
#include "Templates/Function.h"
#include "Async/Async.h"
#include "HAL/CriticalSection.h"

#include "DataflowInputOutput.generated.h"


struct FDataflowOutput;

//
//  Input
//
namespace UE::Dataflow
{
	struct FInputParameters : public FConnectionParameters
	{
		FInputParameters(FName InType = NAME_None, FName InName = NAME_None, FDataflowNode* InOwner = nullptr, const FProperty* InProperty = nullptr, uint32 InOffset = INDEX_NONE, FGuid InGuid = FGuid::NewGuid())
			: FConnectionParameters(InType, InName, InOwner, InProperty, InOffset, InGuid)
		{}
	};

	struct FArrayInputParameters : public FInputParameters
	{
		FArrayInputParameters()
			: FInputParameters()
		{}
		const FArrayProperty* ArrayProperty = nullptr;
		uint32 InnerOffset = INDEX_NONE;
	};
}

USTRUCT()
struct FDataflowInput : public FDataflowConnection
{
	GENERATED_USTRUCT_BODY()

	static FDataflowInput NoOpInput;

	friend struct FDataflowConnection;

	FDataflowOutput* Connection = nullptr ;

protected:
	friend struct FDataflowOutput;
	virtual void FixAndPropagateType(FName InType) override;

private:
	// if the Input has DATAFLOW_INTRINSIC metatag specified
	bool bIsRequired = false;
	DATAFLOWCORE_API void CheckRequired(UE::Dataflow::FContext& Context) const;

public:
	DATAFLOWCORE_API FDataflowInput(const UE::Dataflow::FInputParameters& Param = {});

	virtual bool IsConnected() const override;
	virtual bool AddConnection(FDataflowConnection* InOutput) override;
	virtual bool RemoveConnection(FDataflowConnection* InOutput) override;

	virtual void GetConnections(TArray<FDataflowConnection*>& OutConnections) const override;

	FDataflowOutput* GetConnection() { return Connection; }
	const FDataflowOutput* GetConnection() const { return Connection; }
	bool HasAnyConnections() const { return Connection != nullptr; }

	virtual TArray< FDataflowOutput* > GetConnectedOutputs();
	virtual const TArray< const FDataflowOutput* > GetConnectedOutputs() const;

	/** 
	* Get the value of this input by evaluating the value of the connected output 
	* @return the typed value of the input 
	*/
	template<class T>
	const T& GetValue(UE::Dataflow::FContext& Context, const T& Default) const;

	template<typename TAnyType UE_REQUIRES(!std::is_same_v<typename TAnyType::FStorageType, void>)>
	typename TAnyType::FStorageType GetValueFromAnyType(UE::Dataflow::FContext& Context, const typename TAnyType::FStorageType& Default) const;

	template<typename TAnyType UE_REQUIRES(std::is_same_v<typename TAnyType::FStorageType, void>)>
	typename UE::Dataflow::FContextValue GetValueFromAnyType(UE::Dataflow::FContext& Context) const;

	/**
	* pull the value from the upstream connections
	* the upstream graph is evaluated if necessary and values are cached along the way 
	*/
	DATAFLOWCORE_API void PullValue(UE::Dataflow::FContext& Context) const;

	template<class T>
	UE_DEPRECATED(5.8, "This code path is not supported, use context to evaluate nodes and connections asynchronously")
	TFuture<const T&> GetValueParallel(UE::Dataflow::FContext& Context, const T& Default) const;

	virtual void Invalidate(const UE::Dataflow::FTimestamp& ModifiedTimestamp = UE::Dataflow::FTimestamp::Current()) override;

	bool IsRequired() const { return bIsRequired; }
	void SetIsRequired(const bool bInIsRequired) { bIsRequired = bInIsRequired; }
};

USTRUCT()
struct FDataflowArrayInput : public FDataflowInput
{
	GENERATED_USTRUCT_BODY()

private:
	int32 Index;
	uint32 ElementOffset; // Offset to Property inside an array element
	const FArrayProperty* ArrayProperty = nullptr;
	// uint32 Offset; // On base class. This is the Offset to ArrayProperty from OwningNode.

public:
	explicit FDataflowArrayInput(int32 InIndex = INDEX_NONE, const UE::Dataflow::FArrayInputParameters& Param = {});

	DATAFLOWCORE_API virtual void* RealAddress() const override;
	virtual int32 GetContainerIndex() const override { return Index; }
	virtual uint32 GetContainerElementOffset() const override { return ElementOffset; }
};

//
// Output
//
namespace UE::Dataflow
{
	struct FOutputParameters: public FConnectionParameters
	{
		FOutputParameters(FName InType = NAME_None, FName InName = NAME_None, FDataflowNode* InOwner = nullptr, const FProperty* InProperty = nullptr, uint32 InOffset = INDEX_NONE, FGuid InGuid = FGuid::NewGuid())
			: FConnectionParameters(InType, InName, InOwner, InProperty, InOffset, InGuid)
		{}
	};

	struct FArrayOutputParameters : public FOutputParameters
	{
		FArrayOutputParameters()
			: FOutputParameters()
		{}
		const FArrayProperty* ArrayProperty = nullptr;
		uint32 InnerOffset = INDEX_NONE;
	};
}

USTRUCT()
struct FDataflowOutput : public FDataflowConnection
{
	GENERATED_USTRUCT_BODY()

	friend struct FDataflowConnection;
	
	TArray< FDataflowInput* > Connections;

	UE::Dataflow::FConnectionKey PassthroughKey;

protected:
	friend struct FDataflowInput;
	virtual void FixAndPropagateType(FName InType) override;

public:
	static DATAFLOWCORE_API FDataflowOutput NoOpOutput;
	
	mutable TSharedPtr<FCriticalSection> OutputLock;
	
	DATAFLOWCORE_API FDataflowOutput(const UE::Dataflow::FOutputParameters& Param = {});

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FDataflowOutput() = default;
	FDataflowOutput(const FDataflowOutput&) = delete; 
	FDataflowOutput(FDataflowOutput&&) = delete;
	FDataflowOutput& operator=(const FDataflowOutput&) = delete;
	FDataflowOutput& operator=(FDataflowOutput&&) = delete;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	DATAFLOWCORE_API TArray<FDataflowInput*>& GetConnections();
	DATAFLOWCORE_API const TArray<FDataflowInput*>& GetConnections() const;
	bool HasAnyConnections() const { return !Connections.IsEmpty(); }
	virtual bool IsConnected() const override { return HasAnyConnections(); }

	DATAFLOWCORE_API virtual TArray<FDataflowInput*> GetConnectedInputs();
	DATAFLOWCORE_API virtual const TArray<const FDataflowInput*> GetConnectedInputs() const;

	DATAFLOWCORE_API virtual bool AddConnection(FDataflowConnection* InOutput) override;

	DATAFLOWCORE_API virtual bool RemoveConnection(FDataflowConnection* InInput) override;

	virtual void GetConnections(TArray<FDataflowConnection*>& OutConnections) const override;

	DATAFLOWCORE_API FDataflowOutput& SetPassthroughInput(const UE::Dataflow::FConnectionReference& Reference);
	DATAFLOWCORE_API FDataflowOutput& SetPassthroughInput(const UE::Dataflow::FConnectionKey& Key);

	DATAFLOWCORE_API const FDataflowInput* GetPassthroughInput() const;

	virtual void* GetPassthroughRealAddress() const
	{
		if(const FDataflowInput* const PassthroughInput = GetPassthroughInput())
		{
			return PassthroughInput->RealAddress();
		}
		return nullptr;
	}
 
	template<class T>
	void SetValue(T&& InVal, UE::Dataflow::FContext& Context) const
	{
		if (Property)
		{
			Context.SetData(CacheKey(), Property, Forward<T>(InVal), GetOwningNodeGuid(), GetOwningNodeValueHash(), GetOwningNodeTimestamp());
		}
	}

	void SetValueFromStructView(const FConstStructView& StructView, UE::Dataflow::FContext& Context) const
	{
		if (Property)
		{
			Context.SetDataFromStructView(CacheKey(), Property, StructView, GetOwningNodeGuid(), GetOwningNodeValueHash(), GetOwningNodeTimestamp());
		}
	}

	void SetValueFromStructArrayView(const FConstStructArrayView& StructArrayView, UE::Dataflow::FContext& Context) const
	{
		if (Property)
		{
			Context.SetDataFromStructArrayView(CacheKey(), Property, StructArrayView, GetOwningNodeGuid(), GetOwningNodeValueHash(), GetOwningNodeTimestamp());
		}
	}

	template<typename TAnyType>
	void SetValueFromAnyType(const typename TAnyType::FStorageType& InVal, UE::Dataflow::FContext& Context) const
	{
		TAnyType::FPolicyType::VisitPolicyByType(GetType(),
			[this, &Context, &InVal](auto SingleTypePolicy)
			{
				using FSingleType = typename decltype(SingleTypePolicy)::FType;
				FSingleType ValueToSet{};
				FDataflowConverter<typename TAnyType::FStorageType>::To(InVal, ValueToSet);

				Context.SetData(CacheKey(), GetProperty(), Forward<FSingleType>(ValueToSet), GetOwningNodeGuid(), GetOwningNodeValueHash(), GetOwningNodeTimestamp());
			});
	}

	/** Set a null value : this means that the connected input will get a default value and that this output will not be set re-evaluate next time */
	DATAFLOWCORE_API void SetNullValue(UE::Dataflow::FContext& Context) const;

	/*
	* Read value returns the cache value on the output without causing an evaluation of the corresponding node
	* as a result it does not cause a cascading evaluation of the graph
	* if there's no cached value this will return the default value passed as a parameter 
	* @see GetValue
	*/
	template<class T, typename = std::enable_if_t<!std::is_base_of_v<FDataflowAnyType, T>>>
	const T& ReadValue(UE::Dataflow::FContext& Context, const T& Default) const
	{
		if (HasFrozenValue())
		{
			return GetFrozenValue(Default);
		}
		if (Context.HasData(CacheKey()))
		{
			return Context.GetData(CacheKey(), Property, Default);
		}
		return Default;
	}

	template<typename TAnyType, typename = std::enable_if_t<std::is_base_of_v<FDataflowAnyType, TAnyType>>>
	typename TAnyType::FStorageType ReadValue(UE::Dataflow::FContext& Context, const typename TAnyType::FStorageType& Default) const
	{
		if (HasFrozenValue())
		{
			return GetFrozenValue(Default);
		}
		if (const TUniquePtr<UE::Dataflow::FContextCacheElementBase>* CacheEntry = Context.GetDataImpl(CacheKey()))
		{
			if (*CacheEntry)
			{
				typename TAnyType::FStorageType ReturnValue = Default;
				TAnyType::FPolicyType::VisitPolicyByType(GetType(),
					[this, &Context, &CacheEntry, &ReturnValue](auto SingleTypePolicy)
					{
						using FSingleType = typename decltype(SingleTypePolicy)::FType;
						FSingleType Default{};
						const FSingleType& CachedValue = (*CacheEntry)->GetTypedData<FSingleType>(Context, nullptr, Default);
						FDataflowConverter<typename TAnyType::FStorageType>::From(CachedValue, ReturnValue);
					});
				return ReturnValue;
			}
		}
		return Default;
	}

	/*
	* Get most updated value of the output
	* if the value is cache or frozen return it otherwise evaluates the node with potentially cascading evaluation of the graph 
	* @see GetValue
	*/
	template<class T>
	const T& GetValue(UE::Dataflow::FContext& Context, const T& Default) const
	{
		if (HasFrozenValue())
		{
			return GetFrozenValue(Default);
		}
		if (!this->Evaluate(Context))
		{
			Context.SetData(CacheKey(), Property, Default, GetOwningNodeGuid(), GetOwningNodeValueHash(), GetOwningNodeTimestamp());
		}

		if (Context.HasData(CacheKey()))
		{
			return Context.GetData(CacheKey(), Property, Default);
		}

		return Default;
	}

	/** Return a pointer to the node's output storage if it has been successfully evaluated and frozen, or the provided default value otherwise. */
	template<class T>
	const T& GetFrozenValue(const T& Default) const
	{
		return *(const T*)GetFrozenPropertyValue((const uint8*)&Default);
	}
	/** Freeze the value of this output to the node property bag. */
	DATAFLOWCORE_API void Freeze(UE::Dataflow::FContext& Context, struct FInstancedPropertyBag& FrozenProperties);
	/** Return whether the output has a frozen value in its owner node. */
	DATAFLOWCORE_API bool HasFrozenValue() const;

	bool HasCachedValue(const UE::Dataflow::FContext& Context) const
	{
		return Context.HasData(CacheKey(), GetOwningNodeTimestamp());
	}

	// there's no need for a templatized version as the parameter will not be used
	// the method do check if the type of the input is the same as the output type though 
	DATAFLOWCORE_API void ForwardInput(const UE::Dataflow::FConnectionReference& InputReference, UE::Dataflow::FContext& Context) const;
	DATAFLOWCORE_API void ForwardInput(const FDataflowInput* Input, UE::Dataflow::FContext& Context) const;

	DATAFLOWCORE_API bool HasValidData(UE::Dataflow::FContext& Context) const;

	DATAFLOWCORE_API bool EvaluateImpl(UE::Dataflow::FContext& Context) const;
	
	DATAFLOWCORE_API bool Evaluate(UE::Dataflow::FContext& Context) const;

	UE_DEPRECATED(5.8, "This code path is not supported, use context to evaluate nodes and connections asynchronously")
	DATAFLOWCORE_API TFuture<bool> EvaluateParallel(UE::Dataflow::FContext& Context) const;

	DATAFLOWCORE_API virtual void Invalidate(const UE::Dataflow::FTimestamp& ModifiedTimestamp = UE::Dataflow::FTimestamp::Current()) override;

	UE_DEPRECATED(5.8, "HasNodeFailedOrErrored is deprecated and will only return false, failure or error of node should be done by listening to the context log messages")
	DATAFLOWCORE_API bool HasNodeFailedOrErrored(UE::Dataflow::FContext& Context) const;

private:
	DATAFLOWCORE_API const uint8* GetFrozenPropertyValue(const uint8* Default) const;
};

template<>
struct TStructOpsTypeTraits<FDataflowOutput> : public  TStructOpsTypeTraitsBase2<FDataflowOutput>
{
	enum
	{
		WithCopy = false,
	};
};

USTRUCT()
struct FDataflowArrayOutput : public FDataflowOutput
{
	GENERATED_USTRUCT_BODY()

private:
	int32 Index;
	uint32 ElementOffset; // Offset to Property inside an array element
	const FArrayProperty* ArrayProperty = nullptr;

public:
	explicit FDataflowArrayOutput(int32 InIndex = INDEX_NONE, const UE::Dataflow::FArrayOutputParameters& Param = {});

	DATAFLOWCORE_API virtual void* RealAddress() const override;
	virtual int32 GetContainerIndex() const override { return Index; }
	virtual uint32 GetContainerElementOffset() const override { return ElementOffset; }
};

template<>
struct TStructOpsTypeTraits<FDataflowArrayOutput> : public  TStructOpsTypeTraitsBase2<FDataflowArrayOutput>
{
	enum
	{
		WithCopy = false,
	};
};

template<typename T>
const T& FDataflowInput::GetValue(UE::Dataflow::FContext& Context, const T& Default) const
{
	CheckRequired(Context);
	if (const FDataflowOutput* ConnectionOut = GetConnection())
	{
		if (ConnectionOut->HasFrozenValue())
		{
			return ConnectionOut->GetFrozenValue(Default);
		}
		if (!ConnectionOut->Evaluate(Context))
		{
			Context.SetData(ConnectionOut->CacheKey(), Property, Default, GetOwningNodeGuid(), GetOwningNodeValueHash(), GetOwningNodeTimestamp());
		}
		if (Context.HasData(ConnectionOut->CacheKey()))
		{
			const T& Data = Context.GetData(ConnectionOut->CacheKey(), Property, Default);
			return Data;
		}
	}
	return Default;
}

template<typename TAnyType UE_REQUIRES(!std::is_same_v<typename TAnyType::FStorageType, void>)>
typename TAnyType::FStorageType FDataflowInput::GetValueFromAnyType(UE::Dataflow::FContext& Context, const typename TAnyType::FStorageType& Default) const
{
	CheckRequired(Context);
	typename TAnyType::FStorageType ReturnValue = Default;
	if (const FDataflowOutput* ConnectionOut = GetConnection())
	{
		if (ConnectionOut->HasFrozenValue())
		{
			return ConnectionOut->GetFrozenValue(Default);
		}
		if (ConnectionOut->Evaluate(Context))
		{
			if (const TUniquePtr<UE::Dataflow::FContextCacheElementBase>* CacheEntry = Context.GetDataImpl(ConnectionOut->CacheKey()))
			{
				if (*CacheEntry)
				{
					TAnyType::FPolicyType::VisitPolicyByType(GetType(),
						[this, &Context, &CacheEntry, &ReturnValue](auto SingleTypePolicy)
						{
							using FSingleType = typename decltype(SingleTypePolicy)::FType;
							FSingleType Default{};
							const FSingleType& CachedValue = (*CacheEntry)->GetTypedData<FSingleType>(Context, nullptr, Default);
							FDataflowConverter<typename TAnyType::FStorageType>::From(CachedValue, ReturnValue);
						});
				}
			}
		}
	}
	return ReturnValue;
}

template<typename TAnyType UE_REQUIRES(std::is_same_v<typename TAnyType::FStorageType, void>)>
typename UE::Dataflow::FContextValue FDataflowInput::GetValueFromAnyType(UE::Dataflow::FContext& Context) const
{
	CheckRequired(Context);
	UE::Dataflow::FContextCacheKey CacheKey = 0;
	if (const FDataflowOutput* ConnectionOut = GetConnection())
	{
		if (ConnectionOut->HasFrozenValue())
		{
			//return ConnectionOut->GetFrozenValue(Default);
			ensure(false); // unsupported because the frozen values are actually not in the cache, they should probably be 
		}
		else if (ConnectionOut->Evaluate(Context))
		{
			CacheKey = ConnectionOut->CacheKey();
		}
	}
	return UE::Dataflow::FContextValue(Context, CacheKey);
}

template<class T>
UE_DEPRECATED(5.8, "This code path is not supported, use context to evaluate nodes and connections asynchronously")
TFuture<const T&> FDataflowInput::GetValueParallel(UE::Dataflow::FContext& Context, const T& Default) const
{
	return Async(EAsyncExecution::TaskGraph, [&]() -> const T& { return this->GetValue<T>(Context, Default); });
}

