// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosLog.h"
#include "CoreMinimal.h"
#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowConnection.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowSettings.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/StructOnScope.h"
#include "Templates/SharedPointer.h"
#include "Delegates/DelegateCombinations.h"

#include "DataflowNode.generated.h"

class UScriptStruct;
struct FDataflowBaseElement;

namespace UE::Dataflow {
	struct FNodeParameters {
		FName Name;
		UObject* OwningObject = nullptr;
	};
	class FGraph;
}
class IDataflowDebugDrawInterface;
class IDataflowConstructionViewMode;

struct FDataflowNodeDelegates
{
#if WITH_EDITOR
	/** Called when the node is being invalidated */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodeInvalidated, FDataflowNode&);
	static DATAFLOWCORE_API FOnNodeInvalidated OnNodeInvalidated;
#endif
};

/** Empty class for the freeze UI customization. */
USTRUCT()
struct FDataflowFreezeActions final
{
	GENERATED_USTRUCT_BODY()
};

/**
* FNode
*		Base class for node based evaluation within the Dataflow graph. 
* 
* Note : Do NOT create mutable variables in the classes derived from FDataflowNode. The state
*        is stored on the FContext. The Evaluate is const to allow support for multithreaded
*        evaluation. 
*/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct FDataflowNode
#if CPP
	: public TSharedFromThis<FDataflowNode, ESPMode::ThreadSafe>
#endif
{
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	GENERATED_USTRUCT_BODY()

	friend class UE::Dataflow::FGraph;
	friend struct FDataflowConnection;

	UE_DEPRECATED(5.7, "Will be made private in 5.9, use IsColorOverriden() instead.")
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (InlineEditConditionToggle, DisplayAfter = "bActive"))
	bool bOverrideColor = false;

	UE_DEPRECATED(5.7, "Will be made private in 5.9, use GetOverrideColor() instead.")
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (EditCondition="bOverrideColor"))
	FLinearColor OverrideColor = FLinearColor::White;

	DATAFLOWCORE_API FDataflowNode();
	DATAFLOWCORE_API FDataflowNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FDataflowNode() { ClearInputs(); ClearOutputs(); }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Warning: FDataflowNodes aren't actually safe to copy/move yet. These are here to disable deprecation warnings from the implicit operators that were getting created anyways.
	// (Deleting the operators would require tagging all derived classes since this is a USTRUCT, so also not doing that here).
	FDataflowNode& operator=(const FDataflowNode& Other)
	{
		checkNoEntry();
		return *this;
	}

	FGuid GetGuid() const { return Guid; }
	FName GetName() const { return Name; }
	void SetName(FName InName) { Name = InName; }
	UE::Dataflow::FTimestamp GetTimestamp() const { return LastModifiedTimestamp;  }
	DATAFLOWCORE_API uint32 GetValueHash();

	static FName StaticType() { return FName("FDataflowNode"); }
	virtual FName GetType() const { return StaticType(); }
	virtual FName GetDisplayName() const { return ""; }
	virtual FName GetCategory() const { return ""; }
	virtual FString GetTags() const { return ""; }
	DATAFLOWCORE_API virtual FString GetToolTip() const;
	DATAFLOWCORE_API FString GetPinToolTip(const FName& PropertyName, const UE::Dataflow::FPin::EDirection Direction = UE::Dataflow::FPin::EDirection::NONE) const;
	DATAFLOWCORE_API FText GetPinDisplayName(const FName& PropertyName, const UE::Dataflow::FPin::EDirection Direction = UE::Dataflow::FPin::EDirection::NONE) const;
	DATAFLOWCORE_API TArray<FString> GetPinMetaData(const FName& PropertyName, const UE::Dataflow::FPin::EDirection Direction = UE::Dataflow::FPin::EDirection::NONE) const;
	virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderParameters() const { return GetRenderParametersImpl(); }

	/** Override if node will generate primitives from the RenderCollection.*/
	virtual bool HasRenderCollectionPrimitives() const
	{
		return false;
	}

#if WITH_EDITOR
	struct FDebugDrawParameters
	{
		/** Boolean to check if the node is selected or not */
		bool bNodeIsSelected = false;

		/** Boolean to check if the node is pinned or not */
		bool bNodeIsPinned = false;

		/** Current view mode to use in the debug draw */
		FName CurrentViewMode = NAME_None;
	};

	/** Override if DebugDraw has been implemented.*/
	virtual bool CanDebugDraw() const
	{
		return false;
	}
	/** Specify whether or not DebugDraw supports this ViewMode */
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const
	{
		return false;
	}
	/** Debug drawing callback. This will only be called for nodes which CanDebugDrawViewMode for the current view mode.*/
	virtual void DebugDraw(UE::Dataflow::FContext& Context,
		IDataflowDebugDrawInterface& DataflowRenderingInterface,
		const FDebugDrawParameters& DebugDrawParameters) const {}

	virtual void OnDoubleClicked(UE::Dataflow::FContext* Context) const {};
#endif

	//
	// Connections
	//

	DATAFLOWCORE_API TArray<UE::Dataflow::FPin> GetPins() const;

	/** Override this function to add the AddOptionPin functionality to the node's context menu. */
	virtual TArray<UE::Dataflow::FPin> AddPins()
	{
		return TArray<UE::Dataflow::FPin>();
	}
	/** Override this function to add the AddOptionPin functionality to the node's context menu. */
	virtual bool CanAddPin() const { return false; }
	/** Override this function to add the RemoveOptionPin functionality to the node's context menu. OnPinRemoved will be called in this order.*/
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const
	{
		return TArray<UE::Dataflow::FPin>();
	}
	/** 
	 * Override this to update any bookkeeping when a pin is being removed.
	 * This will be called before the pin is unregistered as an input.
	 */
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) {}
	/** Override this function to add the RemoveOptionPin functionality to the node's context menu. */
	virtual bool CanRemovePin() const { return false; }


	DATAFLOWCORE_API bool InputSupportsType(FName Name, FName Type) const;
	DATAFLOWCORE_API bool OutputSupportsType(FName Name, FName Type) const;

	DATAFLOWCORE_API virtual void AddInput(FDataflowInput* InPtr);
	DATAFLOWCORE_API int32 GetNumInputs() const;
	DATAFLOWCORE_API TArray< FDataflowInput* > GetInputs() const;
	DATAFLOWCORE_API void ClearInputs();
	DATAFLOWCORE_API bool HasHideableInputs() const;
	DATAFLOWCORE_API bool HasHiddenInputs() const;

	DATAFLOWCORE_API const FDataflowConnection* FindConnection(const UE::Dataflow::FConnectionKey& Key) const;
	DATAFLOWCORE_API FDataflowConnection* FindConnection(const UE::Dataflow::FConnectionKey& Key);
	DATAFLOWCORE_API FDataflowConnection* FindConnection(const UE::Dataflow::FConnectionReference& Reference);

	DATAFLOWCORE_API FDataflowInput* FindInput(FName Name);
	DATAFLOWCORE_API FDataflowInput* FindInput(const UE::Dataflow::FConnectionKey& Key);
	/**This version can find array inputs if only the Reference is supplied by searching through all inputs */
	DATAFLOWCORE_API FDataflowInput* FindInput(const UE::Dataflow::FConnectionReference& Reference);
	DATAFLOWCORE_API const FDataflowInput* FindInput(FName Name) const;
	DATAFLOWCORE_API const FDataflowInput* FindInput(const UE::Dataflow::FConnectionKey& Key) const;
	/**This version can find array inputs if only the Reference is supplied by searching through all inputs */
	DATAFLOWCORE_API const FDataflowInput* FindInput(const UE::Dataflow::FConnectionReference& Reference) const;
	DATAFLOWCORE_API const FDataflowInput* FindInput(const FGuid& InGuid) const;

	DATAFLOWCORE_API virtual void AddOutput(FDataflowOutput* InPtr);
	DATAFLOWCORE_API int NumOutputs() const;
	DATAFLOWCORE_API TArray< FDataflowOutput* > GetOutputs() const;
	DATAFLOWCORE_API void ClearOutputs();
	DATAFLOWCORE_API bool HasHideableOutputs() const;
	DATAFLOWCORE_API bool HasHiddenOutputs() const;

	DATAFLOWCORE_API FDataflowOutput* FindOutput(FName Name);
	DATAFLOWCORE_API FDataflowOutput* FindOutput(uint32 GuidHash);
	DATAFLOWCORE_API FDataflowOutput* FindOutput(const UE::Dataflow::FConnectionKey& Key);
	DATAFLOWCORE_API FDataflowOutput* FindOutput(const UE::Dataflow::FConnectionReference& Reference);
	DATAFLOWCORE_API const FDataflowOutput* FindOutput(FName Name) const;
	DATAFLOWCORE_API const FDataflowOutput* FindOutput(uint32 GuidHash) const;
	DATAFLOWCORE_API const FDataflowOutput* FindOutput(const UE::Dataflow::FConnectionKey& Key) const;
	DATAFLOWCORE_API const FDataflowOutput* FindOutput(const UE::Dataflow::FConnectionReference& Reference) const;
	DATAFLOWCORE_API const FDataflowOutput* FindOutput(const FGuid& InGuid) const;

	DATAFLOWCORE_API bool TryRenameInput(const UE::Dataflow::FConnectionReference& Reference, FName NewName);
	DATAFLOWCORE_API bool TryRenameInput(FDataflowInput& InputToRename, FName NewName);
	DATAFLOWCORE_API bool TryRenameOutput(const UE::Dataflow::FConnectionReference& Reference, FName NewName);
	DATAFLOWCORE_API bool TryRenameOutput(FDataflowOutput& OutputToRename, FName NewName);

	/** Return a property's byte offset from the dataflow base node address using the full property name (must includes its parent struct property names). Does not work with array properties.*/
	uint32 GetPropertyOffset(const FName& PropertyFullName) const;

	static DATAFLOWCORE_API const FName DataflowInput;
	static DATAFLOWCORE_API const FName DataflowOutput;
	static DATAFLOWCORE_API const FName DataflowPassthrough;
	static DATAFLOWCORE_API const FName DataflowIntrinsic;
	static DATAFLOWCORE_API const FName DataflowSkipConnection;

	static DATAFLOWCORE_API const FLinearColor DefaultNodeTitleColor;
	static DATAFLOWCORE_API const FLinearColor DefaultNodeBodyTintColor;

	/** Override this method to provide custom serialization for this node. */
	virtual void Serialize(FArchive& Ar) {}
	/** Override this method to provide custom post-serialization for this node. This method will be called after Serialize. It is also called after copy-paste with
	 ArchiveState IsLoading. */
	virtual void PostSerialize(const FArchive& Ar) {}

	/** Override this method to provide custom reconnections when a node inputs has been deprecated and removed. */
	virtual FDataflowInput* RedirectSerializedInput(const FName& MissingInputName) { return nullptr; }
	/** Override this method to provide custom reconnections when a node outputs has been deprecated and removed. */
	virtual FDataflowOutput* RedirectSerializedOutput(const FName& MissingOutputName) { return nullptr; }

	//
	//  Struct Support
	//

	virtual void SerializeInternal(FArchive& Ar) { check(false); }
	virtual FStructOnScope* NewStructOnScope() { return nullptr; }
	virtual const UScriptStruct* TypedScriptStruct() const { return nullptr; }

	TUniquePtr<const FStructOnScope> NewStructOnScopeConst() const;

	/** Register the Input and Outputs after the creation in the factory. Use PropertyName to disambiguate a struct name from its first property. */
	template<typename T>
	FDataflowInput& RegisterInputConnection(const UE::Dataflow::TConnectionReference<T>& Reference, const FName& PropertyName = NAME_None)
	{
		FDataflowInput& Input = RegisterInputConnectionInternal(Reference, PropertyName);
		if constexpr (std::is_base_of_v<FDataflowAnyType, T>)
		{
			Input.SetTypePolicy(T::FPolicyType::GetInterface());
		}
		return Input;
	}

	template <typename T>
	FDataflowInput& RegisterInputConnection(const T* Reference, const FName& PropertyName = NAME_None)
	{
		return RegisterInputConnection(UE::Dataflow::TConnectionReference<T>(Reference), PropertyName);
	}

	template <typename T>
	FDataflowOutput& RegisterOutputConnection(const UE::Dataflow::TConnectionReference<T>& Reference, const UE::Dataflow::TConnectionReference<T>& Passthrough = UE::Dataflow::TConnectionReference<T>(nullptr), const FName& PropertyName = NAME_None)
	{
		FDataflowOutput& Output = RegisterOutputConnectionInternal(Reference, PropertyName);
		if constexpr (std::is_base_of_v<FDataflowAnyType, T>)
		{
			Output.SetTypePolicy(T::FPolicyType::GetInterface());
		}

		if (Passthrough.Reference != nullptr)
		{
			Output.SetPassthroughInput(Passthrough);
		}
		return Output;
	}

	template <typename T>
	FDataflowOutput& RegisterOutputConnection(const T* Reference, const T* Passthrough = nullptr, const FName& PropertyName = NAME_None)
	{
		return RegisterOutputConnection(UE::Dataflow::TConnectionReference<T>(Reference), UE::Dataflow::TConnectionReference<T>(Passthrough), PropertyName);
	}

	template<typename T>
	FDataflowInput& RegisterInputArrayConnection(const UE::Dataflow::TConnectionReference<T>& Reference, const FName& ElementPropertyName = NAME_None,
		const FName& ArrayPropertyName = NAME_None)
	{
		FDataflowInput& Input = RegisterInputArrayConnectionInternal(Reference, ElementPropertyName, ArrayPropertyName);
		if constexpr (std::is_base_of_v<FDataflowAnyType, T>)
		{
			Input.SetTypePolicy(T::FPolicyType::GetInterface());
		}
		return Input;
	}

	template<typename T>
	FDataflowInput& FindOrRegisterInputArrayConnection(const UE::Dataflow::TConnectionReference<T>& Reference, const FName& ElementPropertyName = NAME_None,
		const FName& ArrayPropertyName = NAME_None)
	{
		if (FDataflowInput* const FoundInput = FindInput(Reference))
		{
			return *FoundInput;
		}

		FDataflowInput& Input = RegisterInputArrayConnectionInternal(Reference, ElementPropertyName, ArrayPropertyName);
		if constexpr (std::is_base_of_v<FDataflowAnyType, T>)
		{
			Input.SetTypePolicy(T::FPolicyType::GetInterface());
		}
		return Input;
	}

	template<typename T>
	FDataflowOutput& RegisterOutputArrayConnection(const UE::Dataflow::TConnectionReference<T>& Reference, const FName& ElementPropertyName = NAME_None,
		const FName& ArrayPropertyName = NAME_None)
	{
		FDataflowOutput& Output = RegisterOutputArrayConnectionInternal(Reference, ElementPropertyName, ArrayPropertyName);
		if constexpr (std::is_base_of_v<FDataflowAnyType, T>)
		{
			Output.SetTypePolicy(T::FPolicyType::GetInterface());
		}
		return Output;
	}

	template<typename T>
	FDataflowOutput& FindOrRegisterOutputArrayConnection(const UE::Dataflow::TConnectionReference<T>& Reference, const FName& ElementPropertyName = NAME_None,
		const FName& ArrayPropertyName = NAME_None)
	{
		if (FDataflowOutput* const FoundOutput = FindOutput(Reference))
		{
			return *FoundOutput;
		}

		FDataflowOutput& Output = RegisterOutputArrayConnectionInternal(Reference, ElementPropertyName, ArrayPropertyName);
		if constexpr (std::is_base_of_v<FDataflowAnyType, T>)
		{
			Output.SetTypePolicy(T::FPolicyType::GetInterface());
		}
		return Output;
	}

	/** Unregister the input connection if one exists matching this property, and then invalidate the graph. */
	void UnregisterInputConnection(const UE::Dataflow::FConnectionReference& Reference)
	{
		UnregisterInputConnection(GetKeyFromReference(Reference));
	}
	DATAFLOWCORE_API void UnregisterInputConnection(const UE::Dataflow::FConnectionKey& Key);
	/** Unregister the connection if one exists matching this pin, then invalidate the graph. */
	DATAFLOWCORE_API void UnregisterPinConnection(const UE::Dataflow::FPin& Pin);

	void UnregisterOutputConnection(const UE::Dataflow::FConnectionReference& Reference)
	{
		UnregisterOutputConnection(GetKeyFromReference(Reference));
	}
	DATAFLOWCORE_API void UnregisterOutputConnection(const UE::Dataflow::FConnectionKey& Key);

	//
	// Evaluation
	//

	/** 
	* This evaluate the node based on the existing state of the outputs
	* if the node has no output then evaluate will be called with a null output 
	* otherwise Evaluate will be called for each connected output if it has an outdated cached value in the context
	* Return true if at least one output was evaluated or forwarded
	*/
	DATAFLOWCORE_API bool EvaluateNode(UE::Dataflow::FContext& Context) const;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput*) const { ensure(false); }

	virtual bool EvaluateOnGameThreadOnly() const { return false; }

	/** Freeze the node's outputs. */
	DATAFLOWCORE_API void Freeze(UE::Dataflow::FContext& Context);
	/** Unfreeze the node's outputs. */
	DATAFLOWCORE_API void Unfreeze(UE::Dataflow::FContext& Context);
	/** Return whether the node is frozen. */
	bool IsFrozen() const
	{
		return bIsFrozen;
	}
	/** Return the frozen output property bag. */
	const FInstancedPropertyBag& GetFrozenFroperties() const
	{
		return FrozenProperties;
	}

	bool IsColorOverriden() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Will be made private in 5.9, use IsColorOverriden() instead.")
		return bOverrideColor;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const FLinearColor& GetOverrideColor() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Will be made private in 5.9, use GetOverrideColor() instead.")
		return OverrideColor;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gets the property name for bActive. This exists so the property can be made private later.
	 */
	static const FName GetActivePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FDataflowNode, bActive);
	}

	/** 
	 * If bCheckFlagOnly is false, this function returns true if both the bActive property and the CVAR p.Dataflow.EnableGraphEval are true
	 * If bCheckFlagOnly is true, this function returns true if the bActive property is true
	 */
	DATAFLOWCORE_API bool IsActive(bool bCheckFlagOnly = false) const;

	/**
	*   GetValue(...)
	*
	*	Get the value of the Reference output, invoking up stream evaluations if not 
	*   cached in the contexts data store. 
	* 
	*   @param Context : The evaluation context that holds the data store.
	*   @param Reference : Pointer to a member of this node that corresponds with the output to set.
	*						*Reference will be used as the default if the input is not connected. 
	*/
	template<class T UE_REQUIRES(!std::is_base_of_v<FDataflowAnyType, T>)>
	const T& GetValue(UE::Dataflow::FContext& Context, const UE::Dataflow::TConnectionReference<T>& Reference) const
	{
		checkSlow(FindInput(Reference));
		return FindInput(Reference)->template GetValue<T>(Context, *static_cast<const T*>(Reference.Reference));
	}

	template<typename TAnyType UE_REQUIRES(std::is_base_of_v<FDataflowAnyType, TAnyType> && !std::is_same_v<typename TAnyType::FStorageType, void>)>
	typename TAnyType::FStorageType GetValue(UE::Dataflow::FContext& Context, const UE::Dataflow::TConnectionReference<TAnyType>& Reference) const
	{
		checkSlow(Reference.Reference && FindInput(Reference));
		return FindInput(Reference)->template GetValueFromAnyType<TAnyType>(Context, static_cast<const TAnyType*>(Reference.Reference)->Value);
	}

	template<typename TAnyType UE_REQUIRES(std::is_base_of_v<FDataflowAnyType, TAnyType> && std::is_same_v<typename TAnyType::FStorageType, void>)>
	typename UE::Dataflow::FContextValue GetValue(UE::Dataflow::FContext& Context, const UE::Dataflow::TConnectionReference<TAnyType>& Reference) const
	{
		checkSlow(Reference.Reference && FindInput(Reference));
		return FindInput(Reference)->template GetValueFromAnyType<TAnyType>(Context);
	}

	template<class T UE_REQUIRES(!std::is_base_of_v<FDataflowAnyType, T>)>
	const T& GetValue(UE::Dataflow::FContext& Context, const T* Reference) const
	{
		return GetValue(Context, UE::Dataflow::TConnectionReference<T>(Reference));
	}
	
	template<typename TAnyType UE_REQUIRES(std::is_base_of_v<FDataflowAnyType, TAnyType> && !std::is_same_v<typename TAnyType::FStorageType, void>)>
	typename TAnyType::FStorageType GetValue(UE::Dataflow::FContext& Context, const TAnyType* Reference) const
	{
		return GetValue(Context, UE::Dataflow::TConnectionReference<TAnyType>(Reference));
	}

	template<typename TAnyType UE_REQUIRES(std::is_base_of_v<FDataflowAnyType, TAnyType> && std::is_same_v<typename TAnyType::FStorageType, void>)>
	typename UE::Dataflow::FContextValue GetValue(UE::Dataflow::FContext& Context, const TAnyType* Reference) const
	{
		return GetValue(Context, UE::Dataflow::TConnectionReference<TAnyType>(Reference));
	}
	
	template<class T>
	UE_DEPRECATED(5.8, "This code path is not supported, use context to evaluate nodes and connections asynchronously")
	TFuture<const T&> GetValueParallel(UE::Dataflow::FContext& Context, const UE::Dataflow::TConnectionReference<T>& Reference) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		checkSlow(FindInput(Reference));
		return FindInput(Reference)->template GetValueParallel<T>(Context, *static_cast<const T*>(Reference.Reference));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	template<class T>
	UE_DEPRECATED(5.8, "This code path is not supported, use context to evaluate nodes and connections asynchronously")
	TFuture<const T&> GetValueParallel(UE::Dataflow::FContext& Context, const T* Reference) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetValueParallel(Context, UE::Dataflow::TConnectionReference<T>(Reference));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	*   GetValue(...)
	*
	*	Get the value of the Reference output, invoking up stream evaluations if not
	*   cached in the contexts data store.
	*
	*   @param Context : The evaluation context that holds the data store.
	*   @param Reference : Pointer to a member of this node that corresponds with the output to set.
	*   @param Default : Default value if the input is not connected.
	*/
	template<class T> const T& GetValue(UE::Dataflow::FContext& Context, const UE::Dataflow::TConnectionReference<T>& Reference, const T& Default) const
	{
		checkSlow(FindInput(Reference));
		return FindInput(Reference)->template GetValue<T>(Context, Default);
	}

	template<class T> const T& GetValue(UE::Dataflow::FContext& Context, const T* Reference, const T& Default) const
	{
		return GetValue(Context, UE::Dataflow::TConnectionReference<T>(Reference), Default);
	}

	template<class T> const T& GetOutputValue(UE::Dataflow::FContext& Context, const T* Reference, const T& Default) const
	{
		if (const FDataflowOutput* Output = FindOutput(Reference))
		{
			return Output->GetValue(Context, Default);
		}
		return Default;
	}

	/**
	*   SetValue(...)
	*
	*   Set the value of the Reference output.
	* 
	*   Note: If the compiler errors out with "You cannot bind an lvalue to an rvalue reference", then simply remove
	*         the explicit template parameter from the function call to allow for a const reference type to be deducted.
	*         const int32 Value = 0; SetValue<int32>(Context, Value, &Reference);  // Error: You cannot bind an lvalue to an rvalue reference
	*         const int32 Value = 0; SetValue(Context, Value, &Reference);  // Fine
	*         const int32 Value = 0; SetValue<const int32&>(Context, Value, &Reference);  // Fine
	* 
	*   @param Context : The evaluation context that holds the data store.
	*   @param Value : The value to store in the contexts data store. 
	*   @param Reference : Pointer to a member of this node that corresponds with the output to set. 
	*/
	template<class T, class U UE_REQUIRES(!std::is_base_of_v<FDataflowAnyType, U>)>
	void SetValue(UE::Dataflow::FContext& Context, T&& Value, const U* Reference) const
	{
		using FInValueType = typename TDecay<decltype(Value)>::Type;
		using FInReferenceType = typename TDecay<decltype(*Reference)>::Type;
		static_assert(std::is_same_v<FInValueType, FInReferenceType>);
		if (const FDataflowOutput* Output = FindOutput(Reference))
		{
			Output->template SetValue<T>(Forward<T>(Value), Context);
		}
		else
		{
			ensureMsgf(false, TEXT("This output could not be found within this node, check this has been properly registered in the node constructor"));
		}
	}

	template<typename TAnyType UE_REQUIRES(std::is_base_of_v<FDataflowAnyType, TAnyType> && !std::is_same_v<typename TAnyType::FStorageType, void>)>
	void SetValue(UE::Dataflow::FContext& Context, const typename TAnyType::FStorageType& Value, const TAnyType* Reference) const
	{
		if (const FDataflowOutput* Output = FindOutput(Reference))
		{
			Output->template SetValueFromAnyType<TAnyType>(Value, Context);
		}
		else
		{
			ensureMsgf(false, TEXT("This output could not be found within this node, check this has been properly registered in the node constructor"));
		}
	}

	template<typename TAnyType UE_REQUIRES(std::is_base_of_v<FDataflowAnyType, TAnyType> && std::is_same_v<typename TAnyType::FStorageType, void>)>
	void SetValue(UE::Dataflow::FContext& Context, UE::Dataflow::FContextValue&& Value, const TAnyType* Reference) const
	{
		if (const FDataflowOutput* Output = FindOutput(Reference))
		{
			Output->SetValue(Forward<UE::Dataflow::FContextValue>(Value), Context);
		}
		else
		{
			ensureMsgf(false, TEXT("This output could not be found within this node, check this has been properly registered in the node constructor"));
		}
	}

	void SetError(UE::Dataflow::FContext& Context, const void* Reference, const FString& InError) const
	{
		Context.Error(InError, this, FindOutput(Reference));
	}

	/**
	*   ForwardInput(...)
	*
	*   Forward an input to this output.
	*   This will not cache the value itself but cache a reference to the input connection cache entry.
	*   This is memory efficient and do not require a runtime copy of the data.
	*   Input and output references must match in type.
	*   Note that forwarding an input never sets a default value when no input is connected, use SafeForwardInput instead.
	*
	*   @param Context : The evaluation context that holds the data store.
	*   @param InputReference : Pointer to a input member of this node that needs to be forwarded.
	*   @param Reference : Pointer to a member of this node that corresponds with the output to set.
	*/
	DATAFLOWCORE_API void ForwardInput(UE::Dataflow::FContext& Context, const UE::Dataflow::FConnectionReference& InputReference, const UE::Dataflow::FConnectionReference& Reference) const;

	/**
	*   SafeForwardInput(...)
	*
	*   Forward an input to this output or set a default value if no input is connected.
	*   This is more memory efficient when an input is connected than setting the value.
	*   Input and output references must match in type.
	*
	*   @param Context : The evaluation context that holds the data store.
	*   @param InputReference : Pointer to a input member of this node that needs to be forwarded.
	*   @param Reference : Pointer to a member of this node that corresponds with the output to set.
	*/
	template<class T>
	void SafeForwardInput(UE::Dataflow::FContext& Context, const UE::Dataflow::FConnectionReference& InputReference, const T* Reference) const
	{
		if (IsConnected(InputReference))
		{
			ForwardInput(Context, InputReference, UE::Dataflow::TConnectionReference<T>(Reference));
		}
		else if constexpr (std::is_base_of_v<FDataflowAnyType, T>)
		{
			SetValue(Context, static_cast<const T*>(InputReference.Reference)->Value, Reference);
		}
		else
		{
			SetValue(Context, *static_cast<const T*>(InputReference.Reference), Reference);
		}
	}

	/**
	* This method infers the array size from an input and set it on the specified output.
	* When called, the input will be pulled as a result.
	* If the input is not an array, the value set on the output will be 0
	*/
	UE_DEPRECATED(5.7, "Use UE::Dataflow::FContextValue from calling GetValue on the node to access the same functionality")
	DATAFLOWCORE_API void SetArraySizeFromInput(UE::Dataflow::FContext& Context, const UE::Dataflow::FConnectionReference& InputReference, const int32* OutputReference)  const;

	/**
	* This method extracts an element from the specified input at a specific index and set it as output to the specified output.
	* When called, the input will be pulled as a result.
	* If the input is not an array or if the index is out of bounds, a null value will be set on the output (which will result in a default value when read from) 
	*/
	UE_DEPRECATED(5.7, "Use UE::Dataflow::FContextValue from calling GetValue on the node to access the same functionality")
	DATAFLOWCORE_API void SetArrayElementFromInput(UE::Dataflow::FContext& Context, const UE::Dataflow::FConnectionReference& InputReference, int32 Index, const void* OutputReference) const;

	/*
	* Go through all connections and execute a function
	* Signature of the function is bool Function(FDataflowConnection* Connection)
	*/
	template <typename TFunction>
	void ForEachConnection(TFunction Function) const
	{
		for (const TPair<UE::Dataflow::FConnectionKey, FDataflowInput*>& Elem : ExpandedInputs)
		{
			if (Function(Elem.Value) == false)
			{
				return;
			}
		}
		for (const TPair<UE::Dataflow::FConnectionKey, FDataflowOutput*>& Elem : ExpandedOutputs)
		{
			if (Function(Elem.Value) == false)
			{
				return;
			}
		}
	}

	/**
	*   IsConnected(...)
	*
	*	Checks if Reference input is connected.
	*
	*   @param Reference : Pointer to a member of this node that corresponds with the input.
	*/
	bool IsConnected(const UE::Dataflow::FConnectionReference& Reference) const
	{
		checkSlow(FindInput(Reference));
		return (FindInput(Reference)->GetConnection() != nullptr);
	}

	template<typename T>
	bool IsConnected(const T* Reference) const
	{
		return IsConnected(UE::Dataflow::FConnectionReference(Reference));
	}

	void PauseInvalidations()
	{
		if (!bPauseInvalidations)
		{
			bPauseInvalidations = true;
			PausedModifiedTimestamp = UE::Dataflow::FTimestamp::Invalid;
		}
	}

	void ResumeInvalidations()
	{
		if (bPauseInvalidations)
		{
			bPauseInvalidations = false;
			Invalidate(PausedModifiedTimestamp);
		}
	}

	DATAFLOWCORE_API void Invalidate(const UE::Dataflow::FTimestamp& ModifiedTimestamp = UE::Dataflow::FTimestamp::Current());

	virtual void OnInvalidate() {}

	DATAFLOWCORE_API virtual bool ValidateConnections();

	DATAFLOWCORE_API virtual void ValidateProperties();

	// return true if the node supports a specific type of asset to be assign to it 
	virtual bool SupportsAssetProperty(UObject* Asset) const { return false; }

	// Set an asset property on this node ( called when SupportsAssetProperty returns true )
	virtual void SetAssetProperty(UObject* Asset) {};

	// Called when UPROPERTY members of the dataflow node have been changed in the editor
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) {}

	// check if the node support dropped connections
	virtual bool SupportsDropConnectionOnNode(FName TypeName, UE::Dataflow::FPin::EDirection Direction) const { return false; }

	// called when a dragged pin is dropped on the node 
	// if the node creates a connection, returns its pointer
	virtual const FDataflowConnection* OnDropConnectionOnNode(const FDataflowConnection& DroppedConnection) { return nullptr; }

	// Override when a property change musn't invalidate the node. Must return the base implementation when the property name isn't recognised.
	DATAFLOWCORE_API virtual bool ShouldInvalidateOnPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent) const;

	// returns true to disable node properties in its detail view when the corresponding input is connected ( default is true )
	virtual bool MakeConnectedPropertiesReadOnly() const { return true; }

	struct FAttributeKey
	{
		FName AttributeName;
		FName GroupName;
	};

	/**
	* return a vertex attribute to visualize
	* This attribute will be used as vertex color and displayed in the viewport 
	* this is only used for collection based visualizations
	*/
	virtual FAttributeKey GetVertexAttributeToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup) const { return {}; }

	/**
	* Fills a dataflow selection object used to visualize the selection a node may output for a sepcific output and render group
	*/
	virtual void GetSelectionToVisualize(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup, FDataflowSelection& OutSelection) const { }

	// this method is called when the construction viewport render a specific output in the viewport 
	// this allow the node to be able to set extra properties on the component or set it up differently 
	virtual void OnRenderOutput(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup, const TArray<UPrimitiveComponent*>& RenderComponents) const {};

	bool HasValidConnections() const { return bHasValidConnections; }

	/** 
	* Define is a node is a terminal type of node 
	* terminal nodes can be pulled from to generate content (see FDataflowTerminalNode ) 
	* or write results to a simulation for example ( see FDataflowSimulationNode )
	* The dataflow compiler also use this infromation to decide what part of the graph is a dead end and can be optimized out 
	*/
	virtual bool IsTerminal() const
	{
		return false;
	}

	virtual bool IsA(FName InType) const 
	{ 
		return InType.ToString().Equals(StaticType().ToString());
	}

	template<class T>
	const T* AsType() const
	{
		FName TargetType = T::StaticType();
		if (IsA(TargetType))
		{
			return (T*)this;
		}
		return nullptr;
	}

	template<class T>
	T* AsType()
	{
		FName TargetType = T::StaticType();
		if (IsA(TargetType))
		{
			return (T*)this;
		}
		return nullptr;
	}

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodeInvalidated, FDataflowNode*);
	FOnNodeInvalidated& GetOnNodeInvalidatedDelegate() { return OnNodeInvalidatedDelegate; }

	// returns true if the type was changed successfully
	// only unset datatype connection will be set a new type 
	DATAFLOWCORE_API bool TrySetConnectionType(FDataflowConnection* Connection, FName NewType);

	// Only used when forcing types on connection in order to make sure the node properly refreshes the rest of its connection accordingly if there's any dependencies between their types
	DATAFLOWCORE_API void NotifyConnectionTypeChanged(FDataflowConnection* Connection);

	// returns true if the input type was changed successfully
	DATAFLOWCORE_API bool SetInputConcreteType(const UE::Dataflow::FConnectionReference& InputReference, FName NewType);

	// returns true if the output type was changed successfully
	DATAFLOWCORE_API bool SetOutputConcreteType(const UE::Dataflow::FConnectionReference& OutputReference, FName NewType);

	// returns true if the output type was changed successfully
	DATAFLOWCORE_API bool SetOutputConnectionConcreteType(FDataflowConnection* Output, FName NewType);

	/** Check if any connection matching a specific type dependency group are connected */
	DATAFLOWCORE_API bool IsAnytypeDependencyConnected(FName DependencyGroup, const FDataflowConnection* IgnoreConnection = nullptr) const;

	bool IsAsyncEvaluating() const { return bIsAsyncEvaluating; }
	void SetAsyncEvaluating(bool bValue) const { bIsAsyncEvaluating = bValue; }

	DATAFLOWCORE_API TSharedPtr<UE::Dataflow::FGraph> GetDataflowGraph() const;

protected:
	bool OnConnectionTypeChanged(const FDataflowConnection& Connection);

	/** 
	* When enforcing type dependency returns the type of the target connection from a source one being changed
	* the default implementation returns the same type 
	*/
	DATAFLOWCORE_API virtual FName GetDependentConnectionType(const FDataflowConnection& SourceConnection, const FDataflowConnection& DependentConnection) const;

	template <typename T>
	bool SetInputConcreteType(const UE::Dataflow::FConnectionReference& InputReference)
	{
		return SetInputConcreteType(InputReference, TDataflowPolicyTypeName<T>::GetName());
	}

	template <typename T>
	bool SetOutputConcreteType(const UE::Dataflow::FConnectionReference& OutputReference)
	{
		return SetOutputConcreteType(OutputReference, TDataflowPolicyTypeName<T>::GetName());
	}

	/**
	* Set a connection identified by key to be of a specific type
	* If the connection is already connected, the method prevents the connection from changing and returns false
	* returns true if the connection has successfully changed type
	* */
	DATAFLOWCORE_API bool SetConnectionConcreteType(const UE::Dataflow::FConnectionKey& ConnectionKey, FName NewType);

	/**
	* Set a connection to be of a specific type or optional set its type dependency group
	* If the connection is already connected, the method prevents the connection from changing and returns false
	* The type dependency group can only be set on anytype connections and only if the connecion does not have already a type depencency 
	* returns true if the connection has successfully changed type 
	* */
	DATAFLOWCORE_API bool SetConnectionConcreteType(FDataflowConnection* Connection, FName NewType, FName InTypeDependencyGroup = NAME_None);

	// returns true if any of the types was changed successfully
	DATAFLOWCORE_API bool SetAllConnectionConcreteType(FName NewType);

	DATAFLOWCORE_API FDataflowInput& RegisterInputConnectionInternal(const UE::Dataflow::FConnectionReference& Reference, const FName& PropertyName = NAME_None);
	DATAFLOWCORE_API FDataflowOutput& RegisterOutputConnectionInternal(const UE::Dataflow::FConnectionReference& Reference, const FName& PropertyName = NAME_None);
	DATAFLOWCORE_API FDataflowInput& RegisterInputArrayConnectionInternal(const UE::Dataflow::FConnectionReference& Reference, const FName& ElementPropertyName = NAME_None, const FName& ArrayPropertyName = NAME_None);
	DATAFLOWCORE_API FDataflowOutput& RegisterOutputArrayConnectionInternal(const UE::Dataflow::FConnectionReference& Reference, const FName& ElementPropertyName = NAME_None, const FName& ArrayPropertyName = NAME_None);

private:

	void InitConnectionParametersFromPropertyReference(const FStructOnScope& StructOnScope, const void* PropertyRef, const FName& PropertyName, UE::Dataflow::FConnectionParameters& OutParams);
	// This will add [ContainerIndex] to any array it finds unless ContainerIndex == INDEX_NONE.
	static FString GetPropertyFullNameString(const TConstArrayView<const FProperty*>& PropertyChain, int32 ContainerIndex = INDEX_NONE);
	static FName GetPropertyFullName(const TArray<const FProperty*>& PropertyChain, int32 ContainerIndex = INDEX_NONE);
	static FText GetPropertyDisplayNameText(const TArray<const FProperty*>& PropertyChain, int32 ContainerIndex = INDEX_NONE);
	static FString StripContainerIndexFromPropertyFullName(const FString& PropertyFullName);
	static uint32 GetPropertyOffset(const TArray<const FProperty*>& PropertyChain);
	uint32 GetConnectionOffsetFromReference(const void* Reference) const;
	DATAFLOWCORE_API UE::Dataflow::FConnectionKey GetKeyFromReference(const UE::Dataflow::FConnectionReference& Reference) const;

	/**
	* Find a property using the property address and name (not including its parent struct property names).
	* If NAME_None is used as the name, and the same address is shared by a parent structure property and
	* its first child property, then the parent will be returned.
	*/
	const FProperty* FindProperty(const UStruct* Struct, const void* Property, const FName& PropertyName, TArray<const FProperty*>* OutPropertyChain = nullptr) const;
	const FProperty& FindPropertyChecked(const UStruct* Struct, const void* Property, const FName& PropertyName, TArray<const FProperty*>* OutPropertyChain = nullptr) const;

	/** Find a property using the property full name (must includes its parent struct property names). */
	const FProperty* FindProperty(const UStruct* Struct, const FName& PropertyFullName, TArray<const FProperty*>* OutPropertyChain = nullptr) const;

	virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderParametersImpl() const { return TArray<UE::Dataflow::FRenderingParameter>(); }

	FGuid Guid;
	FName Name;
	UE::Dataflow::FTimestamp LastModifiedTimestamp;

	bool bHasValidConnections = true;
	TMap<UE::Dataflow::FConnectionKey, FDataflowInput*> ExpandedInputs;
	TMap<uint32, const FArrayProperty*> InputArrayProperties; // Used to calculate ContainerElementOffsets

	TMap<UE::Dataflow::FConnectionKey, FDataflowOutput*> ExpandedOutputs;
	TMap<uint32, const FArrayProperty*> OutputArrayProperties; // Used to calculate ContainerElementOffsets

#if 0  // Disabled for 5.6
#if WITH_EDITORONLY_DATA
	// Freeze node customization
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FDataflowFreezeActions FreezeActions;
#endif
#endif

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	bool bActive = true;


	UPROPERTY()
	FInstancedPropertyBag FrozenProperties;  // Property bag for frozen outputs

	UPROPERTY()
	bool bIsFrozen = false;

	TWeakPtr<UE::Dataflow::FGraph> WeakDataflowGraph;

	mutable TAtomic<bool> bIsAsyncEvaluating = false;

protected:
	bool bPauseInvalidations = false;
	UE::Dataflow::FTimestamp PausedModifiedTimestamp = UE::Dataflow::FTimestamp::Invalid; // When unpausing invalidations, Invalidate will be called with this timestamp.
	FOnNodeInvalidated OnNodeInvalidatedDelegate;

};

namespace UE::Dataflow
{

class FDataflowNodePauseInvalidationScope
{
public:
	explicit FDataflowNodePauseInvalidationScope(FDataflowNode* InNode)
		:Node(InNode)
	{
		if (Node)
		{
			Node->PauseInvalidations();
		}
	}

	~FDataflowNodePauseInvalidationScope()
	{
		if (Node)
		{
			Node->ResumeInvalidations();
		}
	}

	FDataflowNodePauseInvalidationScope() = delete;
	FDataflowNodePauseInvalidationScope(const FDataflowNodePauseInvalidationScope&) = delete;
	FDataflowNodePauseInvalidationScope(FDataflowNodePauseInvalidationScope&&) = delete;
private:
	FDataflowNode* Node;
};

	//
	// Use these macros to register dataflow nodes. 
	//

#define DATAFLOW_NODE_REGISTER_CREATION_FACTORY(A)									\
	::UE::Dataflow::FNodeFactory::RegisterNodeFromType<A>();

#define DATAFLOW_NODE_RENDER_TYPE(A, B, ...)												\
	virtual TArray<::UE::Dataflow::FRenderingParameter> GetRenderParametersImpl() const {		\
		TArray<::UE::Dataflow::FRenderingParameter> Array;								\
		Array.Add({ A, B, {__VA_ARGS__,} });														\
		return Array;}

#define DATAFLOW_NODE_RENDER_TYPE_START() \
	virtual TArray<::UE::Dataflow::FRenderingParameter> GetRenderParametersImpl() const {		\
		TArray<::UE::Dataflow::FRenderingParameter> Array;								

#define DATAFLOW_NODE_RENDER_TYPE_ADD(A, B, ...)												\
		Array.Add({ A, B, {__VA_ARGS__,} });	

#define DATAFLOW_NODE_RENDER_TYPE_END()	\
		return Array;}

#define DATAFLOW_NODE_DEFINE_INTERNAL(TYPE, DISPLAY_NAME, CATEGORY, TAGS)			\
public:																				\
	static FName StaticType() {return #TYPE;}										\
	static const char* StaticDisplay() {return DISPLAY_NAME;}						\
	static const char* StaticCategory() {return CATEGORY;}							\
	static FString StaticTags() {return TAGS;}										\
	static FString StaticToolTip() {return FString("Create a dataflow node.");}		\
	virtual FName GetType() const { return #TYPE; }									\
	virtual bool IsA(FName InType) const override									\
		{ return InType.ToString().Equals(StaticType().ToString()) || Super::IsA(InType); }	\
	virtual FStructOnScope* NewStructOnScope() override {							\
	   return new FStructOnScope(TYPE::StaticStruct(), (uint8*)this);}				\
	virtual void SerializeInternal(FArchive& Ar) override {							\
		UScriptStruct* const Struct = TYPE::StaticStruct();							\
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);		\
		Serialize(Ar);																\
		PostSerialize(Ar);}															\
	virtual FName GetDisplayName() const override { return TYPE::StaticDisplay(); }	\
	virtual FName GetCategory() const override { return TYPE::StaticCategory(); }	\
	virtual FString GetTags() const override { return TYPE::StaticTags(); }			\
	virtual const UScriptStruct* TypedScriptStruct() const override					\
		{return TYPE::StaticStruct();}												\
	TYPE() {}																		\
private:

#define DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY(CategoryName, C1, C2)	\
{																								\
	::UE::Dataflow::FNodeColorsRegistry::Get().RegisterNodeColors(CategoryName, {C1, C2});		\
}																								\

#define DATAFLOW_REGISTER_CATEGORY_ICON_BY_CATEGORY(CategoryName, IconName)						\
{																								\
	::UE::Dataflow::FIconCategoryRegistry::Get().RegisterCategoryIcon(CategoryName, IconName);	\
}																								\

#define DATAFLOW_REGISTER_CATEGORY_ICON_AND_STYLE_BY_CATEGORY(CategoryName, IconName, StyleName)			\
{																											\
	::UE::Dataflow::FIconCategoryRegistry::Get().RegisterCategoryIcon(CategoryName, IconName, StyleName);	\
}																											\

#define DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE(A, C, T)		\
{																					\
	::UE::Dataflow::FPinSettingsRegistry::Get().RegisterPinSettings(A, {C, T});		\
}																					\

#define DATAFLOW_NODE_REGISTER_GETTER_FOR_ASSET(ASSETTYPE, NODETYPE)	\
	::UE::Dataflow::FNodeFactory::GetInstance()->RegisterGetterNodeForAssetType(ASSETTYPE::StaticClass()->GetFName(), #NODETYPE);

}
