// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowGraph.h"
#include "Dataflow/DataflowSettings.h"
#include "ChaosLog.h"
#include "Textures/SlateIcon.h"

struct FDataflowNode;
struct FDataflowConnection;

class FLazySingleton;

namespace UE::Dataflow
{
	/** Default color scheme for Dataflow nodes before any category-specific overrides are registered. */
	struct FColorScheme
	{
		static inline const FLinearColor Asset = FColor(180, 120, 110);
		static inline const FLinearColor NodeHeader = FColor(180, 120, 110);
		static inline const FLinearColor NodeBody = FColor(18, 12, 11, 127);
	};

	//
	// Registry for custom Node colors
	//
	class FNodeColorsRegistry
	{
	public:
		static DATAFLOWCORE_API FNodeColorsRegistry& Get();
		static DATAFLOWCORE_API void TearDown();

		DATAFLOWCORE_API void RegisterNodeColors(const FName& Category, const FNodeColors& NodeColors);
		DATAFLOWCORE_API FLinearColor GetNodeTitleColor(const FName& Category) const;
		DATAFLOWCORE_API FLinearColor GetNodeBodyTintColor(const FName& Category) const;
		DATAFLOWCORE_API void NodeColorsChangedInSettings(const FNodeColorsMap& NodeColorsMap);

	private:
		DATAFLOWCORE_API FNodeColorsRegistry();
		DATAFLOWCORE_API ~FNodeColorsRegistry();

		FNodeColorsMap ColorsMap;					// [Category] -> Colors
		FDelegateHandle DataflowSettingsChangedDelegateHandle;

		friend FLazySingleton;
	};

	//
	// Registry for custom Pin colors
	//
	class FPinSettingsRegistry
	{
	public:
		static DATAFLOWCORE_API FPinSettingsRegistry& Get();
		static DATAFLOWCORE_API void TearDown();

		DATAFLOWCORE_API void RegisterPinSettings(const FName& PinType, const FPinSettings& InSettings);
		DATAFLOWCORE_API FLinearColor GetPinColor(const FName& PinType) const;
		DATAFLOWCORE_API float GetPinWireThickness(const FName& PinType) const;
		DATAFLOWCORE_API void PinSettingsChangedInSettings(const FPinSettingsMap& PinSettingsrMap);
		DATAFLOWCORE_API bool IsPinTypeRegistered(const FName& PinType) const;

	private:
		DATAFLOWCORE_API FPinSettingsRegistry();
		DATAFLOWCORE_API ~FPinSettingsRegistry();

		FPinSettingsMap SettingsMap;					// [PinType] -> {Color, WireThickness}
		FDelegateHandle DataflowSettingsChangedDelegateHandle;

		friend FLazySingleton;
	};

	//
	// Registry for category icons
	//
	class FIconCategoryRegistry
	{
	public:
		static DATAFLOWCORE_API FIconCategoryRegistry& Get();

		DATAFLOWCORE_API void RegisterCategoryIcon(const TCHAR* Category, const TCHAR* IconName, const TCHAR* StyleName = nullptr);
		DATAFLOWCORE_API FSlateIcon GetCategoryIcon(const FName Category) const;

	private:
		struct FIconInfo
		{
			FName IconName;
			FName StyleName;
		};

		TMap<FName, FIconInfo> IconsByCategory;
	};

}

