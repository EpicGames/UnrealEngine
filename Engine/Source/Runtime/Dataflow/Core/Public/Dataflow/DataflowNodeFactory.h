// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowGraph.h"
#include "UObject/Package.h"

struct FDataflowNode;
struct FDataflowConnection;

namespace UE::Dataflow
{
	struct FNewNodeParameters
	{
		FGuid Guid;
		FName Type;
		FName Name;
		UObject* OwningObject = nullptr;
	};

	struct FFactoryParameters
	{
		FName TypeName;
		FName DisplayName;
		FName Category;
		FString Tags;
		FString ToolTip;
		bool bIsDeprecated = false;
		bool bIsExperimental = false;
		FName NodeVersion = FName("v1");
		TSharedPtr<FDataflowNode> DefaultNodeObject;

		bool IsValid() const
		{
			return !TypeName.ToString().IsEmpty() && !DisplayName.ToString().IsEmpty();
		}

		bool IsDeprecated() const
		{
			return bIsDeprecated;
		}

		bool IsExperimental() const
		{
			return bIsExperimental;
		}

		FName GetVersion() const
		{
			return NodeVersion;
		}
	};

	//
	//
	//
	class FNodeFactory
	{
	public:
		typedef TFunction<TUniquePtr<FDataflowNode> (const FNewNodeParameters&)> FNewNodeFunction;

	private:
		typedef TUniquePtr<FDataflowNode> FRawNewNodeFunction(const FNewNodeParameters&);

		// All Maps indexed by TypeName
		TMap<FName, FNewNodeFunction > ClassMap;			// [TypeName] -> NewNodeFunction
		TMap<FName, FFactoryParameters > ParametersMap;		// [TypeName] -> Parameters
		TMap<FName, TArray<FName>> VersionMap;				// [TypeNameNoVersion] -> Array of TypeName(versions)

		TMap<FName, FName> GetterNodesByAssetType;	// list of getter nodes that relate to a specific asset type

		DATAFLOWCORE_API static FNodeFactory* Instance;
		DATAFLOWCORE_API FNodeFactory();

		static DATAFLOWCORE_API void RegisterNodeStaticInternal(
			UScriptStruct* StaticStruct, FName StaticType, const char* StaticDisplay, const char* StaticCategory,
			const FString& StaticTags, FRawNewNodeFunction CreationFunction
		);

	public:
		DATAFLOWCORE_API ~FNodeFactory();

		static DATAFLOWCORE_API FNodeFactory* GetInstance();

		template<typename T>
		static void RegisterNodeFromType()
		{
			auto CreationFunction = [](const UE::Dataflow::FNewNodeParameters& InParam)
			{
				TUniquePtr<FDataflowNode> Val = MakeUnique<T>(FNodeParameters { InParam.Name, InParam.OwningObject }, InParam.Guid);
				Val->ValidateProperties();
				Val->ValidateConnections();
				return Val;
			};

			RegisterNodeStaticInternal(
				T::StaticStruct(), T::StaticType(), T::StaticDisplay(), T::StaticCategory(),
				T::StaticTags(), CreationFunction
			);
		}

		DATAFLOWCORE_API void RegisterGetterNodeForAssetType(FName AssetTypeName, FName NodeTypeName);

		DATAFLOWCORE_API FName GetGetterNodeFromAssetClass(const UClass& AssetClass) const;

		DATAFLOWCORE_API const FFactoryParameters& GetParameters(FName InTypeName) const;

		DATAFLOWCORE_API TSharedPtr<FDataflowNode> NewNodeFromRegisteredType(FGraph& Graph, const FNewNodeParameters& Param);

		DATAFLOWCORE_API TArray<FFactoryParameters> RegisteredParameters() const;

		DATAFLOWCORE_API static bool IsNodeDeprecated(const FName NodeType);

		DATAFLOWCORE_API static bool IsNodeExperimental(const FName NodeType);

	private:
		DATAFLOWCORE_API void RegisterNode(const FFactoryParameters& Parameters, FNewNodeFunction NewFunction);

		DATAFLOWCORE_API static FName GetVersionFromTypeName(const FName& TypeName);

		DATAFLOWCORE_API static int32 GetNumVersionFromVersion(const FName& Version);

		DATAFLOWCORE_API static bool IsNodeDeprecated(const UStruct* Struct);

		DATAFLOWCORE_API static bool IsNodeExperimental(const UStruct* Struct);

		DATAFLOWCORE_API static FName GetTypeNameNoVersion(const FName& TypeName);

		DATAFLOWCORE_API static FName GetDisplayNameNoVersion(const FName& DisplayName);

		DATAFLOWCORE_API static FString GetToolTipFromStruct(UScriptStruct* InStruct, const FName& InTypeName, const FName& InDisplayName);

		template<class T> TSharedPtr<T> NewNode(FGraph& Graph, const FNewNodeParameters& Param)
		{
			return Graph.AddNode(MakeShared<T>(Param.Name, Param.Guid));
		}

		DATAFLOWCORE_API void RegisterDefaultNodes();
	};
}


