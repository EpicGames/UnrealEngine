// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/ScopeExit.h"
#include "UObject/UnrealType.h"
#include "Templates/UniquePtr.h"
#include "Templates/SharedPointer.h"
#include "HAL/CriticalSection.h"
#include "Dataflow/DataflowContextCache.h"
#include "Dataflow/DataflowContextAssetStore.h"
#include "Dataflow/DataflowContextEvaluator.h"
#include "Serialization/Archive.h"

class  UDataflow;
struct FDataflowNode;
struct FDataflowOutput;
struct FDataflowConnection;

#define DATAFLOW_EDITOR_EVALUATION WITH_EDITOR

namespace UE::Dataflow
{
	struct FRenderingParameter 
	{
		FRenderingParameter() {}
		FRenderingParameter(FString InRenderName, FName InTypeName, const TArray<FName>& InOutputs, const FName& InViewMode = FName("3DView"))
			: Name(InRenderName), Type(InTypeName), Outputs(InOutputs), ViewMode(InViewMode) {}
		FRenderingParameter(FString InRenderName, FName InTypeName, TArray<FName>&& InOutputs, const FName& InViewMode = FName("3DView"))
			: Name(InRenderName), Type(InTypeName), Outputs(InOutputs), ViewMode(InViewMode) {}
		
		bool operator==(const FRenderingParameter& Other) const = default;

		FString Name = FString();
		FName Type = FName();
		TArray<FName> Outputs;
		FName ViewMode = FName();
	};

	//////////////////////////////////////////////////////////////////////////////////

	/*
	* Performance data per connection 
	*/
	struct FContextPerfData
	{
		void Reset();
		void Accumulate(const FDataflowConnection* Connection, uint64 TotalTime, uint64 ExternalTime);
		void Append(const FContextPerfData& InPerfData);

		struct FData
		{
			float InclusiveTimeMs = 0;
			float ExclusiveTimeMs = 0;
			int32 NumCalls = 0;
			FTimestamp LastTimestamp = FTimestamp(0); // timestamp of the owner node when data was last collected
		};
		TMap<FGuid, FData> DataPerOutput;
		bool bEnabled = false;
	};

	//////////////////////////////////////////////////////////////////////////////////

	/*
	* Connection context callstack
	* Used to detected loops and use for error handling
	*/
	struct FContextCallstack
	{
		void Push(const FDataflowConnection* Connection);
		void Pop(const FDataflowConnection* Connection, uint64& OutTotalTime, uint64& OutExternalTime);
		const FDataflowConnection* Top() const;

		int32 Num() const;
		const FDataflowConnection* operator[](int32 Index);
		bool Contains(const FDataflowConnection* Connection) const;

		struct FEntry
		{
			const FDataflowConnection* Connection;
			uint64 StartTime = 0;
			uint64 ExternalTime = 0;

			bool operator==(const FDataflowConnection* OtherConnection) const
			{
				return (OtherConnection == Connection);
			}
		};
		TArray<FEntry> Callstack;
	};

	//////////////////////////////////////////////////////////////////////////////////
	class FScopedOptionalLock final
	{
	public:
		UE_NODISCARD_CTOR explicit FScopedOptionalLock(FCriticalSection* InCriticalSection)
			: CriticalSection(InCriticalSection)
		{
			if (CriticalSection)
			{
				CriticalSection->Lock();
			}
		}

		~FScopedOptionalLock()
		{
			Unlock();
		}

		void Unlock()
		{
			if (CriticalSection)
			{
				CriticalSection->Unlock();
				CriticalSection = nullptr;
			}
		}

	private:
		FScopedOptionalLock() = delete;
		FScopedOptionalLock(const FScopedOptionalLock& InScopeLock) = delete;
		FScopedOptionalLock& operator=(const FScopedOptionalLock& InScopeLock) = delete;

	private:
		FCriticalSection* CriticalSection;
	};

	//////////////////////////////////////////////////////////////////////////////////

	/*
	* Dataflow context base class
	*/
	class FContext: public IContextCacheStore, public IContextAssetStoreInterface
	{
	protected:
		FContext(FContext&&) = delete;
		FContext& operator=(FContext&&) = delete;
		
		FContext(const FContext&) = delete;
		FContext& operator=(const FContext&) = delete;

		FContextCache DataStore;
		TUniquePtr<FCriticalSection> DataLock;

		FContextEvaluator AsyncEvaluator;

	public:
		DATAFLOWCORE_API FContext();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual ~FContext() = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		static FName StaticType() { return FName("FContext"); }

		virtual bool IsA(FName InType) const { return InType==StaticType(); }

		virtual FName GetType() const { return FContext::StaticType(); }

		template<class T>
		const T* AsType() const
		{
			if (IsA(T::StaticType()))
			{
				return static_cast<const T*>(this);
			}
			return nullptr;
		}

		template<class T>
		T* AsType()
		{
			if (IsA(T::StaticType()))
			{
				return static_cast<T*>(this);
			}
			return nullptr;
		}

		bool IsThreaded() const { return DataLock.IsValid(); }
		DATAFLOWCORE_API void SetThreaded(bool bValue);

		DATAFLOWCORE_API bool IsAsyncEvaluating() const;
		DATAFLOWCORE_API void CancelAsyncEvaluation();
		DATAFLOWCORE_API void GetAsyncEvaluationStats(int32& OutNumPendingTasks, int32& OutNumRunningTasks, int32& OutNumCompletedTasks) const;

		DATAFLOWCORE_API virtual void SetDataImpl(FContextCacheKey Key, TUniquePtr<FContextCacheElementBase>&& DataStoreEntry);
		DATAFLOWCORE_API virtual const TUniquePtr<FContextCacheElementBase>* GetDataImpl(FContextCacheKey Key) const;
		DATAFLOWCORE_API virtual bool HasDataImpl(FContextCacheKey Key, FTimestamp InTimestamp = FTimestamp::Invalid) const;
		DATAFLOWCORE_API virtual bool IsEmptyImpl() const;
		DATAFLOWCORE_API virtual int32 GetKeys(TSet<FContextCacheKey>& InKeys) const;

		DATAFLOWCORE_API void Evaluate(const FDataflowNode* Node, const FDataflowOutput* Output, FOnPostEvaluationFunction PostEvaluationFunction);

		DATAFLOWCORE_API virtual void Evaluate(const FDataflowNode* Node, const FDataflowOutput* Output );
		DATAFLOWCORE_API virtual bool Evaluate(const FDataflowOutput& Connection );

		DATAFLOWCORE_API void ClearAllData();

		template<typename T>
		void SetData(FContextCacheKey InKey, const FProperty* InProperty, T&& InValue, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp)
		{
			TUniquePtr<FContextCacheElementBase> DataStoreEntry;

			if constexpr (std::is_same_v<T, FContextValue>)
			{
				if (const FContextCacheElementBase* CacheEntryToClone = InValue.GetCacheEntryPrivate())
				{
					DataStoreEntry = CacheEntryToClone->Clone(*this);
					if (DataStoreEntry)
					{
						DataStoreEntry->UpdatePropertyAndNodeData(InProperty, InNodeGuid, InNodeHash, InTimestamp);
					}
				}
			}
			else
			{
				bool bMadeEntry = false;
				if constexpr (TIsTArray<typename TDecay<T>::Type>::Value)
				{
					typedef typename TDecay<T>::Type ArrayType;
					if constexpr (TIsUObjectPtrElement<typename ArrayType::ElementType>::Value)
					{
						bMadeEntry = true;
						DataStoreEntry = MakeUnique<TContextCacheElementUObjectArray<typename ArrayType::ElementType>>(InNodeGuid, InProperty, Forward<T>(InValue), InNodeHash, InTimestamp);
					}
					else if constexpr (TIsReflectedStruct<ArrayType>::Value)
					{
						DataStoreEntry = MakeUnique<FContextCacheElementUStructArray>(InNodeGuid, InProperty, Forward<T>(InValue), InNodeHash, InTimestamp);
					}
				}
				if (!bMadeEntry)
				{
					if constexpr (TIsUObjectPtrElement<T>::Value)
					{
						DataStoreEntry = MakeUnique<TContextCacheElementUObject<T>>(InNodeGuid, InProperty, Forward<T>(InValue), InNodeHash, InTimestamp);
					}
					else if constexpr (TIsReflectedStruct<T>::Value)
					{
						DataStoreEntry = MakeUnique<FContextCacheElementUStruct>(InNodeGuid, InProperty, Forward<T>(InValue), InNodeHash, InTimestamp);
					}
					else
					{
						DataStoreEntry = MakeUnique<TContextCacheElement<T>>(InNodeGuid, InProperty, Forward<T>(InValue), InNodeHash, InTimestamp);
					}
				}
			}

			if (DataStoreEntry)
			{
				SetDataImpl(InKey, MoveTemp(DataStoreEntry));
			}
			else
			{
				SetNullData(InKey, InProperty, InNodeGuid, InNodeHash, InTimestamp);
			}
		}

		DATAFLOWCORE_API void SetDataFromStructView(FContextCacheKey InKey, const FProperty* InProperty, const FConstStructView& StructView, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp);

		DATAFLOWCORE_API void SetDataFromStructArrayView(FContextCacheKey InKey, const FProperty* InProperty, const FConstStructArrayView& StructArrayView, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp);

		DATAFLOWCORE_API void SetDataReference(FContextCacheKey Key, const FProperty* Property, FContextCacheKey ReferenceKey, const FTimestamp& InTimestamp);

		// this is useful when there's a need to have to have cache entry but  the type is not known and there no connected output
		// ( like reroute nodes with unconnected input for example ) 
		// in that case posting an invalid reference, will allow the evaluatino to go through and the node reading it will get a default value instead
		DATAFLOWCORE_API void SetNullData(FContextCacheKey InKey, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp);

		DATAFLOWCORE_API bool CopyDataToAnotherContext(const FContextCacheKey InSourceKey, FContext& TargetContext, const FContextCacheKey InTargetKey, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp) const;

		template<class T>
		const T& GetData(FContextCacheKey Key, const FProperty* InProperty, const T& Default = T()) const
		{
			if (const TUniquePtr<FContextCacheElementBase>* Cache = GetDataImpl(Key))
			{
				return (*Cache)->GetTypedData<T>(*this, InProperty, Default);
			}
			return Default;
		}

		DATAFLOWCORE_API const void* GetUntypedData(FContextCacheKey Key, const FProperty* InProperty) const;
		DATAFLOWCORE_API bool HasData(FContextCacheKey Key, FTimestamp InTimestamp = FTimestamp::Invalid) const;
		DATAFLOWCORE_API bool IsEmpty() const;
		DATAFLOWCORE_API virtual void Serialize(FArchive& Ar);

		// Begin - IContextCacheStore
		DATAFLOWCORE_API virtual const TUniquePtr<FContextCacheElementBase>* FindCacheElement(FContextCacheKey InKey) const override;
		DATAFLOWCORE_API virtual bool HasCacheElement(FContextCacheKey InKey, FTimestamp InTimestamp = FTimestamp::Invalid) const override;
		// End - IContextCacheStore

		// Begin - IContextAssetStoreInterface
		DATAFLOWCORE_API virtual UObject* AddAsset(const FString& AssetPath, const UClass* AssetClass) override;
		DATAFLOWCORE_API virtual UObject* CommitAsset(const FString& AssetPath) override;
		DATAFLOWCORE_API virtual void ClearAssets() override;
		// End - IContextAssetStoreInterface

		DATAFLOWCORE_API FTimestamp GetTimestamp(FContextCacheKey Key) const;

		DATAFLOWCORE_API void PushToCallstack(const FDataflowConnection* Connection);
		DATAFLOWCORE_API void PopFromCallstack(const FDataflowConnection* Connection);
		DATAFLOWCORE_API bool IsInCallstack(const FDataflowConnection* Connection) const;

		DATAFLOWCORE_API FContextPerfData::FData GetPerfDataForNode(const FDataflowNode& Node) const;
		DATAFLOWCORE_API void ResetPerfDataForNode(const FDataflowNode& Node);
		DATAFLOWCORE_API void AddExternalPerfData(const FContextPerfData& InPerfData);
		DATAFLOWCORE_API void ClearAllPerfData();
		DATAFLOWCORE_API void EnablePerfData(bool bEnable);
		DATAFLOWCORE_API bool IsPerfDataEnabled() const;

		DATAFLOWCORE_API bool IsCacheEntryAfterTimestamp(FContextCacheKey InKey, const FTimestamp InTimestamp) const;

		DATAFLOWCORE_API void Info(const FText& InInfo, const FDataflowNode* InNode = nullptr, const FDataflowOutput* InOutput = nullptr);
		DATAFLOWCORE_API void Info(const FString& InInfo, const FDataflowNode* InNode = nullptr, const FDataflowOutput* InOutput = nullptr);
		
		DATAFLOWCORE_API void Warning(const FText& InWarning, const FDataflowNode* InNode = nullptr, const FDataflowOutput* InOutput = nullptr);
		DATAFLOWCORE_API void Warning(const FString& InWarning, const FDataflowNode* InNode = nullptr, const FDataflowOutput* InOutput = nullptr);
		
		DATAFLOWCORE_API void Error(const FText& InError, const FDataflowNode* InNode = nullptr, const FDataflowOutput* InOutput = nullptr);
		DATAFLOWCORE_API void Error(const FString& InError, const FDataflowNode* InNode = nullptr, const FDataflowOutput* InOutput = nullptr);

		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNodeBeginEvaluateMulticast, const FDataflowNode* Node, const FDataflowOutput* Output);
		FOnNodeBeginEvaluateMulticast OnNodeBeginEvaluateMulticast;

		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNodeFinishEvaluateMulticast, const FDataflowNode* Node, const FDataflowOutput* Output);
		FOnNodeFinishEvaluateMulticast OnNodeFinishEvaluateMulticast;

		struct FLogMessage
		{
			EMessageSeverity::Type Severity = EMessageSeverity::Info;
			TWeakPtr<const FDataflowNode> Node;
			FGuid OutputGuid;
			FText Text;
		};

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnContextLogMulticast, const FLogMessage& Message);
		FOnContextLogMulticast& GetOnContextLogMulticast() { return Delegates->OnContextLog; }

		//---------------------------------------------------------------
		// Deprecated methods section
		//---------------------------------------------------------------
		UE_DEPRECATED(5.7, "Use UE::Dataflow::FContextValue from calling GetValue on the node to access the same functionality")
		DATAFLOWCORE_API int32 GetArraySizeFromData(const FContextCacheKey InKey) const;

		UE_DEPRECATED(5.7, "Use UE::Dataflow::FContextValue from calling GetValue on the node to access the same functionality")
		DATAFLOWCORE_API void SetArrayElementFromData(const FContextCacheKey InArrayKey, int32 Index, const FContextCacheKey InElementKey, const FProperty* InProperty, const FGuid& InNodeGuid, uint32 InNodeHash, const FTimestamp& InTimestamp);

		UE_DEPRECATED(5.8, "Deprecated method, context no longer count the Info calls")
		int32 GetNumInfo() const { return 0; }

		UE_DEPRECATED(5.8, "Deprecated method, context no longer count the Warning calls")
		int32 GetNumWarnings() const { return 0; }

		UE_DEPRECATED(5.8, "Deprecated method, context no longer count the Error calls")
		int32 GetNumErrors() const { return 0; }

		UE_DEPRECATED(5.8, "Deprecated method, context does not store the warning state of a node, this is the responsability of the code using the context")
		bool NodeHasWarning(const FDataflowNode* InNode) const { return false; }

		UE_DEPRECATED(5.8, "Deprecated method, context does not store the error state of a node, this is the responsability of the code using the context")
		bool NodeHasError(const FDataflowNode* InNode) const { return false; }

		UE_DEPRECATED(5.8, "Deprecated state of a node, failure state is not longer reported")
		bool NodeFailed(const FDataflowNode* InNode) const { return false; }

		UE_DEPRECATED(5.8, "ClearNodesData is deprecated as it is no longer needed")
		void ClearNodesData() {}

		UE_DEPRECATED(5.8, "ClearNodeData is deprecated as it is no longer needed")
		void ClearNodeData(const FDataflowNode* InNode) {}

		DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnContextHasInfoMulticast, const FDataflowNode* Node, const FDataflowOutput* Output, const FString& Info);
		UE_DEPRECATED(5.8, "Use OnContextLogMulticast instead.")
		FOnContextHasInfoMulticast OnContextHasInfo;

		DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnContextHasWarningMulticast, const FDataflowNode* Node, const FDataflowOutput* Output, const FString& Info);
		UE_DEPRECATED(5.8, "Use OnContextLogMulticast instead.")
		FOnContextHasWarningMulticast OnContextHasWarning;

		DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnContextHasErrorMulticast, const FDataflowNode* Node, const FDataflowOutput* Output, const FString& Info);
		UE_DEPRECATED(5.8, "Use OnContextLogMulticast instead.")
		FOnContextHasErrorMulticast OnContextHasError;

	protected:
		DATAFLOWCORE_API void BeginContextEvaluation(const FDataflowNode* Node, const FDataflowOutput* Output);

		DATAFLOWCORE_API const FContextPerfData& GetPerfData() const;

		DATAFLOWCORE_API virtual void BroadCastLog(EMessageSeverity::Type Severity, const FText& InMessage, const FDataflowNode* InNode, const FDataflowOutput* InOutput);

	private:
		// have the delegate in a shared ref so we can keep a weakpointer on them when scheduling on the gamethread from another thread 
		struct FDelegates
		{
			FOnContextLogMulticast OnContextLog;
		};
		TSharedRef<FDelegates> Delegates;

#if DATAFLOW_EDITOR_EVALUATION
		FContextCallstack Callstack;
		FContextPerfData PerfData;
#endif
		/** used to store the dependent asset created during the evaluation of the graph */
		FContextAssetStore AssetStore;
	};

	struct FContextScopedCallstack
	{
	public:
		DATAFLOWCORE_API FContextScopedCallstack(FContext& InContext, const FDataflowConnection* InConnection);
		DATAFLOWCORE_API ~FContextScopedCallstack();

		bool IsLoopDetected() const { return bLoopDetected; }

	private:
		bool bLoopDetected;
		FContext& Context;
		const FDataflowConnection* Connection;
	};

#define DATAFLOW_CONTEXT_INTERNAL(PARENTTYPE, TYPENAME)														\
	typedef PARENTTYPE Super;																				\
	static FName StaticType() { return FName(#TYPENAME); }													\
	virtual bool IsA(FName InType) const override { return InType==StaticType() || Super::IsA(InType); }	\
	virtual FName GetType() const override { return StaticType(); }

	class FContextSingle : public FContext
	{
	public:
		DATAFLOW_CONTEXT_INTERNAL(FContext, FContextSingle);
	};
	
	class FContextThreaded : public FContext
	{
	public:
		DATAFLOW_CONTEXT_INTERNAL(FContext, FContextThreaded);

		FContextThreaded()
			: FContext()
		{
			SetThreaded(true);
		}
	};
}


