// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosLog.h"
#include "CoreMinimal.h"
#include "Dataflow/DataflowConnection.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNode.h"

#include "DataflowOverrideNode.generated.h"

struct FDataflowInput;
struct FDataflowOutput;

/**
* FDataflowOverrideNode
*		Base class for override nodes within the Dataflow graph. 
* 
*		Override Nodes allow to access to Override property on
*		the asset. They can read the values by the key.
*/
USTRUCT()
struct FDataflowOverrideNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowOverrideNode, "DataflowOverrideNode", "BaseClass", "")

public:
	FDataflowOverrideNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid())
		: Super(Param,InGuid) 
	{
		RegisterInputConnection(&Key);
		RegisterInputConnection(&Default);
		RegisterOutputConnection(&IsOverriden);
	}

	virtual ~FDataflowOverrideNode() = default;

	DATAFLOWCORE_API bool ShouldInvalidate(FName InKey) const;

	template <class T>
	T GetDefaultValue(UE::Dataflow::FContext& Context) const
	{
		return GetValue<T>(Context, &Default, Default);
	}

	template <>
	DATAFLOWCORE_API int32 GetDefaultValue(UE::Dataflow::FContext& Context) const;

	template <>
	DATAFLOWCORE_API float GetDefaultValue(UE::Dataflow::FContext& Context) const;

	DATAFLOWCORE_API FString GetValueFromAsset(UE::Dataflow::FContext& Context, const UObject* InOwner) const;

	//
	// Evaluate
	//
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const { ensure(false); };

public:
	UPROPERTY(EditAnywhere, Category = "Overrides", meta = (DataflowInput))
	FName Key = "Key";

	UPROPERTY(EditAnywhere, Category = "Overrides", meta = (DataflowInput))
	FString Default = FString("0");

	UPROPERTY(meta = (DataflowOutput))
	bool IsOverriden = false;
};


