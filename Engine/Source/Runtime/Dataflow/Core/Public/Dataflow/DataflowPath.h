// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowPath.generated.h"

USTRUCT()
struct FDataflowPath final
{
	GENERATED_USTRUCT_BODY()

	FDataflowPath() = default;

	explicit FDataflowPath(const FString& InGraph, const FString& InNode, const FString& InInput = TEXT(""), const FString& InOutput = TEXT(""))
		: Graph(InGraph)
		, Node(InNode)
		, Input(InInput)
		, Output(InOutput)
	{}

	const FString& GetGraph() const { return Graph; }
	const FString& GetNode() const { return Node; }
	const FString& GetInput() const { return Input; }
	const FString& GetOutput() const { return Output; }
	void SetGraph(const FString InGraph) { Graph = InGraph; }
	void SetNode(const FString InNode) { Node = InNode; }
	void SetInput(const FString InInput) { Input = InInput; }
	void SetOutput(const FString InOutput) { Output = InOutput; }
	DATAFLOWCORE_API FString ToString() const;
	DATAFLOWCORE_API void DecodePath(const FString InPath);
	DATAFLOWCORE_API bool PathHasInput() const;
	DATAFLOWCORE_API bool PathHasOutput() const;

private:
	FString Graph;
	FString Node;
	FString Input;
	FString Output;
};
