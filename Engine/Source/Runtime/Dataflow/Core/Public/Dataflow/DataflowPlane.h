// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "UObject/Object.h"

#include "DataflowPlane.generated.h"

/** 
* Represents a plane for dataflow
* This uses a point and a normal to represent a plane
*/
USTRUCT()
struct FDataflowPlane
{
	GENERATED_USTRUCT_BODY()

	FDataflowPlane() :
		Point(FVector::ZeroVector),
		Normal(FVector::UpVector)
	{}

	FDataflowPlane(const FVector& InPoint, const FVector& InNormal) :
		Point(InPoint),
		Normal(InNormal.GetSafeNormal())
	{}

	DATAFLOWCORE_API FDataflowPlane(const FPlane& InPlane);

	FVector GetPoint() const { return Point; }
	FVector GetNormal() const { return Normal; }

	DATAFLOWCORE_API FPlane AsPlane() const;
	DATAFLOWCORE_API FTransform AsTransform() const;

	DATAFLOWCORE_API double DistanceFromPoint(const FVector& InPoint) const;

	DATAFLOWCORE_API FDataflowPlane GetTransformed(const FTransform& InTransform) const;

private:
	UPROPERTY(EditAnywhere, Category = "Plane")
	FVector Point;

	UPROPERTY(EditAnywhere, Category = "Plane")
	FVector Normal;
};
