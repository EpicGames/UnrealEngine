// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/Facades/PointsFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "DataflowPoints.generated.h"

/** 
* Represents a pointcloud with attributes for dataflow
* Data stored in Points group
*/
USTRUCT()
struct FDataflowPoints
{
	GENERATED_USTRUCT_BODY()

public:
	DATAFLOWCORE_API const GeometryCollection::Facades::FPointsFacade GetPointsFacade() const;
	DATAFLOWCORE_API GeometryCollection::Facades::FPointsFacade GetPointsFacade();

	DATAFLOWCORE_API void Set(const FManagedArrayCollection& InCollection);
	DATAFLOWCORE_API void Set(FManagedArrayCollection& InCollection);

private:
	UPROPERTY() 
	FManagedArrayCollection Collection;
};
