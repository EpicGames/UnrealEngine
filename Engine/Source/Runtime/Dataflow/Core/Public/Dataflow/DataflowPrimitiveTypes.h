// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowAnyType.h"
#include "StructUtils/InstancedStruct.h"
#include "Misc/TVariant.h"
#include "Dataflow/DataflowPlane.h"

#include "DataflowPrimitiveTypes.generated.h"

USTRUCT()
struct FDataflowPrimitiveTypesStorage
{
	GENERATED_BODY()

	FDataflowPrimitiveTypesStorage() = default;
	FDataflowPrimitiveTypesStorage(const FDataflowPrimitiveTypesStorage& Other) = default;
	FDataflowPrimitiveTypesStorage& operator=(const FDataflowPrimitiveTypesStorage& Other) = default;
	FDataflowPrimitiveTypesStorage(FDataflowPrimitiveTypesStorage&& Other) = default;
	FDataflowPrimitiveTypesStorage& operator=(FDataflowPrimitiveTypesStorage&& Other) = default;
	FDataflowPrimitiveTypesStorage(const FBox& Primitive)
		: Storage(TInPlaceType<FBox>(), Primitive)
	{
	}
	FDataflowPrimitiveTypesStorage(const FSphere& Primitive)
		: Storage(TInPlaceType<FSphere>(), Primitive)
	{
	}
	FDataflowPrimitiveTypesStorage(const FDataflowPlane& Primitive)
		: Storage(TInPlaceType<FDataflowPlane>(), Primitive)
	{
	}

	template<typename T>
	const T* TryGet() const
	{
		return Storage.TryGet<T>();
	}
	
private:
	// Currently we don't support serialization for this storage
	TVariant<FBox, FSphere, FDataflowPlane> Storage;
};

template <>
struct FDataflowConverter<FDataflowPrimitiveTypesStorage>
{
	template <typename TFromType>
	static void From(const TFromType& From, FDataflowPrimitiveTypesStorage& To)
	{
		To = TFromType(From);
	}
	template <typename TToType>
	static void To(const FDataflowPrimitiveTypesStorage& From, TToType& To)
	{
		if (const TToType* Ptr = From.TryGet<TToType>())
		{
			To = *Ptr;
		}
	}
};

/**
* Primitive types 
*/
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowPlane)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowPrimitiveTypesStorage)

struct FDataflowPrimitiveTypePolicy :
	public TDataflowMultiTypePolicy<FBox, FSphere, FDataflowPlane>
{
};

USTRUCT()
struct FDataflowPrimitiveTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowPrimitiveTypePolicy;
	using FStorageType = FDataflowPrimitiveTypesStorage;

	GENERATED_BODY()

	// This needs to be a UPROPERTY() because the anytype system needs to inspect it
	UPROPERTY()
	FDataflowPrimitiveTypesStorage Value;
};

// ------------------------------------------------------------------------------------------------------------------

USTRUCT()
struct FDataflowPrimitiveArrayStorage
{
	GENERATED_BODY()

	FDataflowPrimitiveArrayStorage() = default;
	FDataflowPrimitiveArrayStorage(const FDataflowPrimitiveArrayStorage& Other) = default;
	FDataflowPrimitiveArrayStorage& operator=(const FDataflowPrimitiveArrayStorage& Other) = default;
	FDataflowPrimitiveArrayStorage(FDataflowPrimitiveArrayStorage&& Other) = default;
	FDataflowPrimitiveArrayStorage& operator=(FDataflowPrimitiveArrayStorage&& Other) = default;
	FDataflowPrimitiveArrayStorage(const TArray<FBox>& PrimitiveArray)
		: Storage(TInPlaceType<TArray<FBox>>(), PrimitiveArray)
	{
	}
	FDataflowPrimitiveArrayStorage(const TArray<FSphere>& PrimitiveArray)
		: Storage(TInPlaceType<TArray<FSphere>>(), PrimitiveArray)
	{
	}
	FDataflowPrimitiveArrayStorage(const TArray<FDataflowPlane>& PrimitiveArray)
		: Storage(TInPlaceType<TArray<FDataflowPlane>>(), PrimitiveArray)
	{
	}

	template<typename T>
	const T* TryGet() const
	{
		return Storage.TryGet<T>();
	}

private:
	// Currently we don't support serialization for this storage
	TVariant<TArray<FBox>, TArray<FSphere>, TArray<FDataflowPlane>> Storage;
};

template <>
struct FDataflowConverter<FDataflowPrimitiveArrayStorage>
{
	template <typename TFromType>
	static void From(const TFromType& From, FDataflowPrimitiveArrayStorage& To)
	{
		To = TFromType(From);
	}
	template <typename TToType>
	static void To(const FDataflowPrimitiveArrayStorage& From, TToType& To)
	{
		if (const TToType* Ptr = From.TryGet<TToType>())
		{
			To = *Ptr;
		}
	}
};

UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowPrimitiveArrayStorage)

struct FDataflowPrimitiveArrayPolicy :
	public TDataflowMultiTypePolicy<TArray<FBox>, TArray<FSphere>, TArray<FDataflowPlane>>
{
};

USTRUCT()
struct FDataflowPrimitiveArrayTypes : public FDataflowAnyType
{
	using FPolicyType = FDataflowPrimitiveArrayPolicy;
	using FStorageType = FDataflowPrimitiveArrayStorage;

	GENERATED_BODY()

	// This needs to be a UPROPERTY() because the anytype system needs to inspect it
	UPROPERTY()
	FDataflowPrimitiveArrayStorage Value;
};

namespace UE::Dataflow
{
	DATAFLOWCORE_API void RegisterPrimitiveTypes();
};
