// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

#include "DataflowSelection.generated.h"

struct FManagedArrayCollection;
struct FDataflowConnection;

USTRUCT()
struct FDataflowSelection
{
	GENERATED_USTRUCT_BODY()

	FDataflowSelection()
	{}

	FDataflowSelection(FName InGroupName)
		: GroupName(InGroupName)
	{}

	DATAFLOWCORE_API void Initialize(int32 NumBits, bool Value);
	DATAFLOWCORE_API void Initialize(const FDataflowSelection& Other);
	DATAFLOWCORE_API void Initialize(const TBitArray<>& Other);
	DATAFLOWCORE_API void InitializeFromCollection(const FManagedArrayCollection& InCollection, bool Value);
	DATAFLOWCORE_API void Clear();
	DATAFLOWCORE_API int32 Num() const;
	DATAFLOWCORE_API int32 NumSelected() const;
	DATAFLOWCORE_API bool AnySelected() const;

	DATAFLOWCORE_API bool IsValid() const;

	inline bool IsValidIndex(int32 Idx) const
	{
		return SelectionArray.IsValidIndex(Idx);
	}
	bool IsSelected(int32 Idx) const { return SelectionArray[Idx]; }
	void SetSelected(int32 Idx) { SelectionArray[Idx] = true; }
	DATAFLOWCORE_API void SetSelected(const TArray<int32>& Indices);
	DATAFLOWCORE_API bool SetSelectedWithCheck(const TArray<int32>& Indices);
	void SetNotSelected(int32 Idx) { SelectionArray[Idx] = false; }
	DATAFLOWCORE_API void AsArray(TArray<int32>& SelectionArr) const;
	DATAFLOWCORE_API TArray<int32> AsArray() const;
	DATAFLOWCORE_API void AsArrayValidated(TArray<int32>& SelectionArr, const FManagedArrayCollection& InCollection) const;
	DATAFLOWCORE_API TArray<int32> AsArrayValidated(const FManagedArrayCollection& InCollection) const;
	
	/**
	 * Sets the selection from a sparse array (it only contains the indices of the selected items)
	 * @return true if all indices were successfully set, false if some indices were invalid and could not be set
	 */
	DATAFLOWCORE_API bool SetFromArray(const TArray<int32>& SelectionArr);
	DATAFLOWCORE_API void SetFromBitArray(TBitArray<>&& Other);

	/**
	* Sets the selection from a dense array (it contains a true/false element for every item)
	* (for example from the "Internal" attr from FacesGroup)
	*/
	DATAFLOWCORE_API void SetFromArray(const TArray<bool>& SelectionArr);
	DATAFLOWCORE_API void AND(const FDataflowSelection& Other, FDataflowSelection& Result) const;
	DATAFLOWCORE_API void OR(const FDataflowSelection& Other, FDataflowSelection& Result) const;
	DATAFLOWCORE_API void XOR(const FDataflowSelection& Other, FDataflowSelection& Result) const;
	// subtract the selected elements of 'Other' from this selection
	DATAFLOWCORE_API void Subtract(const FDataflowSelection& Other, FDataflowSelection& Result) const;
	void Invert() { SelectionArray.BitwiseNOT(); }
	DATAFLOWCORE_API void SetWithMask(const bool Value, const FDataflowSelection& Mask);
	const TBitArray<>& GetBitArray() const { return SelectionArray; };

	/**
	 * Initialize from a Collection and an array
	 * @return true if all indices were successfully set, false if some indices were invalid and could not be set
	 */
	DATAFLOWCORE_API bool InitFromArray(const FManagedArrayCollection& InCollection, const TArray<int32>& InSelectionArr);

	// @return true if this selection is valid for the given collection -- i.e., if the selection expects the group to have the correct number of elements
	DATAFLOWCORE_API bool IsValidForCollection(const FManagedArrayCollection& InCollection) const;

	/**
	* Print selection in 
	* "Selected Transforms: 23 of 34" format;
	*/
	DATAFLOWCORE_API FString ToString();

	FName GetGroupName() const { return GroupName; }

	/**
	* Returns group name from input or output connection
	*/
	DATAFLOWCORE_API static FName GetSelectionTypeFromCollection(const FDataflowConnection& InConnection);

private:
	FName GroupName;
	TBitArray<> SelectionArray;
};

USTRUCT()
struct FDataflowTransformSelection : public FDataflowSelection
{
	GENERATED_USTRUCT_BODY()

	inline static const FName TransformGroupName = "Transform";

	FDataflowTransformSelection()
		: FDataflowSelection(TransformGroupName)
	{}
};


USTRUCT()
struct FDataflowVertexSelection : public FDataflowSelection
{
	GENERATED_USTRUCT_BODY()

	inline static const FName VerticesGroupName = "Vertices";

	FDataflowVertexSelection()
		: FDataflowSelection(VerticesGroupName)
	{}
};


USTRUCT()
struct FDataflowFaceSelection : public FDataflowSelection
{
	GENERATED_USTRUCT_BODY()

	inline static const FName FacesGroupName = "Faces";

	FDataflowFaceSelection()
		: FDataflowSelection(FacesGroupName)
	{}
};

USTRUCT()
struct FDataflowGeometrySelection : public FDataflowSelection
{
	GENERATED_USTRUCT_BODY()

	inline static const FName GeometryGroupName = "Geometry";

	FDataflowGeometrySelection()
		: FDataflowSelection(GeometryGroupName)
	{}
};

USTRUCT()
struct FDataflowMaterialSelection : public FDataflowSelection
{
	GENERATED_USTRUCT_BODY()

	inline static const FName MaterialGroupName = "Material";

	FDataflowMaterialSelection()
		: FDataflowSelection(MaterialGroupName)
	{}
};

USTRUCT()
struct FDataflowCurveSelection : public FDataflowSelection
{
	GENERATED_USTRUCT_BODY()

	inline static const FName CurveGroupName = "Curves";

	FDataflowCurveSelection()
		: FDataflowSelection(CurveGroupName)
	{}
};

USTRUCT()
struct FDataflowPointsSelection : public FDataflowSelection
{
	GENERATED_USTRUCT_BODY()

	inline static const FName PointsGroupName = "Points";

	FDataflowPointsSelection()
		: FDataflowSelection(PointsGroupName)
	{
	}
};

UENUM(BlueprintType)
enum class EDataflowSelectionType : uint8
{
	Transform UMETA(DisplayName = "Transform"),
	Vertices UMETA(DisplayName = "Vertices"),
	Faces UMETA(DisplayName = "Faces"),
	Geometry UMETA(DisplayName = "Geometry"),
	Curves UMETA(DisplayName = "Curves"),
	Points UMETA(DisplayName = "Points"),
	Material UMETA(DisplayName = "Material"),
};

