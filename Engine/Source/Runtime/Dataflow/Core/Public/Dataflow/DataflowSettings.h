// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "InputCoreTypes.h"

#include "DataflowSettings.generated.h"

typedef TMap<FName, FNodeColors> FNodeColorsMap;
typedef TMap<FName, FPinSettings> FPinSettingsMap;

USTRUCT()
struct FNodeColors
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Colors)
	FLinearColor NodeTitleColor = FLinearColor(0.f, 0.f, 0.f);

	UPROPERTY(EditAnywhere, Category = Colors)
	FLinearColor NodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f);
};

USTRUCT()
struct FPinSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FLinearColor PinColor = FLinearColor(0.f, 0.f, 0.f);

	UPROPERTY(EditAnywhere, Category = Settings)
	float WireThickness = 1.f;
};

USTRUCT()
struct FTransformLevelColors
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Colors)
	TArray<FLinearColor> LevelColors;

	UPROPERTY(EditAnywhere, Category = Colors)
	FLinearColor BlankColor = FLinearColor(255, 255, 255, 255);
};

UCLASS(config = Editor, defaultconfig, meta = (DisplayName = "Dataflow"), MinimalAPI)
class UDataflowSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(config, EditAnywhere, Category = NodeColors)
	TMap<FName, FNodeColors> NodeColorsMap;

	UPROPERTY(config, EditAnywhere, Category = PinSettings)
	TMap<FName, FPinSettings> PinSettingsMap;

	UPROPERTY(config, EditAnywhere, Category = TransformLevelColors)
	FTransformLevelColors TransformLevelColors;

	// Begin UDeveloperSettings Interface
	DATAFLOWCORE_API virtual FName GetCategoryName() const override;

	DATAFLOWCORE_API FNodeColors RegisterColors(const FName& Category, const FNodeColors& Colors);
	DATAFLOWCORE_API FPinSettings RegisterPinSettings(const FName& PinType, const FPinSettings& InSettings);

	const TMap<FName, FNodeColors>& GetNodeColorsMap() { return NodeColorsMap; }
	const TMap<FName, FPinSettings>& GetPinSettingsMap() { return PinSettingsMap; }

#if WITH_EDITOR
	DATAFLOWCORE_API virtual FText GetSectionText() const override;
#endif
	// END UDeveloperSettings Interface

#if WITH_EDITOR
	DATAFLOWCORE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDataflowSettingsChanged, const FNodeColorsMap&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDataflowSettingsChangedPinSettings, const FPinSettingsMap&);

	/** Gets a multicast delegate which is called whenever one of the parameters in this settings object changes. */
	FOnDataflowSettingsChanged& GetOnDataflowSettingsChangedDelegate() { return OnDataflowSettingsChangedDelegate; }
	FOnDataflowSettingsChangedPinSettings& GetOnDataflowSettingsChangedPinSettingsDelegate() { return OnDataflowSettingsChangedPinSettingsDelegate; }

protected:
	FOnDataflowSettingsChanged OnDataflowSettingsChangedDelegate;
	FOnDataflowSettingsChangedPinSettings OnDataflowSettingsChangedPinSettingsDelegate;

};

