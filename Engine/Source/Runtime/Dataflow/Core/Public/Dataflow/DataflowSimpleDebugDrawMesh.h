// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowDebugDrawInterface.h"

struct FSimpleDebugDrawMesh : public IDataflowDebugDrawInterface::IDebugDrawMesh
{
	TArray<FVector> Vertices;
	TArray<FIntVector3> Triangles;
	TArray<FVector> VertexNormals;

	virtual ~FSimpleDebugDrawMesh() = default;
	DATAFLOWCORE_API virtual int32 GetMaxVertexIndex() const override;
	DATAFLOWCORE_API virtual bool IsValidVertex(int32 VertexIndex) const override;
	DATAFLOWCORE_API virtual FVector GetVertexPosition(int32 VertexIndex) const override;
	DATAFLOWCORE_API virtual FVector GetVertexNormal(int32 VertexIndex) const override;
	DATAFLOWCORE_API void SetVertex(const int32 VertexIndex, const FVector& VertexPosition);

	DATAFLOWCORE_API virtual int32 GetMaxTriangleIndex() const override;
	DATAFLOWCORE_API virtual bool IsValidTriangle(int32 VertexIndex) const override;
	DATAFLOWCORE_API virtual FIntVector3 GetTriangle(int32 VertexIndex) const override;
	DATAFLOWCORE_API void SetTriangle(const int32 TriangleIndex, const int32 VertexIndexA, const int32 VertexIndexB, const int32 VertexIndexC);

	DATAFLOWCORE_API void MakeRectangleMesh(const FVector& Origin, const float Width, const float Height, const int32 WidthVertexCount, const int32 HeightVertexCount);
	DATAFLOWCORE_API void TransformVertices(const FTransform& Transform);
};
