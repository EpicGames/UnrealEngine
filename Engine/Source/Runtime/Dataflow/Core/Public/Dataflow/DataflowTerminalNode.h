// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosLog.h"
#include "CoreMinimal.h"
#include "Dataflow/DataflowConnection.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNode.h"

#include "DataflowTerminalNode.generated.h"

#define UE_API DATAFLOWCORE_API

struct FDataflowInput;
struct FDataflowOutput;

class UObject;
class UClass;

/**
* FDataflowTerminalNode
*		Base class for terminal nodes within the Dataflow graph. 
* 
*		Terminal Nodes allow for non-const access to UObjects as
*       edges in the graph. They are used to push data out to
*       asset or the world from the calling client. Terminals
*       may not have outputs, they are only leaf nodes in the 
*       evaluation graph. 
*/
USTRUCT()
struct FDataflowTerminalNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	FDataflowTerminalNode()
		: Super() { }

	FDataflowTerminalNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid())
		: Super(Param,InGuid) {
	}

	virtual ~FDataflowTerminalNode() = default;

	static FName StaticType() { return FName("FDataflowTerminalNode"); }

	virtual bool IsA(FName InType) const override 
	{ 
		return InType.ToString().Equals(StaticType().ToString()) 
			|| Super::IsA(InType); 
	}

	virtual bool IsTerminal() const override
	{
		return true;
	}

	/** Return the terminal asset */
	virtual TObjectPtr<UObject> GetTerminalAsset() const {return nullptr;}

	//
	// Error Checking
	//
	/**
	 * Write the terminal node's evaluated value into the bound asset.
	 * Note: Asset may be null when the graph is evaluated standalone (no owning asset).
	 * Implementations must tolerate a null Asset and simply do nothing in that case.
	 */
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const { ensure(false); }

	//
	// Evaluate
	//

	virtual void Evaluate(UE::Dataflow::FContext& Context) const { ensure(false); }

protected:
	DATAFLOWCORE_API static FString GetAssetPath(const FString& InputPath, UObject* BoundAsset);

	DATAFLOWCORE_API UObject* GetOrCreateAsset(UE::Dataflow::FContext& Context, const FString& InAssetPath, const UClass* AssetClass) const;

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override
	{
		Evaluate(Context);
	};
};

#undef UE_API 
