// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/TVariant.h"
#include "UObject/Object.h"
#include "Math/MathFwd.h"
#include "Dataflow/DataflowSelection.h"
#include <string>

class UObject;

template<typename TType>
struct TDataflowPolicyTypeName
{
};

namespace UE::Dataflow
{
	template<typename T>
	FName GetTypeName() { return TDataflowPolicyTypeName<T>::GetName(); }

	template<typename T>
	FName GetTypeName(bool bAsArray)
	{ 
		if (bAsArray)
		{
			return TDataflowPolicyTypeName<TArray<T>>::GetName();
		}
		return TDataflowPolicyTypeName<T>::GetName();
	}

}


// special version for void 
template<>
struct TDataflowPolicyTypeName<void>
{
	static const TCHAR* GetName()
	{
		return TEXT("");
	}
};

#define UE_DATAFLOW_MAKE_STRING(x) #x
#define UE_DATAFLOW_POLICY_DECLARE_TYPENAME(TType) \
template<> \
struct TDataflowPolicyTypeName<TType> \
{  \
	static const TCHAR* GetName() \
	{ \
		return TEXT(#TType); \
	} \
}; \
template<> \
struct TDataflowPolicyTypeName<TArray<TType>> \
{  \
	static const TCHAR* GetName() \
	{ \
		return TEXT(UE_DATAFLOW_MAKE_STRING(TArray<TType>)); \
	} \
}; 

UE_DATAFLOW_POLICY_DECLARE_TYPENAME(bool)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(uint8)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(uint16)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(uint32)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(uint64)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(int8)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(int16)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(int32)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(int64)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(float)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(double)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FName)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FText)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FString)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(TObjectPtr<UObject>)

UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FVector2D)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FVector)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FVector4)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FVector2f)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FVector3f)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FVector4f)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FQuat)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FQuat4f)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FLinearColor)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FIntPoint)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FIntVector3)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FIntVector4)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FRotator)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FTransform)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FBox)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FSphere)

UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowSelection)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowTransformSelection)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowVertexSelection)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowFaceSelection)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowGeometrySelection)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowMaterialSelection)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowCurveSelection)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FDataflowPointsSelection)

struct IDataflowTypePolicy
{
	// Check if a specific type if supported 
	virtual bool SupportsType(FName Type) const = 0;

	// Add the supported list of types if there's a short list of them 
	// for full anytype, no types will be added to the array
	virtual void GetSupportedTypes(TArray<FName>& OutSupportedTypes) const {};
};

struct FDataflowAllTypesPolicy : public IDataflowTypePolicy
{
	virtual bool SupportsType(FName InType) const override
	{
		return true;
	}

	static bool SupportsTypeStatic(FName InType)
	{
		return true;
	}

	static IDataflowTypePolicy* GetInterface()
	{
		static FDataflowAllTypesPolicy Instance;
		return &Instance;
	}
};

template <typename T>
struct TDataflowSingleTypePolicy : public IDataflowTypePolicy
{
	using FType = T;

	virtual bool SupportsType(FName InType) const override
	{
		return SupportsTypeStatic(InType);
	}

	static bool SupportsTypeStatic(FName InType)
	{
		return (InType == TypeName);
	}

	virtual void GetSupportedTypes(TArray<FName>& OutSupportedTypes) const override
	{
		GetSupportedTypesStatic(OutSupportedTypes);
	}

	static void GetSupportedTypesStatic(TArray<FName>& OutSupportedTypes)
	{
		OutSupportedTypes.Add(TypeName);
	}

	template <typename TVisitor>
	static bool VisitPolicyByType(FName RequestedType, TVisitor Visitor)
	{
		if (RequestedType == TypeName)
		{
			TDataflowSingleTypePolicy<T> SingleTypePolicy;
			Visitor(SingleTypePolicy);
			return true;
		}
		return false;
	}

	static IDataflowTypePolicy* GetInterface()
	{
		static TDataflowSingleTypePolicy Instance;
		return &Instance;
	}

	inline static const FName TypeName = FName(TDataflowPolicyTypeName<T>::GetName());
};

template <typename... TTypes>
struct TDataflowMultiTypePolicy;

template <>
struct TDataflowMultiTypePolicy<>: public IDataflowTypePolicy
{
	virtual bool SupportsType(FName InType) const override
	{
		return false;
	}

	static bool SupportsTypeStatic(FName InType)
	{
		return false;
	}

	template <typename TVisitor>
	static bool VisitPolicyByType(FName RequestedType, TVisitor Visitor)
	{
		return false;
	}
};

template <typename T, typename... TTypes>
struct TDataflowMultiTypePolicy<T, TTypes...>: public TDataflowMultiTypePolicy<TTypes...>
{
	using Super = TDataflowMultiTypePolicy<TTypes...>;

	virtual bool SupportsType(FName InType) const override
	{
		return SupportsTypeStatic(InType);
	}

	static bool SupportsTypeStatic(FName InType)
	{
		return TDataflowSingleTypePolicy<T>::SupportsTypeStatic(InType)
			|| Super::SupportsTypeStatic(InType);
	}

	virtual void GetSupportedTypes(TArray<FName>& OutSupportedTypes) const override
	{
		Super::GetSupportedTypes(OutSupportedTypes);
		TDataflowSingleTypePolicy<T>::GetSupportedTypesStatic(OutSupportedTypes);
	}

	template <typename TVisitor>
	static bool VisitPolicyByType(FName RequestedType, TVisitor Visitor)
	{
		if (TDataflowSingleTypePolicy<T>::VisitPolicyByType(RequestedType, Visitor))
		{
			return true;
		}
		return Super::VisitPolicyByType(RequestedType, Visitor);
	}

	static IDataflowTypePolicy* GetInterface()
	{
		static TDataflowMultiTypePolicy<T, TTypes...> Instance;
		return &Instance;
	}
};

struct FDataflowArrayTypePolicy: public IDataflowTypePolicy
{
	static constexpr TCHAR ArrayPrefix[] = TEXT("TArray<");
	static constexpr TCHAR ArrayFormat[] = TEXT("TArray<{0}>");

	virtual bool SupportsType(FName InType) const override
	{
		return FDataflowArrayTypePolicy::SupportsTypeStatic(InType);
	}

	static bool SupportsTypeStatic(FName InType)
	{
		return (InType.ToString().StartsWith(ArrayPrefix));
	}

	static IDataflowTypePolicy* GetInterface()
	{
		static FDataflowArrayTypePolicy Instance;
		return &Instance;
	}

	static FName GetElementType(FName InType)
	{
		if (SupportsTypeStatic(InType))
		{
			const int32 ArrayPrefixLen = FCString::Strlen(ArrayPrefix);
			const FString RightStr = InType.ToString().RightChop(ArrayPrefixLen);
			check(!RightStr.IsEmpty() && RightStr[RightStr.Len() - 1] == '>');
			return FName(RightStr.LeftChop(1)); // remove the last ">"
		}
		// not really an array just return the original type
		return InType;
	}

	static FName GetArrayType(FName InType)
	{
		return FName(FString::Format(ArrayFormat, { InType.ToString() }));
	}
};

struct FDataflowUStructTypePolicy : IDataflowTypePolicy
{
	virtual bool SupportsType(FName InType) const override
	{
		return SupportsTypeStatic(InType);
	}

	static bool SupportsTypeStatic(FName InType)
	{
		FString InnerTypeStr;
		if (GetStructType(InType.ToString(), InnerTypeStr))
		{
			if (StaticFindFirstObject(UStruct::StaticClass(), *InnerTypeStr, EFindFirstObjectOptions::NativeFirst))
			{
				return true;
			}
		}
		
		// not a proper Ustruct
		return false;
	}
	
	static IDataflowTypePolicy* GetInterface()
	{
		static FDataflowUStructTypePolicy Instance;
		return &Instance;
	}

	/**
	 * Provided an InTypeString outputs the same type without the type decorations (like prefixes).
	 * @param InTypeStr The object type
	 * @param OutInnerType The object type without the struct prefix ("F")
	 * @return true if the prefix was found, false otherwise
	 */
	static bool GetStructType(const FString& InTypeStr, FString& OutInnerType)
	{
		static constexpr const TCHAR* ObjectPtrPrefix = TEXT("F");
		static const int32 ObjectPtrPrefixLen = FCString::Strlen(ObjectPtrPrefix);;
		if (InTypeStr.StartsWith(ObjectPtrPrefix))
		{
			OutInnerType = InTypeStr
				.RightChop(ObjectPtrPrefixLen) // remove the type prefix
				.TrimStartAndEnd();
			return true;
		}
		return false;
	}
};

template <typename BaseStructType>
struct FDataflowUDerivedStructTypePolicy : IDataflowTypePolicy
{
	virtual bool SupportsType(FName InType) const override
	{
		return SupportsTypeStatic(InType);
	}

	static bool SupportsTypeStatic(FName InType)
	{
		FString InnerTypeStr;
		if (FDataflowUStructTypePolicy::GetStructType(InType.ToString(), InnerTypeStr))
		{
			if (UStruct* StructObject = Cast< UStruct>(StaticFindFirstObject(UStruct::StaticClass(), *InnerTypeStr, EFindFirstObjectOptions::NativeFirst)))
			{
				return StructObject->IsChildOf<BaseStructType>();
			}
		}

		// not a proper Ustruct
		return false;
	}

	static IDataflowTypePolicy* GetInterface()
	{
		static FDataflowUDerivedStructTypePolicy< BaseStructType> Instance;
		return &Instance;
	}
};

struct FDataflowNumericTypePolicy : 
	public TDataflowMultiTypePolicy<double, float, int64, uint64, int32, uint32, int16, uint16, int8, uint8>
{
};

struct FDataflowNumericArrayPolicy: 
	public TDataflowMultiTypePolicy<TArray<double>, TArray<float>, TArray<int64>,
		TArray<uint64>, TArray<int32>, TArray<uint32>, TArray<int16>, TArray<uint16>, TArray<int8>, TArray<uint8>>
{
};

struct FDataflowVectorTypePolicy:
	public TDataflowMultiTypePolicy<FVector2D, FVector, FVector4, FVector2f, FVector3f, FVector4f, FQuat, FQuat4f, FLinearColor, FIntPoint, FIntVector, FIntVector4, FRotator>
{
};

struct FDataflowVectorArrayPolicy:
	public TDataflowMultiTypePolicy<TArray<FVector2D>, TArray<FVector>, TArray<FVector4>,
		TArray<FVector2f>, TArray<FVector3f>, TArray<FVector4f>, TArray<FQuat>, TArray<FQuat4f>,
		TArray<FLinearColor>, TArray<FIntPoint>, TArray<FIntVector>, TArray<FIntVector4>, TArray<FRotator>>
{
};

struct FDataflowStringTypePolicy :
	public TDataflowMultiTypePolicy<FString, FName, FText>
{
};

struct FDataflowSelectionTypePolicy :
	public TDataflowMultiTypePolicy<FDataflowTransformSelection, FDataflowVertexSelection, FDataflowFaceSelection, FDataflowGeometrySelection, FDataflowMaterialSelection, FDataflowCurveSelection>
{
};

struct FDataflowStringArrayPolicy :
	public TDataflowMultiTypePolicy<TArray<FString>, TArray<FName>>
{
};

struct FDataflowRotationTypePolicy :
	public TDataflowMultiTypePolicy<FVector, FQuat, FRotator>
{
};

struct FDataflowMeshSelectionTypePolicy :
	public TDataflowMultiTypePolicy<FDataflowVertexSelection, FDataflowFaceSelection>
{
};

/**
* string comvertible types
* - FString / Fname
* - Numeric types ( see FDataflowNumericTypePolicy )
* - Vector types ( see FDataflowVectorTypePolicy )
* - bool
*/
struct FDataflowStringConvertibleTypePolicy : IDataflowTypePolicy
{
	virtual bool SupportsType(FName InType) const override
	{
		return SupportsTypeStatic(InType);
	}

	static bool SupportsTypeStatic(FName InType)
	{
		return FDataflowStringTypePolicy::SupportsTypeStatic(InType)
			|| FDataflowNumericTypePolicy::SupportsTypeStatic(InType)
			|| FDataflowVectorTypePolicy::SupportsTypeStatic(InType)
			|| TDataflowSingleTypePolicy<bool>::SupportsTypeStatic(InType)
			|| TDataflowSingleTypePolicy<FTransform>::SupportsTypeStatic(InType)
			;
	}

	template <typename TVisitor>
	static bool VisitPolicyByType(FName RequestedType, TVisitor Visitor)
	{
		return FDataflowStringTypePolicy::VisitPolicyByType(RequestedType, Visitor)
			|| FDataflowNumericTypePolicy::VisitPolicyByType(RequestedType, Visitor)
			|| FDataflowVectorTypePolicy::VisitPolicyByType(RequestedType, Visitor)
			|| TDataflowSingleTypePolicy<bool>::VisitPolicyByType(RequestedType, Visitor)
			|| TDataflowSingleTypePolicy<FTransform>::VisitPolicyByType(RequestedType, Visitor)
			;
	}

	static IDataflowTypePolicy* GetInterface()
	{
		static FDataflowStringConvertibleTypePolicy Instance;
		return &Instance;
	}
};

struct FDataflowUObjectConvertibleTypePolicy : IDataflowTypePolicy
{
	virtual bool SupportsType(FName InType) const override
	{
		return SupportsTypeStatic(InType);
	}

	static bool SupportsTypeStatic(FName InType)
	{
		FString InnerTypeStr;
		if (GetObjectPtrInnerType(InType.ToString(), InnerTypeStr))
		{
			if (StaticFindFirstObject(UObject::StaticClass(), *InnerTypeStr, EFindFirstObjectOptions::NativeFirst))
			{
				return true;
			}
		}
		// not a proper object pointer 
		return false;
	}

	template <typename TVisitor>
	static bool VisitPolicyByType(FName RequestedType, TVisitor Visitor)
	{
		if (SupportsTypeStatic(RequestedType))
		{
			TDataflowSingleTypePolicy<TObjectPtr<UObject>> SingleTypePolicy;
			Visitor(SingleTypePolicy);
			return true;
		}
		return false;
	}

	static IDataflowTypePolicy* GetInterface()
	{
		static FDataflowUObjectConvertibleTypePolicy Instance;
		return &Instance;
	}

	// returns true if the type was a TObjectPtr and the inner type was properly extracted
	static bool GetObjectPtrInnerType(const FString& InTypeStr, FString& InnerType)
	{
		static constexpr const TCHAR* ObjectPtrPrefix = TEXT("TObjectPtr<U");
		static constexpr size_t ObjectPtrPrefixLen = std::char_traits<TCHAR>::length(ObjectPtrPrefix);
		if (InTypeStr.StartsWith(ObjectPtrPrefix))
		{
			InnerType = InTypeStr
				.RightChop(ObjectPtrPrefixLen) // remove the TObjectPtr< type
				.LeftChop(1) // remove the last ">"
				.TrimStartAndEnd();
				return true;
		}
		return false;
	}
};

// type Converters

template <typename T>
struct FDataflowConverter
{
	template <typename TFromType>
	static void From(const TFromType& From, T& To) { To = From; }

	template <typename TToType>
	static void To(const T& From, TToType& To) { To = From; }
};

template<typename T>
concept HasToStringMethod =
	requires(T t) {
		static_cast<FString>(t.ToString());
};


template<typename T>
concept HasInitFromStringMethod =
	requires(T t, const FString& s ) {
		static_cast<bool>(t.InitFromString(s));
};

template <>
struct FDataflowConverter<FString>
{
	template <typename TFromType>
	static void From(const TFromType& From, FString& To)
	{
		if constexpr (std::is_same_v<TFromType, FName>)
		{
			To = From.ToString();
		}
		else if constexpr (std::is_same_v<TFromType, FText>)
		{
			To = From.ToString();
		}
		else if constexpr (std::is_same_v<TFromType, bool>)
		{
			To = FString((From == true) ? "True" : "False");
		}
		else if constexpr (std::is_convertible_v<TFromType, double>)
		{
			To = FString::SanitizeFloat(double(From), 0);
		}
		else if constexpr (HasToStringMethod<TFromType>)
		{
			To = From.ToString();
		}
		else
		{
			To = From;
		}
	}

	template <typename TToType>
	static void To(const FString& From, TToType& To)
	{
		if constexpr (std::is_same_v<TToType, FName>)
		{
			To = FName(From);
		}
		else if constexpr (std::is_same_v<TToType, FText>)
		{
			To = FText::FromString(From);
		}
		else if constexpr (std::is_same_v<TToType, bool>)
		{
			To = From.ToBool();
		}
		else if constexpr (std::is_convertible_v<double, TToType>)
		{
			double Result = {0};
			LexTryParseString(Result, *From);
			To = Result;
		}
		else if constexpr (HasInitFromStringMethod<TToType>)
		{
			To.InitFromString(From);
		}
		else
		{
			To = From;
		}
	}
};

template <>
struct FDataflowConverter<FVector4>
{
	template <typename TFromType>
	static void From(const TFromType& From, FVector4& To)
	{
		if constexpr (std::is_same_v<TFromType, FVector2D> || std::is_same_v<TFromType, FVector2f>)
		{
			To = FVector4{ (double)From.X, (double)From.Y, 0, 0};
		}
		else if constexpr (std::is_same_v<TFromType, FVector> || std::is_same_v<TFromType, FVector3f>)
		{
			To = FVector4{ (double)From.X, (double)From.Y, (double)From.Z, 0 };
		}
		else if constexpr (std::is_same_v<TFromType, FVector4f>)
		{
			To = FVector4{ (double)From.X, (double)From.Y, (double)From.Z, (double)From.W };
		}
		else if constexpr (std::is_same_v<TFromType, FQuat4f> || std::is_same_v<TFromType, FQuat>)
		{
			To = FVector4{ (double)From.X, (double)From.Y, (double)From.Z, (double)From.W };
		}
		else if constexpr (std::is_same_v<TFromType, FLinearColor>)
		{
			To = FVector4 {(double)From.R, (double)From.G, (double)From.B, (double)From.A };
		}
		else if constexpr (std::is_same_v<TFromType, FIntVector3>)
		{
			To = FVector4 {(double)From.X, (double)From.Y, (double)From.Z, 0};
		}
		else if constexpr (std::is_same_v<TFromType, FIntVector4>)
		{
			To = FVector4 {(double)From.X, (double)From.Y, (double)From.Z, (double)From.W};
		}
		else if constexpr (std::is_same_v<TFromType, FIntPoint>)
		{
			To = FVector4 {(double)From.X, (double)From.Y, 0, 0};
		}
		else if constexpr (std::is_same_v<TFromType, FRotator>)
		{
			To = FVector4 {(double)From.Pitch, (double)From.Yaw, (double)From.Roll, 0};
		}
		else
		{
			To = From;
		}
	}

	template <typename TToType>
	static void To(const FVector4& From, TToType& To)
	{
		if constexpr (std::is_same_v<TToType, FVector2D>)
		{
			To = FVector2D{ From.X, From.Y };
		}
		else if constexpr (std::is_same_v<TToType, FVector2f>)
		{
			To = FVector2f{ (float)From.X, (float)From.Y };
		}
		else if constexpr (std::is_same_v<TToType, FVector>)
		{
			To = FVector{ From.X, From.Y, From.Z };
		}
		else if constexpr (std::is_same_v<TToType, FVector3f>)
		{
			To = FVector3f{ (float)From.X, (float)From.Y, (float)From.Z };
		}
		else if constexpr (std::is_same_v<TToType, FVector4f>)
		{
			To = FVector4f{ (float)From.X, (float)From.Y, (float)From.Z, (float)From.W };
		}
		else if constexpr (std::is_same_v<TToType, FQuat>)
		{
			To = FQuat{ From.X, From.Y, From.Z, From.W };
		}
		else if constexpr (std::is_same_v<TToType, FQuat4f>)
		{
			To = FQuat4f{ (float)From.X, (float)From.Y, (float)From.Z, (float)From.W };
		}
		else if constexpr (std::is_same_v<TToType, FLinearColor>)
		{
			To = FLinearColor{ (float)From.X, (float)From.Y, (float)From.Z, (float)From.W };
		}
		else if constexpr (std::is_same_v<TToType, FIntPoint>)
		{
			To = FIntPoint{ (int32)From.X, (int32)From.Y };
		}
		else if constexpr (std::is_same_v<TToType, FIntVector3>)
		{
			To = FIntVector3{ (int32)From.X, (int32)From.Y, (int32)From.Z };
		}
		else if constexpr (std::is_same_v<TToType, FIntVector4>)
		{
			To = FIntVector4{ (int32)From.X, (int32)From.Y, (int32)From.Z, (int32)From.W };
		}
		else if constexpr (std::is_same_v<TToType, FRotator>)
		{
			To = FRotator{ (double)From.X, (double)From.Y, (double)From.Z};
		}
		else
		{
			To = From;
		}
	}
};

template <>
struct FDataflowConverter<FDataflowSelection>
{
	template <typename TFromType>
	static void From(const TFromType& From, FDataflowSelection& To)
	{
		To.Initialize(From);
	}

	template <typename TToType>
	static void To(const FDataflowSelection& From, TToType& To)
	{
		To.Initialize(From);
	}
};

template <typename ArrayType>
struct FDataflowConverter<TArray<ArrayType>>
{
	template <typename TFromType>
	static void From(const TFromType& From, TArray<ArrayType>& To)
	{
		To.SetNum(From.Num());
		for (int32 ArrayIndex = 0; ArrayIndex < From.Num(); ++ArrayIndex)
		{
			FDataflowConverter<ArrayType>::From(From[ArrayIndex], To[ArrayIndex]);
		}
	}

	template <typename TToType>
	static void To(const TArray<ArrayType>& From, TToType& To)
	{
		To.SetNum(From.Num());
		for (int32 ArrayIndex = 0; ArrayIndex < From.Num(); ++ArrayIndex)
		{
			FDataflowConverter<ArrayType>::To(From[ArrayIndex], To[ArrayIndex]);
		}
	}
};

template <>
struct FDataflowConverter<FRotator>
{
	template <typename TFromType>
	static void From(const TFromType& From, FRotator& To)
	{
		if constexpr (std::is_same_v<TFromType, FQuat>)
		{
			To = FRotator(From);
		}
		else if constexpr (std::is_same_v<TFromType, FVector>)
		{
			To = FRotator::MakeFromEuler(From);
		}
		else
		{
			To = From;
		}
	}

	template <typename TToType>
	static void To(const FRotator& From, TToType& To)
	{
		if constexpr (std::is_same_v<TToType, FQuat>)
		{
			To = FQuat::MakeFromRotator(From);
		}
		else if constexpr (std::is_same_v<TToType, FVector>)
		{
			To = FQuat::MakeFromRotator(From).Euler();
		}
		else
		{
			To = From;
		}
	}
};

