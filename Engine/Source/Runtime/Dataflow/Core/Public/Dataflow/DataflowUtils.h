// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/Array.h"

namespace UE::Dataflow
{
	namespace Utils
	{
		enum class EErrorCode
		{
			None = 0,
			InvalidChars = 1,
			InvalidFormatInSegment = 2,
			NumberInRangeTooBig = 3,
			InvalidRangeStartEnd = 4
		};

		/* e.g. "0, 2, 5-10, 12-15" */
		DATAFLOWCORE_API EErrorCode ParseIndicesStr(const FString& InFramesString, TArray<int32>& OutIndices);

		// Compute Min/Max value in an array
		DATAFLOWCORE_API void GetMinMaxInArray(const TArray<float>& InArray, float& OutMin, float& OutMax);
	}
}


