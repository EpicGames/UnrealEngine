// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataflowEngine : ModuleRules
	{
        public DataflowEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			bTreatAsEngineModule = true;
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"SlateCore",
					"Engine",
					"DataflowCore",
					"Chaos",
					"ImageCore",
					"GeometryCore"
				}
			);
		}
	}
}
