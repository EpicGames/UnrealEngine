// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowAssetPathNodes.h"

#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowObjectInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowAssetPathNodes)

#define LOCTEXT_NAMESPACE "DataflowAssetPathNodes"

class UStaticMesh;
class USkeletalMesh;
class USkeleton;

namespace UE::Dataflow
{
	namespace Private
	{
		static bool AutoConvertUObjectConvertibleToString(FName FromType, FName ToType)
		{
			return FDataflowUObjectConvertibleTypePolicy::SupportsTypeStatic(FromType);
		}
	}

	void RegisterAssetPathNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowGetAssetPathNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowPathReplaceNode);

		UE_DATAFLOW_REGISTER_AUTOCONVERT_WITH_FILTER(FDataflowUObjectConvertibleTypes, FString, FDataflowGetAssetPathNode, &Private::AutoConvertUObjectConvertibleToString);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FDataflowGetAssetPathNode::FDataflowGetAssetPathNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Asset);
	RegisterOutputConnection(&Path);
}

void FDataflowGetAssetPathNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Path))
	{
		FString OutPath;

		TObjectPtr<UObject> InAsset = GetValue(Context, &Asset);
		if (InAsset == nullptr)
		{
			// check if there's any bound asset to the context 
			if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
			{
				InAsset = EngineContext->Owner;
			}
		}
		if (InAsset == nullptr)
		{
			Context.Warning(LOCTEXT("NullAsset", "Asset is null, empty path will be returned"), this, Out);
		}
		else
		{
			// Make sure we get the main object and not a subObject ( that can happen in teh case of attachements ) 
			FSoftObjectPath SoftPath(InAsset);
			OutPath = SoftPath.GetAssetPath().ToString();
		}
		SetValue(Context, OutPath, &Path);
	}
}

bool FDataflowGetAssetPathNode::SupportsAssetProperty(UObject* InAsset) const
{
	return (InAsset != nullptr);
}

void FDataflowGetAssetPathNode::SetAssetProperty(UObject* InAsset)
{
	if (InAsset)
	{
		Asset.Value = InAsset;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FDataflowPathReplaceNode::FDataflowPathReplaceNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Path);
	RegisterOutputConnection(&Path, &Path);
}

void FDataflowPathReplaceNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Path))
	{
		const FString& InPath = GetValue(Context, &Path);

		FString PathSection;
		FString NameSection;
		InPath.Split("/", &PathSection, &NameSection, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		FString LeftName;
		FString RightName;
		NameSection.Split(".", &LeftName, &RightName, ESearchCase::CaseSensitive, ESearchDir::FromEnd);

		if (!PathSection.IsEmpty())
		{
			for (const FDataflowPathReplaceRule& PathRule : PathRules)
			{
				PathSection = ApplyRule(PathSection, PathRule);
			}
			PathSection += "/";
		}

		if (!LeftName.IsEmpty())
		{
			for (const FDataflowPathReplaceRule& NameRule : NameRules)
			{
				LeftName = ApplyRule(LeftName, NameRule);
			}

			LeftName += ".";
		}

		if (!RightName.IsEmpty())
		{
			for (const FDataflowPathReplaceRule& NameRule : NameRules)
			{
				RightName = ApplyRule(RightName, NameRule);
			}
		}
		
		const FString OutPath = PathSection + LeftName + RightName;
		SetValue(Context, OutPath, &Path);
	}
}

FString FDataflowPathReplaceNode::ApplyRule(const FString& InString, const FDataflowPathReplaceRule& Rule) const
{
	const ESearchCase::Type SearchCase = Rule.bIgnoreCase ? ESearchCase::IgnoreCase : ESearchCase::CaseSensitive;

	FString OutString = InString;
	switch (Rule.Method)
	{
	case EDataflowPathReplaceMethod::Anywhere:
		OutString =  InString.Replace(*Rule.From, *Rule.To, SearchCase);
		break;
		
	case EDataflowPathReplaceMethod::Beginning:
		if (InString.StartsWith(Rule.From, SearchCase))
		{
			OutString = Rule.To + InString.RightChop(Rule.From.Len());
		}
		else if (Rule.bInsertIfNoFound)
		{
			OutString = Rule.To + InString;
		}
		break;

	case EDataflowPathReplaceMethod::End:
		if (InString.EndsWith(Rule.From, SearchCase))
		{
			OutString = InString.LeftChop(Rule.From.Len()) + Rule.To;
		}
		else if (Rule.bInsertIfNoFound)
		{
			OutString = InString + Rule.To;
		}
		break;
	}
	return OutString;
}

#undef LOCTEXT_NAMESPACE
