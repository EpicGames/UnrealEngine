// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowNode.h"

#include "DataflowAssetPathNodes.generated.h"

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/*
* Get the path from an asset
* If no asset is specified via the input or in the properties the asset bound to the dataflow will be used 
*/
USTRUCT()
struct FDataflowGetAssetPathNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowGetAssetPathNode, "GetAssetPath", "Path", "UObject Folder Name String")

private:
	/** Asset to get the path from */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FDataflowUObjectConvertibleTypes Asset;

	UPROPERTY(meta = (DataflowOutput))
	FString Path;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FDataflowGetAssetPathNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual bool SupportsAssetProperty(UObject* InAsset) const override;
	virtual void SetAssetProperty(UObject* InAsset) override;
};

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UENUM()
enum class EDataflowPathReplaceMethod: uint8
{
	Anywhere,
	Beginning,
	End,
};

USTRUCT()
struct FDataflowPathReplaceRule
{
	GENERATED_USTRUCT_BODY()

public:
	/** Method to use to find the From string */
	UPROPERTY(EditAnywhere, Category = Rule)
	EDataflowPathReplaceMethod Method = EDataflowPathReplaceMethod::Anywhere;

	/** String to find and replace */
	UPROPERTY(EditAnywhere, Category = Rule)
	FString From;

	/** String to use as a replacement */
	UPROPERTY(EditAnywhere, Category = Rule)
	FString To;

	UPROPERTY(EditAnywhere, Category = Rule)
	bool bIgnoreCase = true;

	/** if true insert the string even if not found ( only for Beginning and End methods )*/
	UPROPERTY(EditAnywhere, Category = Rule, meta = (EditCondition = "Method == EDataflowPathReplaceMethod::Beginning || EDataflowPathReplaceMethod::End"))
	bool bInsertIfNoFound = false;
};

/*
* Replace part of a path
* The path is assumed to have the form  Path/Names ( Path can have multiple / and folders and Names can have multiple names seperated by a .)
*/
USTRUCT()
struct FDataflowPathReplaceNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowPathReplaceNode, "PathReplace", "Path", "Folder File Name String")

private:
	/** Path to modify and return */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Path))
	FString Path;

	/** 
	* Rules to apply to the folder part of the path 
	* Note that Rules are applied in order 
	*/
	UPROPERTY(EditAnywhere, Category = Rules, meta = (TitleProperty = "'{From}' -> '{To}' [{Method}]"))
	TArray<FDataflowPathReplaceRule> PathRules;

	/** 
	* Rules to apply to the name part of the path
	* Note that Rules are applied in order 
	*/
	UPROPERTY(EditAnywhere, Category = Rules, meta=(TitleProperty = "'{From}' -> '{To}' [{Method}]"))
	TArray<FDataflowPathReplaceRule> NameRules;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	FString ApplyRule(const FString& InString, const FDataflowPathReplaceRule& Rule) const;

public:
	FDataflowPathReplaceNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow
{
	void RegisterAssetPathNodes();
}

