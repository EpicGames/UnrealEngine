// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowAttachment.h"
#include "PhysicsEngine/PhysicsAsset.h"

UDataflowAttachment::UDataflowAttachment()
	: Instance(this)
{}

const FDataflowInstance& UDataflowAttachment::GetDataflowInstance() const
{
	return Instance;
}

FDataflowInstance& UDataflowAttachment::GetDataflowInstance()
{
	return Instance;
}

void UDataflowAttachment::PostLoad()
{
	Super::PostLoad();
	Instance.PostLoad();
}

// Provides trhe path to the preview actor to be used in the corresponding Dataflow editor
FString UDataflowAttachment::GetPreviewActorPath() const
{
	return FString();
}


//==============================================================================================

/*static*/ FDataflowAttachmentFactory& FDataflowAttachmentFactory::Get()
{
	static FDataflowAttachmentFactory Instance;
	return Instance;
}

void FDataflowAttachmentFactory::Register(FName ClassName, FCreateAttachmentFunction CreateFunction)
{
	CreateFunctionsByName.Add(ClassName, CreateFunction);
}

UDataflowAttachment* FDataflowAttachmentFactory::FDataflowAttachmentFactory::Create(UObject* Owner)
{
	const FName OwnerClassName = Owner->GetClass()->GetFName();
	if (FCreateAttachmentFunction* Function = CreateFunctionsByName.Find(OwnerClassName))
	{
		if (Function->IsSet())
		{
			return (*Function)(Owner);
		}
	}
	return nullptr;
}

