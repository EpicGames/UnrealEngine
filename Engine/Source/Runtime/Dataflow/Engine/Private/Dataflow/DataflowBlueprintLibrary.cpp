// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowBlueprintLibrary.h"

#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowInstance.h"
#include "StructUtils/PropertyBag.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowBlueprintLibrary)

void UDataflowBlueprintLibrary::EvaluateTerminalNodeByName(UDataflow* Dataflow, FName TerminalNodeName, UObject* ResultAsset)
{
	if (Dataflow && Dataflow->Dataflow)
	{
		if (const TSharedPtr<FDataflowNode> Node = Dataflow->Dataflow->FindFilteredNode(FDataflowTerminalNode::StaticType(), TerminalNodeName))
		{
			if (const FDataflowTerminalNode* TerminalNode = Node->AsType<const FDataflowTerminalNode>())
			{
				UE_LOGF(LogChaosDataflow, Verbose, "UDataflowBlueprintLibrary::EvaluateTerminalNodeByName(): Node [%ls]", *TerminalNodeName.ToString());
				UE::Dataflow::FEngineContext Context(ResultAsset);
				// Note: If the node is deactivated and has any outputs, then these outputs might still need to be forwarded.
				//       Therefore the Evaluate method has to be called for whichever value of bActive.
				//       This however isn't the case of SetAssetValue() for which the active state needs to be checked before the call.
				TerminalNode->Evaluate(Context);
				if (TerminalNode->IsActive())
				{
					UE_LOGF(LogChaosDataflow, Verbose, "FDataflowTerminalNode::SetAssetValue(): TerminalNode [%ls], Asset [%ls]", *TerminalNodeName.ToString(), *(ResultAsset? ResultAsset->GetName(): FString()));
					TerminalNode->SetAssetValue(ResultAsset, Context);
				}
			}
		}
		else
		{
			UE_LOGF(LogChaos, Warning, "EvaluateTerminalNodeByName : Could not find terminal node : [%ls], skipping evaluation", *TerminalNodeName.ToString());
		}
	}
}

bool UDataflowBlueprintLibrary::EvaluateDataflow(UDataflow* Dataflow, UObject* AssetToUpdate)
{
	if (Dataflow && Dataflow->Dataflow)
	{
		TAtomic<bool> bHasError = false;

		using namespace UE::Dataflow;
		FEngineContext Context(AssetToUpdate);
		Context.GetOnContextLogMulticast().AddLambda(
			[&bHasError](const FContext::FLogMessage& Message)
			{
				if (Message.Severity == EMessageSeverity::Error)
				{
					bHasError = true;

					FString NodeString(TEXT("None"));
					if (TSharedPtr<const FDataflowNode> Node = Message.Node.Pin())
					{
						NodeString = Node->GetName().ToString();
					}
					UE_LOGF(LogChaosDataflow, Error, "EvaluateDataflow : (%ls) %ls", *NodeString, *Message.Text.ToString());
				}
			});
		Context.EvaluateGraph<FDataflowDirectEvaluator>(Dataflow, {});
		return bHasError;
	}
	UE_LOGF(LogChaosDataflow, Error, "EvaluateDataflow : Invalid Dataflow Asset : %ls", *(Dataflow? Dataflow->GetFName().ToString(): FString(TEXT("null"))));
	return false;
}

bool UDataflowBlueprintLibrary::RegenerateAssetFromDataflow(UObject* AssetToRegenerate, bool bRegenerateDependentAssets)
{
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(AssetToRegenerate))
	{
		FDataflowInstance& DataflowInstance = DataflowInterface->GetDataflowInstance();
		return DataflowInstance.UpdateOwnerAsset(bRegenerateDependentAssets);
	}
	return false;
}

bool UDataflowBlueprintLibrary::OverrideDataflowVariableBool(UObject* Asset, FName VariableName, bool VariableValue)
{
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(Asset))
	{
		return DataflowInterface->GetDataflowInstance().GetVariableOverrides().OverrideVariableBool(VariableName, VariableValue);
	}
	return false;
}

bool UDataflowBlueprintLibrary::OverrideDataflowVariableBoolArray(UObject* Asset, FName VariableName, const TArray<bool>& VariableArrayValue)
{
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(Asset))
	{
		return DataflowInterface->GetDataflowInstance().GetVariableOverrides().OverrideVariableBoolArray(VariableName, VariableArrayValue);
	}
	return false;
}

bool UDataflowBlueprintLibrary::OverrideDataflowVariableInt(UObject* Asset, FName VariableName, int64 VariableValue)
{
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(Asset))
	{
		return DataflowInterface->GetDataflowInstance().GetVariableOverrides().OverrideVariableInt(VariableName, VariableValue);
	}
	return false;
}

bool UDataflowBlueprintLibrary::OverrideDataflowVariableIntArray(UObject* Asset, FName VariableName, const TArray<int32>& VariableArrayValue)
{
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(Asset))
	{
		return DataflowInterface->GetDataflowInstance().GetVariableOverrides().OverrideVariableInt32Array(VariableName, VariableArrayValue);
	}
	return false;
}

bool UDataflowBlueprintLibrary::OverrideDataflowVariableFloat(UObject* Asset, FName VariableName, float VariableValue)
{
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(Asset))
	{
		return DataflowInterface->GetDataflowInstance().GetVariableOverrides().OverrideVariableFloat(VariableName, VariableValue);
	}
	return false;
}

bool UDataflowBlueprintLibrary::OverrideDataflowVariableFloatArray(UObject* Asset, FName VariableName, const TArray<float>& VariableArrayValue)
{
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(Asset))
	{
		return DataflowInterface->GetDataflowInstance().GetVariableOverrides().OverrideVariableFloatArray(VariableName, VariableArrayValue);
	}
	return false;
}

bool UDataflowBlueprintLibrary::OverrideDataflowVariableObject(UObject* Asset, FName VariableName, UObject* VariableValue)
{
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(Asset))
	{
		return DataflowInterface->GetDataflowInstance().GetVariableOverrides().OverrideVariableObject(VariableName, VariableValue);
	}
	return false;
}

bool UDataflowBlueprintLibrary::OverrideDataflowVariableObjectArray(UObject* Asset, FName VariableName, const TArray<UObject*>& VariableArrayValue)
{
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(Asset))
	{
		return DataflowInterface->GetDataflowInstance().GetVariableOverrides().OverrideVariableObjectArray(VariableName, VariableArrayValue);
	}
	return false;
}



TArray<FDataflowVariable>  UDataflowBlueprintLibrary::GetDataflowVariableList(UObject* Asset)
{
	TArray<FDataflowVariable> PropArray;
	if (IDataflowInstanceInterface* DataflowInterface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(Asset))
	{
		FInstancedPropertyBag& AssetBag = DataflowInterface->GetDataflowInstance().GetVariableOverrides().GetVariables();
		TConstArrayView<FPropertyBagPropertyDesc> Descs = AssetBag.GetPropertyBagStruct()->GetPropertyDescs();
		for (FPropertyBagPropertyDesc desc : Descs)
		{
			FDataflowVariable NewPropVar;
			NewPropVar.VariableName = desc.Name;
			NewPropVar.Type = desc.ValueType;
			NewPropVar.ContainerType = desc.ContainerTypes.GetFirstContainerType();
			if (desc.ValueType == EPropertyBagPropertyType::Int64)
			{
				const TValueOrError<int64, EPropertyBagResult> res = AssetBag.GetValueInt64(desc.Name);
				if (res.HasValue())
				{
					NewPropVar.Value = FString::FromInt(res.GetValue());
					PropArray.Add(NewPropVar);
				}
				else
				{
					EPropertyBagResult Err = res.GetError();
					UE_LOGF(LogTemp, Warning, "Could not read int: error code %d", (int32)Err);
				}
			}
			else if (desc.ValueType == EPropertyBagPropertyType::Int32)
			{
				const TValueOrError<int32, EPropertyBagResult> res = AssetBag.GetValueInt32(desc.Name);
				if (res.HasValue())
				{
					NewPropVar.Value = FString::FromInt(res.GetValue());
					PropArray.Add(NewPropVar);
				}
				else
				{
					EPropertyBagResult Err = res.GetError();
					UE_LOGF(LogTemp, Warning, "Could not read int: error code %d", (int32)Err);
				}
			}
			else if (desc.ValueType == EPropertyBagPropertyType::Float)
			{
				const TValueOrError<float, EPropertyBagResult> res = AssetBag.GetValueFloat(desc.Name);
				if (res.HasValue())
				{
					NewPropVar.Value = LexToString(res.GetValue());
					PropArray.Add(NewPropVar);
				}
				else
				{
					EPropertyBagResult Err = res.GetError();
					UE_LOGF(LogTemp, Warning, "Could not read float: error code %d", (int32)Err);
				}
			}
			else if (desc.ValueType == EPropertyBagPropertyType::Double)
			{
				const TValueOrError<double, EPropertyBagResult> res = AssetBag.GetValueDouble(desc.Name);
				if (res.HasValue())
				{
					NewPropVar.Value = LexToString(res.GetValue());
					PropArray.Add(NewPropVar);
				}
				else
				{
					EPropertyBagResult Err = res.GetError();
					UE_LOGF(LogTemp, Warning, "Could not read double: error code %d", (int32)Err);
				}
			}
			else if (desc.ValueType == EPropertyBagPropertyType::Object)
			{
				const TValueOrError<UObject*, EPropertyBagResult> res = AssetBag.GetValueObject(desc.Name, UObject::StaticClass());
				if (res.HasValue())
				{
					NewPropVar.Value = res.GetValue()->GetOutermost()->GetName();
					PropArray.Add(NewPropVar);
				}
				else
				{
					EPropertyBagResult Err = res.GetError();
					UE_LOGF(LogTemp, Warning, "Could not read Object: error code %d", (int32)Err);
				}
			}
			else if (desc.ValueType == EPropertyBagPropertyType::Bool)
			{
				const TValueOrError<bool, EPropertyBagResult> res = AssetBag.GetValueBool(desc.Name);
				if (res.HasValue())
				{
					NewPropVar.Value = LexToString(res.GetValue());
					PropArray.Add(NewPropVar);
				}
				else
				{
					EPropertyBagResult Err = res.GetError();
					UE_LOGF(LogTemp, Warning, "Could not read bool: error code %d", (int32)Err);
				}
			}
		}
	}
	return PropArray;
}