// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowColorRamp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowColorRamp)


namespace UE::Dataflow::ColorMap::Private
{
	static const FName RedCurveName("R");
	static const FName GreenCurveName("G");
	static const FName BlueCurveName("B");
	static const FName AlphaCurveName("A");
}

bool FDataflowColorCurveOwner::IsEmpty() const
{
	return (RichCurves[0].IsEmpty() 
		&& RichCurves[1].IsEmpty() 
		&& RichCurves[2].IsEmpty() 
		&& RichCurves[3].IsEmpty());
}

void FDataflowColorCurveOwner::SetColorAtTime(float Time, const FLinearColor& Color, bool bOnlyRBG)
{
	RichCurves[0].AddKey(Time, Color.R);
	RichCurves[1].AddKey(Time, Color.G);
	RichCurves[2].AddKey(Time, Color.B);
	if (!bOnlyRBG)
	{
		RichCurves[3].AddKey(Time, Color.A);
	}
}

TArray<FRichCurveEditInfoConst> FDataflowColorCurveOwner::GetCurves() const
{
	using namespace UE::Dataflow::ColorMap;

	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[0], Private::RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[1], Private::GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[2], Private::BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[3], Private::AlphaCurveName));
	return Curves;
}

void FDataflowColorCurveOwner::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	using namespace UE::Dataflow::ColorMap;

	Curves.Add(FRichCurveEditInfoConst(&RichCurves[0], Private::RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[1], Private::GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[2], Private::BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[3], Private::AlphaCurveName));
}

TArray<FRichCurveEditInfo> FDataflowColorCurveOwner::GetCurves()
{
	using namespace UE::Dataflow::ColorMap;

	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(FRichCurveEditInfo(&RichCurves[0], Private::RedCurveName));
	Curves.Add(FRichCurveEditInfo(&RichCurves[1], Private::GreenCurveName));
	Curves.Add(FRichCurveEditInfo(&RichCurves[2], Private::BlueCurveName));
	Curves.Add(FRichCurveEditInfo(&RichCurves[3], Private::AlphaCurveName));
	return Curves;
}

void FDataflowColorCurveOwner::ModifyOwner()
{}

TArray<const UObject*> FDataflowColorCurveOwner::GetOwners() const
{
	static TArray<const UObject*> EmptyObjectArray;
	return EmptyObjectArray;
}

void FDataflowColorCurveOwner::MakeTransactional()
{}

void FDataflowColorCurveOwner::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	TArray<FRichCurve*> ChangedCurves;
	for (const FRichCurveEditInfo& ChangedCurveEditInfo : ChangedCurveEditInfos)
	{
		ChangedCurves.Add((FRichCurve*)ChangedCurveEditInfo.CurveToEdit);
	}
	if (ChangedCurves.Num() > 0)
	{
		OnColorCurveChangedDelegate.Broadcast(ChangedCurves);
	}
}

bool FDataflowColorCurveOwner::IsLinearColorCurve() const
{
	return true;
}

FLinearColor FDataflowColorCurveOwner::GetLinearColorValue(float InTime) const
{
	return FLinearColor(
		RichCurves[0].Eval(InTime),
		RichCurves[1].Eval(InTime),
		RichCurves[2].Eval(InTime),
		RichCurves[3].Eval(InTime));
}

bool FDataflowColorCurveOwner::HasAnyAlphaKeys() const
{
	return RichCurves[3].GetNumKeys() != 0;
}

bool FDataflowColorCurveOwner::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	return (CurveInfo.CurveToEdit == &RichCurves[0]
		|| CurveInfo.CurveToEdit == &RichCurves[1]
		|| CurveInfo.CurveToEdit == &RichCurves[2]
		|| CurveInfo.CurveToEdit == &RichCurves[3]);
}

FLinearColor FDataflowColorCurveOwner::GetCurveColor(FRichCurveEditInfo CurveInfo) const
{
	return FLinearColor::White;
}

////////////////////////////////////////////////////////////////////////////////

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FDataflowLinearColorRamp::SetFrom(const FDataflowLinearColorRamp& Other)
{
	RichCurves[0] = Other.RichCurves[0];
	RichCurves[1] = Other.RichCurves[1];
	RichCurves[2] = Other.RichCurves[2];
	RichCurves[3] = Other.RichCurves[3];
	OnCurveChanged(GetCurves());
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FDataflowLinearColorRamp::IsEmpty() const
{
	return (RichCurves[0].IsEmpty()
		&& RichCurves[1].IsEmpty()
		&& RichCurves[2].IsEmpty()
		&& RichCurves[3].IsEmpty());
}

void FDataflowLinearColorRamp::Reset()
{
	RichCurves[0].Reset();
	RichCurves[1].Reset();
	RichCurves[2].Reset();
	RichCurves[3].Reset();
}

void FDataflowLinearColorRamp::SetColorAtTime(float Time, const FLinearColor& Color, bool bOnlyRBG)
{
	RichCurves[0].AddKey(Time, Color.R);
	RichCurves[1].AddKey(Time, Color.G);
	RichCurves[2].AddKey(Time, Color.B);
	if (!bOnlyRBG)
	{
		RichCurves[3].AddKey(Time, Color.A);
	}
}

TArray<FRichCurveEditInfoConst> FDataflowLinearColorRamp::GetCurves() const
{
	using namespace UE::Dataflow::ColorMap;

	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[0], Private::RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[1], Private::GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[2], Private::BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[3], Private::AlphaCurveName));
	return Curves;
}

void FDataflowLinearColorRamp::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	using namespace UE::Dataflow::ColorMap;

	Curves.Add(FRichCurveEditInfoConst(&RichCurves[0], Private::RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[1], Private::GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[2], Private::BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RichCurves[3], Private::AlphaCurveName));
}

TArray<FRichCurveEditInfo> FDataflowLinearColorRamp::GetCurves()
{
	using namespace UE::Dataflow::ColorMap;

	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(FRichCurveEditInfo(&RichCurves[0], Private::RedCurveName));
	Curves.Add(FRichCurveEditInfo(&RichCurves[1], Private::GreenCurveName));
	Curves.Add(FRichCurveEditInfo(&RichCurves[2], Private::BlueCurveName));
	Curves.Add(FRichCurveEditInfo(&RichCurves[3], Private::AlphaCurveName));
	return Curves;
}

void FDataflowLinearColorRamp::ModifyOwner()
{
}

TArray<const UObject*> FDataflowLinearColorRamp::GetOwners() const
{
	static TArray<const UObject*> EmptyObjectArray;
	return EmptyObjectArray;
}

void FDataflowLinearColorRamp::MakeTransactional()
{
}

void FDataflowLinearColorRamp::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	TArray<FRichCurve*> ChangedCurves;
	for (const FRichCurveEditInfo& ChangedCurveEditInfo : ChangedCurveEditInfos)
	{
		ChangedCurves.Add((FRichCurve*)ChangedCurveEditInfo.CurveToEdit);
	}
	if (ChangedCurves.Num() > 0)
	{
		OnColorCurveChangedDelegate.Broadcast(ChangedCurves);
	}
}

bool FDataflowLinearColorRamp::IsLinearColorCurve() const
{
	return true;
}

FLinearColor FDataflowLinearColorRamp::GetLinearColorValue(float InTime) const
{
	return FLinearColor(
		RichCurves[0].Eval(InTime),
		RichCurves[1].Eval(InTime),
		RichCurves[2].Eval(InTime),
		RichCurves[3].Eval(InTime));
}

bool FDataflowLinearColorRamp::HasAnyAlphaKeys() const
{
	return RichCurves[3].GetNumKeys() != 0;
}

bool FDataflowLinearColorRamp::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	return (CurveInfo.CurveToEdit == &RichCurves[0]
		|| CurveInfo.CurveToEdit == &RichCurves[1]
		|| CurveInfo.CurveToEdit == &RichCurves[2]
		|| CurveInfo.CurveToEdit == &RichCurves[3]);
}

FLinearColor FDataflowLinearColorRamp::GetCurveColor(FRichCurveEditInfo CurveInfo) const
{
	return FLinearColor::White;
}
