// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowCompiler.h"

#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowGraph.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSubGraph.h"

#define LOCTEXT_NAMESPACE "DataflowCompiler"

DEFINE_LOG_CATEGORY_STATIC(LogDataflowCompiler, Log, All);

namespace UE::Dataflow
{
	namespace Private
	{
		struct FNodeSequence;
		using FNodeSequenceRef = TSharedRef<FNodeSequence>;

		// this represent a linear list of node that can be sequencially executed
		// and make sure data is ready when they do execute
		// this can be executed in a single task 
		struct FNodeSequence
		{
		public:
			void Build(FConstNodeRef StartNode);

			const TArray<FConstNodeRef>& GetNodes() const { return Nodes; }
			const TArray<FConstNodeRef>& GetDownstreamNodes() const { return DownstreamNodes; }

			bool IsEmpty() const
			{
				return Nodes.IsEmpty();
			}

			bool HasTerminalNode() const
			{
				for (const FConstNodeRef& Node : Nodes)
				{
					if (Node->IsTerminal())
					{
						return true;
					}
				}
				return false;
			}

			bool IsDeadEnd() const
			{
				return (DownstreamNodes.IsEmpty() && !HasTerminalNode());
			}

			static void AddDependency(FNodeSequenceRef UpstreamSequence, FNodeSequenceRef DownstreamSequence)
			{
				if (ensure(!UpstreamSequence->IsEmpty() && !DownstreamSequence->IsEmpty()))
				{
					UpstreamSequence->AddDownstreamDependency(DownstreamSequence);
					DownstreamSequence->AddUpstreamDependency(UpstreamSequence);

					FConstNodeRef DownstreamNode = DownstreamSequence->GetNodes()[0];
					ensure(UpstreamSequence->GetDownstreamNodes().Contains(DownstreamNode));
				}
			}

			static void CutFromUpstream(FNodeSequenceRef SequenceToCut)
			{
				SequenceToCut->Log(TEXT("CutFromUpstream"));
				for (FNodeSequenceRef UpstreamSequence : SequenceToCut->GetUpstreamDependencies())
				{
					UpstreamSequence->RemoveDownstreamDependency(SequenceToCut);
				}
				SequenceToCut->ClearUpstreamDependencies();
			}

			int32 GetExecutionIndex() const
			{
				return ExecutionIndex;
			}

			void InitExecutionIndex(int32 Value)
			{
				ExecutionIndex = FMath::Max(ExecutionIndex, Value);
				for (FNodeSequenceRef DownstreamSequence : DownstreamDependencies)
				{
					DownstreamSequence->InitExecutionIndex(ExecutionIndex + 1);
				}
			}

			void SetTaskIndex(int32 Index) { TaskIndex = Index; }
			int32 GetTaskIndex() const { return TaskIndex; }

			void AddToCompiledGraph(FCompiledGraph& OutCompiledGraph)
			{
				Log(TEXT("AddToCompiledGraph"));
				const int32 NewTaskIndex = OutCompiledGraph.StartTask();
				{
					TArray<FConstNodeRef> UpstreamNodes;
					TArray<FGuid> UpstreamNodeIds;
					for (FConstNodeRef Node : Nodes)
					{
						FCompiler::FindUpstreamNodes(Node, UpstreamNodes);
						UpstreamNodeIds.Reset(UpstreamNodes.Num());
						for (const FConstNodeRef& UpstreamNode : UpstreamNodes)
						{
							UpstreamNodeIds.Add(UpstreamNode->GetGuid());
						}
						OutCompiledGraph.AddNode(Node->GetGuid(), UpstreamNodeIds, Node->IsTerminal());
					}

					for (FNodeSequenceRef Dependency : UpstreamDependencies)
					{
						OutCompiledGraph.AddDependency(Dependency->GetTaskIndex());
					}
				}
				OutCompiledGraph.EndTask();
				ensure(NewTaskIndex == GetTaskIndex());
			}

			void AddDownstreamDependency(FNodeSequenceRef DownstreamSequence)
			{
				DownstreamDependencies.AddUnique(DownstreamSequence);
			}

			void AddUpstreamDependency(FNodeSequenceRef UpstreamSequence)
			{
				UpstreamDependencies.AddUnique(UpstreamSequence);
			}

			void RemoveDownstreamDependency(FNodeSequenceRef DownStreamSequence) 
			{ 
				DownstreamDependencies.Remove(DownStreamSequence);
				const TArray<FConstNodeRef>& DownStreamSequenceNodes = DownStreamSequence->GetNodes();
				if (DownStreamSequenceNodes.Num() > 0)
				{
					DownstreamNodes.Remove(DownStreamSequenceNodes[0]);
				}
			}

			const TArray<FNodeSequenceRef>& GetUpstreamDependencies()
			{
				return UpstreamDependencies;
			}

			const TArray<FNodeSequenceRef>& GetDownstreamDependencies()
			{
				return DownstreamDependencies;
			}

			void ClearUpstreamDependencies()
			{
				UpstreamDependencies.Reset();
			}

			void Log(const TCHAR* Message) const
			{
				if (UE_LOG_ACTIVE(LogDataflowCompiler, Verbose))
				{
					TStringBuilder<256> NodeList;
					for (const FConstNodeRef& Node : Nodes)
					{
						NodeList.Append(Node->GetName().ToString());
						NodeList.Append(", ");
					}
					UE_LOGF(LogDataflowCompiler, Verbose, "%ls - [%p] - nodes:[%ls]", Message, this, *NodeList);
				}
			}

		private:
			int32 TaskIndex = INDEX_NONE;
			int32 ExecutionIndex = INDEX_NONE;
			TArray<FConstNodeRef> Nodes;
			TArray<FConstNodeRef> DownstreamNodes;
			TArray<FNodeSequenceRef> DownstreamDependencies;
			TArray<FNodeSequenceRef> UpstreamDependencies;
		};

		void FNodeSequence::Build(FConstNodeRef StartNode)
		{
			Nodes.Reset();
			DownstreamNodes.Reset();

			Nodes.Add(StartNode);

			TArray<FConstNodeRef> UpstreamNodes;

			FConstNodePtr NodeToProcess = StartNode;
			while (NodeToProcess)
			{
				// Important : get the downstream nodes first 
				FCompiler::FindDownstreamNodes(NodeToProcess.ToSharedRef(), DownstreamNodes);

				// if the current node is a terminal we stop the sequence right away
				if (NodeToProcess->IsTerminal())
				{
					break;
				}

				// check if we have a singular node connected to this one
				if (DownstreamNodes.Num() != 1)
				{
					// No more nodes or too many, stop building 
					break;
				}

				// one downstream, let's make sure this connected node is not a merge node (upstream noded different from the one we process )
				FCompiler::FindUpstreamNodes(DownstreamNodes[0], UpstreamNodes);
				if (UpstreamNodes.Num() != 1 || UpstreamNodes[0] != NodeToProcess)
				{
					// This is a merge node, stop the build here
					break;
				}

				Nodes.Add(DownstreamNodes[0]);
				NodeToProcess = Nodes.Last();
				DownstreamNodes.Reset();
			}
			Log(TEXT("Built"));
		}
	}

	/////////////////////////////////////////////////////////////////////////////////////////////////////

	bool FCompilerResults::Succedded() const
	{
		return NumErrors == 0;
	}

	bool FCompilerResults::Failed() const
	{
		return NumErrors > 0;
	}

	void FCompilerResults::AddMessage(EMessageSeverity::Type InSeverity, const FGuid& InNode, const FGuid& InOutput, const FText& Text)
	{
		if (InSeverity == EMessageSeverity::Error)
		{
			NumErrors++;
		}
		else if (InSeverity == EMessageSeverity::Warning)
		{
			NumWarnings++;
		}
		Messages.Emplace(
			FCompilerMessage
			{
				.Severity = InSeverity,
				.Node = InNode,
				.Output = InOutput,
				.Text = Text,
			});
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////



	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	/*static*/ bool FCompiler::Compile(const FCompilerParameters& Params, FCompilerResults& Results)
	{
		static FGuid NoGuid;

		UE_LOGF(LogDataflowCompiler, Verbose, "Dataflow Compiler - START");

		const double StartCompilationTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

		// Get dataflow Object to compile
		TStrongObjectPtr<const UDataflow> DataflowObject = Params.DataflowObjectToCompile.Pin();
		if (DataflowObject)
		{
			UE_LOGF(LogDataflowCompiler, Verbose, "Dataflow Compiler - Asset [%ls]", *DataflowObject->GetFName().ToString());

			TArray<FConstNodeRef> NodesInGraph;

			// compile main graph 
			CompileGraph(*DataflowObject, Params, Results);

			// now compile subgraphs
			for (const UDataflowSubGraph* SubGraph : DataflowObject->GetSubGraphs())
			{
				if (SubGraph)
				{
					CompileGraph(*SubGraph, Params, Results);
				}
			}
		}
		else
		{
			Results.AddMessage(EMessageSeverity::Error, NoGuid, NoGuid, LOCTEXT("NullDataflowObject", "Dataflow object is null"));
		}

		const double EndCompilationTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
		Results.CompilationTimeMs = (EndCompilationTimeMs - StartCompilationTimeMs);

		UE_LOGF(LogDataflowCompiler, Verbose, "Dataflow Compiler - END - time=%.3fms", (float)Results.CompilationTimeMs);

		return Results.Succedded();
	}

	/*static*/ bool FCompiler::CompileGraph(const UEdGraph& GraphObject, const FCompilerParameters& Params, FCompilerResults& Results)
	{
		const UDataflow* Dataflow = UDataflow::GetDataflowAssetFromEdGraph(&GraphObject);
		if (!Dataflow || !Dataflow->Dataflow.IsValid())
		{
			return false;
		}

		// Now build the command buffer
		TSharedRef<FCompiledGraph> CompiledGraph = MakeShared<FCompiledGraph>(Dataflow->Dataflow.ToSharedRef());
		Results.AddCompiledGraph(&GraphObject, CompiledGraph);

		// Find first the left nost thread as start node for the node thread
		TArray<FConstNodeRef> StartNodes;
		FindStartNodes(GraphObject, StartNodes);

		TArray<FConstNodeRef> NodeToProcess;
		NodeToProcess.Append(StartNodes);

		TMap<FConstNodeRef, Private::FNodeSequenceRef> NodeSequencesByStartNode;

		// let create all the needed node sequences
		while (NodeToProcess.Num() > 0)
		{
			FConstNodeRef StartNode = NodeToProcess.Pop();
			if (ensure(!NodeSequencesByStartNode.Contains(StartNode)))
			{
				Private::FNodeSequenceRef NodeSequence = MakeShared<Private::FNodeSequence>();
				NodeSequence->Build(StartNode);
				NodeSequencesByStartNode.Add(StartNode, NodeSequence);

				for (FConstNodeRef DownstreamNode : NodeSequence->GetDownstreamNodes())
				{
					if (!NodeSequencesByStartNode.Contains(DownstreamNode))
					{
						NodeToProcess.AddUnique(DownstreamNode);
					}
				}
			}
		}

		const Private::FNodeSequenceRef EmptyNodeSequence = MakeShared<Private::FNodeSequence>();

		// Connect the sequences together
		for (const TPair<FConstNodeRef, Private::FNodeSequenceRef>& NodeSequenceByStartNode : NodeSequencesByStartNode)
		{
			const Private::FNodeSequenceRef UpstreamNodeSequence = NodeSequenceByStartNode.Value;
			const TArray<FConstNodeRef>& DownstreamNodes = UpstreamNodeSequence->GetDownstreamNodes();
			for (FConstNodeRef DownstreamNode: DownstreamNodes)
			{
				const Private::FNodeSequenceRef DownstreamNodeSequence = NodeSequencesByStartNode.FindRef(DownstreamNode, EmptyNodeSequence);
				if (!DownstreamNodeSequence->IsEmpty())
				{
					Private::FNodeSequence::AddDependency(UpstreamNodeSequence, DownstreamNodeSequence);
				}
			}
		}

		// if needed, let's remove dead end sequences recursively 
		// a dead end is a sequence that does not have downstream dependecies and does not end with a terminal or output node
		if (Params.bRemoveDeadEnds)
		{
			TArray<TPair<FConstNodeRef, Private::FNodeSequenceRef>> SequencesToProcess = NodeSequencesByStartNode.Array();
			while (SequencesToProcess.Num() > 0)
			{
				TPair<FConstNodeRef, Private::FNodeSequenceRef> SequenceToProcess = SequencesToProcess.Pop();
				if (SequenceToProcess.Value->IsDeadEnd())
				{
					// add the previous sequences as they may become dead end themselves after we cut the current one
					for (Private::FNodeSequenceRef UpstreamSequence : SequenceToProcess.Value->GetUpstreamDependencies())
					{
						TArray<FConstNodeRef> UpstreamNodes = UpstreamSequence->GetNodes();
						if (UpstreamNodes.Num())
						{
							TPair<FConstNodeRef, Private::FNodeSequenceRef> SequenceToAdd{ UpstreamNodes[0], UpstreamSequence };
							if (!SequencesToProcess.Contains(SequenceToAdd))
							{
								SequencesToProcess.Push(SequenceToAdd);
							}
						}
					}
					Private::FNodeSequence::CutFromUpstream(SequenceToProcess.Value);
					NodeSequencesByStartNode.Remove(SequenceToProcess.Key);
				}
			}
		}

		// Sort the sequences
		TArray<Private::FNodeSequenceRef> SortedSequencesByDependencies;
		NodeSequencesByStartNode.GenerateValueArray(SortedSequencesByDependencies);

		for (Private::FNodeSequenceRef Sequence : SortedSequencesByDependencies)
		{
			Sequence->InitExecutionIndex(0);
		}

		Algo::Sort(SortedSequencesByDependencies,
			[](const Private::FNodeSequenceRef& A, const Private::FNodeSequenceRef& B) -> bool
			{
				return (A->GetExecutionIndex() < B->GetExecutionIndex());
			});

		for (int32 TaskIndex = 0; TaskIndex < SortedSequencesByDependencies.Num(); ++TaskIndex)
		{
			SortedSequencesByDependencies[TaskIndex]->SetTaskIndex(TaskIndex);
		}
		for (Private::FNodeSequenceRef Sequence : SortedSequencesByDependencies)
		{
			Sequence->AddToCompiledGraph(*CompiledGraph);
		}
		// TODO : transfer the compiled graph over to the results
		return Results.Succedded();
	}

	/*static*/ void FCompiler::FindStartNodes(const UEdGraph& GraphObject, TArray<FConstNodeRef>& OutStartNodes)
	{
		OutStartNodes.Reset();
		for (const UEdGraphNode* EdNode : GraphObject.Nodes)
		{
			if (const UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
			{
				if (TSharedPtr<const FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
				{
					bool bHasAnyInputConnected = false;
					for (const FDataflowInput* Input : DataflowNode->GetInputs())
					{
						if (Input && Input->IsConnected())
						{
							bHasAnyInputConnected = true;
							break;
						}
					}
					if (!bHasAnyInputConnected)
					{
						OutStartNodes.Add(DataflowNode.ToSharedRef());
					}
				}
			}
		}
	}

	/*static*/ void FCompiler::FindDownstreamNodes(const FConstNodeRef& Node, TArray<FConstNodeRef>& OutDownstreamNodes)
	{
		OutDownstreamNodes.Reset();
		for (const FDataflowOutput* Output : Node->GetOutputs())
		{
			if (Output && Output->IsConnected())
			{
				for (const FDataflowInput* Input : Output->GetConnections())
				{
					if (Input && Input->GetOwningNode())
					{
						OutDownstreamNodes.AddUnique(Input->GetOwningNode()->AsShared());
					}
				}
			}
		}
	}

	/*static*/ void FCompiler::FindUpstreamNodes(const FConstNodeRef& Node, TArray<FConstNodeRef>& OutUpstreamNodes)
	{
		OutUpstreamNodes.Reset();
		for (const FDataflowInput* Input : Node->GetInputs())
		{
			if (Input && Input->IsConnected())
			{
				if (const FDataflowOutput* Output = Input->GetConnection())
				{
					if (Output && Output->GetOwningNode())
					{
						OutUpstreamNodes.AddUnique(Output->GetOwningNode()->AsShared());
					}
				}
			}
		}
	}

}

#undef LOCTEXT_NAMESPACE