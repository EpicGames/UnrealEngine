// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Dataflow/DataflowCompiledGraph.h"
#include "HAL/Platform.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

struct FDataflowNode;
class UDataflow;
class UEdGraph;

namespace UE::Dataflow
{
	class FGraph;

	using FConstNodeRef = TSharedRef<const FDataflowNode>;
	using FConstNodePtr = TSharedPtr<const FDataflowNode>;
	/**
	* Dataflow compiler takes a UDataflowGraph and build a representation of the graph execution and stores it into compiled graph objects
	* the compiled graph are only valid for the topology of the graph at the time of compilation 
	*/
	struct FCompilerParameters
	{
	public:
		bool bRemoveDeadEnds = true;
		TWeakObjectPtr<const UDataflow> DataflowObjectToCompile;
	};

	struct FCompilerMessage
	{
		EMessageSeverity::Type Severity = EMessageSeverity::Info;
		FGuid Node;
		FGuid Output;
		FText Text;
	};

	struct FCompilerResults
	{
	public:
		bool Succedded() const;
		bool Failed() const;

		void AddMessage(EMessageSeverity::Type InSeverity, const FGuid& InNode, const FGuid& InOutput, const FText& Text);

		void AddCompiledGraph(const UObject* GraphObject, TSharedRef<FCompiledGraph> CompiledGraph)
		{
			ensure(!CompiledGraphs.Contains(GraphObject));
			CompiledGraphs.Add(GraphObject, CompiledGraph);
		}

		TSharedPtr<const FCompiledGraph> GetCompiledGraph(const UObject* GraphObject) const
		{
			const TSharedRef<FCompiledGraph>* CompiledGraphPtr = CompiledGraphs.Find(GraphObject);
			return CompiledGraphPtr ? CompiledGraphPtr->ToSharedPtr() : TSharedPtr<const FCompiledGraph>();
		}

		const TMap<const UObject*, TSharedRef<FCompiledGraph>> GetCompiledGraphs() const { return CompiledGraphs; }

		double GetCompilationTimeMs() const { return CompilationTimeMs; }

	private:
		friend struct FCompiler;

		double CompilationTimeMs = 0;
		int32 NumErrors = 0;
		int32 NumWarnings = 0;
		TArray<FCompilerMessage> Messages;
		TMap<const UObject*, TSharedRef<FCompiledGraph>> CompiledGraphs;
	};

	struct FCompiler
	{
	public:
		DATAFLOWENGINE_API static bool Compile(const FCompilerParameters& Params, FCompilerResults& Results);

		static bool CompileGraph(const UEdGraph& GraphObject, const FCompilerParameters& Params, FCompilerResults& Results);
		static void FindStartNodes(const UEdGraph& GraphObject, TArray<FConstNodeRef>& OutStartNodes);
		static void FindDownstreamNodes(const FConstNodeRef& Node, TArray<FConstNodeRef>& OutDownstreamNodes);
		static void FindUpstreamNodes(const FConstNodeRef& Node, TArray<FConstNodeRef>& OutUpstreamNodes);
	};
}