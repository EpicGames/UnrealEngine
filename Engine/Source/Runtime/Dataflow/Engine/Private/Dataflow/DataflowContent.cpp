// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowInstance.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowObject.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/Skeleton.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "GameFramework/Actor.h"
#include "Misc/PackageName.h"
#include "PreviewScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowContent)

bool bDataflowEnableContextCaching = false;
FAutoConsoleVariableRef CVARDataflowEnableContextCaching(TEXT("p.Dataflow.Editor.ContextCaching"), bDataflowEnableContextCaching,
	TEXT("Allow the Dataflow editor to crate and use a pre-evaluated graph when the dataflow editor is re-opened.[def:true]"));


namespace UE::DataflowContextHelpers
{

	/*
	* BindContextToGraph
	*
	* On load the Property in the cache will be a nullptr
	* This block will rebind the property in the cache to the
	* the Property on the UDataflow asset. If there is ever an
	* mis-match, just return false to indicate that the binding
	* has failed.
	*/
	bool BindContextToGraph(TObjectPtr<UObject>& Asset, UDataflow* DataflowAsset)
	{
		using namespace UE::Dataflow;

		TSharedPtr<FGraph> Dataflow = DataflowAsset->GetDataflow();
		if (!Dataflow) return false;

		UDataflowBaseContent* BaseContent = Cast< UDataflowBaseContent>(Asset.Get());
		if (!BaseContent) return false;

		TSharedPtr<FEngineContext>& Context = BaseContent->GetDataflowContext();
		if (!Context) return false;

		TSet<FContextCacheKey> Keys; Context->GetKeys(Keys);
		for (FContextCacheKey Key : Keys)
		{
			bool bValidKey = false;

			if (const TUniquePtr<FContextCacheElementBase>* Data = Context->GetBaseData(Key))
			{
				if ((*Data) && !(*Data)->GetProperty())
				{
					if (TSharedPtr<FDataflowNode> Node = Dataflow->FindBaseNode((*Data)->GetNodeGuid()))
					{
						if (FDataflowOutput* Output = Node->FindOutput(Key))
						{
							if (const FProperty* Property = Output->GetProperty())
							{
								(*Data)->SetProperty(Property);
								bValidKey = true;
							}
						}
					}
				}
				else
				{
					bValidKey = true;
				}
			}

			if (!bValidKey)
			{
				return false;
			}
		}
		return true;
	}

	/*
	* ValidateCachedNodeHash
	*
	* Check that the hashes stored in the cache reflect the hash of the nodes
	* properties.
	*
	*/
	bool ValidateCachedNodeHash(TObjectPtr<UObject>& Asset, UDataflow* DataflowAsset)
	{
		using namespace UE::Dataflow;

		TSharedPtr<FGraph> Dataflow = DataflowAsset->GetDataflow();
		if (!Dataflow) return false;

		UDataflowBaseContent* BaseContent = Cast< UDataflowBaseContent>(Asset.Get());
		if (!BaseContent) return false;

		TSharedPtr<FEngineContext>& Context = BaseContent->GetDataflowContext();
		if (!Context) return false;

		TSet<FContextCacheKey> Keys; Context->GetKeys(Keys);
		for (FContextCacheKey Key : Keys)
		{
			bool bValidKey = false;
			if (const TUniquePtr<FContextCacheElementBase>* Data = Context->GetBaseData(Key))
			{
				if (!(*Data)) return false;
				if (!(*Data)->GetProperty()) return false;
				if (TSharedPtr<FDataflowNode> Node = Dataflow->FindBaseNode((*Data)->GetNodeGuid()))
				{
					if ((*Data)->GetNodeHash() != Node->GetValueHash())
					{
						return false;
					}
					bValidKey = true;
				}
			}
			if (!bValidKey)
			{
				return false;
			}
		}


		return true;
	}

	bool ResetCacheTimestamp(TObjectPtr<UObject>& Asset, UDataflow* DataflowAsset)
	{
		using namespace UE::Dataflow;

		UDataflowBaseContent* BaseContent = Cast< UDataflowBaseContent>(Asset.Get());
		if (!BaseContent) return false;
		if (!BaseContent->GetDataflowContext()) return false;

		UE::Dataflow::FTimestamp NewTimestamp = DataflowAsset->GetRenderingTimestamp().Value + 1;
		BaseContent->SetLastModifiedTimestamp(NewTimestamp, false /*bMakeDirty*/);

		TSharedPtr<FEngineContext>& Context = BaseContent->GetDataflowContext();
		TSet<FContextCacheKey> Keys; Context->GetKeys(Keys);
		for (FContextCacheKey Key : Keys)
		{
			if (const TUniquePtr<FContextCacheElementBase>* Data = Context->GetBaseData(Key))
			{
				if (*Data)
				{
					(*Data)->SetTimestamp(NewTimestamp);
				}
				else
				{
					return false;
				}
			}
			else
			{
				return false;
			}
		}
		return true;
	}

	/*
	* CreateNewDataflowContent
	*
	*/
	template<class T>
	TObjectPtr<T> CreateNewDataflowContent(const TObjectPtr<UObject>& ContentOwner)
	{
		check(ContentOwner.Get());
		bool bNeedsNewAsset = true;
		TObjectPtr<UObject> Asset = nullptr;

		UDataflow* DataflowAsset = UE::Dataflow::InstanceUtils::GetDataflowAssetFromObject(ContentOwner);
		if (bDataflowEnableContextCaching)
		{
			UClass* DataflowClass = T::StaticClass();

			const FString AssetPackageName = ContentOwner->GetOutermost()->GetName();
			const FString AssetDefaultPath = FPackageName::GetLongPackagePath(AssetPackageName);

			FString PackageName = FString::Printf(TEXT("%s/Cache/Dataflow/DataflowContext_%s"), *AssetDefaultPath, *ContentOwner.GetName());
			if (DataflowAsset)
			{
				PackageName = FString::Printf(TEXT("%s_%s"), *PackageName, *DataflowAsset->GetName());
			}

			UPackage* Package = FindPackage(nullptr, *PackageName);
			if (!Package)
			{
				Package = CreatePackage(*PackageName);
			}


			Asset = StaticLoadObject(DataflowClass, Package, *PackageName);
			if (Asset && DataflowAsset)
			{
				// Validate the loaded cache
				bNeedsNewAsset =
					!BindContextToGraph(Asset, DataflowAsset) ||
					!ValidateCachedNodeHash(Asset, DataflowAsset) ||
					!ResetCacheTimestamp(Asset, DataflowAsset);
			}

			if (!Asset || bNeedsNewAsset)
			{
				const FName AssetName(FPackageName::GetLongPackageAssetName(PackageName));
				Asset = NewObject<UObject>(Package, DataflowClass, AssetName, RF_Public | RF_Standalone | RF_Transactional);

				Asset->MarkPackageDirty();

				if (UDataflowBaseContent* BaseContent = Cast< UDataflowBaseContent>(Asset.Get()))
				{
					BaseContent->SetIsSaved(true);
					BaseContent->SetDataflowOwner(ContentOwner);
					BaseContent->SetDataflowAsset(DataflowAsset);
				}
			}
		}
		else
		{
			Asset = NewObject<T>(GetTransientPackage(), T::StaticClass(), NAME_None, RF_Transient);
			if (UDataflowBaseContent* BaseContent = Cast< UDataflowBaseContent>(Asset.Get()))
			{
				BaseContent->SetIsSaved(false);
				BaseContent->SetDataflowOwner(ContentOwner);
				BaseContent->SetDataflowAsset(DataflowAsset);
			}
		}


		if (UDataflowBaseContent* BaseContent = Cast< UDataflowBaseContent>(Asset.Get()))
		{
			BaseContent->SetDataflowOwner(ContentOwner);
		}
		return Cast<T>(Asset.Get());
	}

	template DATAFLOWENGINE_API TObjectPtr<UDataflowBaseContent> CreateNewDataflowContent(const TObjectPtr<UObject>& ContentOwner);
	template DATAFLOWENGINE_API TObjectPtr<UDataflowSkeletalContent> CreateNewDataflowContent(const TObjectPtr<UObject>& ContentOwner);
}

TObjectPtr<UDataflowBaseContent> IDataflowContentOwner::BuildDataflowContent()
{
	if(TObjectPtr<UDataflowBaseContent> DataflowContent = CreateDataflowContent())
	{
		// Delegate used for notifying owner data invalidation
		OnContentOwnerChanged.AddUObject(DataflowContent, &UDataflowBaseContent::UpdateContentDatas);
		return DataflowContent;
	}
	return nullptr;
}

//
// UDataflowBaseContent
//

void UDataflowBaseContent::SetConstructionDirty(bool InDirty) 
{ 
	bIsConstructionDirty = InDirty;
}

void UDataflowBaseContent::SetSimulationDirty(bool InDirty) 
{ 
	bIsSimulationDirty = InDirty;
}

UDataflowBaseContent::UDataflowBaseContent()
{
}

UDataflowBaseContent::~UDataflowBaseContent()
{
	// Do not unbind from the owner's OnContentOwnerChanged here: during GC it may already be destroyed.
	// It holds only a weak binding to this content anyway, which self-drops.
}

void UDataflowBaseContent::UpdateContentDatas()
{
	if(GetDataflowOwner())
	{
		if(const IDataflowContentOwner* ContentOwner = Cast<IDataflowContentOwner>(GetDataflowOwner()))
		{
			ContentOwner->WriteDataflowContent(this);
		}
	}
}

void UDataflowBaseContent::SetDataflowOwner(const TObjectPtr<UObject>& InOwner)
{
	if(!DataflowContext)
	{
		DataflowContext = MakeShared<UE::Dataflow::FEngineContext>(nullptr);
	}
	DataflowContext->Owner = InOwner;  
	SetConstructionDirty(true);
	SetSimulationDirty(true);
}

TObjectPtr<UObject> UDataflowBaseContent::GetDataflowOwner() const 
{
	return DataflowContext ? DataflowContext->Owner : nullptr; 
}

void UDataflowBaseContent::SetDataflowAsset(const TObjectPtr<UDataflow>& DataflowAsset)
{
	if(!DataflowContext)
	{
		DataflowContext = MakeShared<UE::Dataflow::FEngineContext>(nullptr);
	}
	DataflowGraph = DataflowAsset;  
	SetConstructionDirty(true);
	SetSimulationDirty(true);
}

#if WITH_EDITOR

void UDataflowBaseContent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) 
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if(IDataflowContentOwner* ContentOwner = Cast<IDataflowContentOwner>(GetDataflowOwner()))
	{
		ContentOwner->ReadDataflowContent(this);
	}
}

#endif //if WITH_EDITOR

void UDataflowBaseContent::SetLastModifiedTimestamp(UE::Dataflow::FTimestamp InTimestamp, bool bMakeDirty) 
{ 
	if (InTimestamp.IsInvalid() || LastModifiedTimestamp < InTimestamp)
	{
		LastModifiedTimestamp = InTimestamp; 
		if (bMakeDirty)
		{
			if(GetDataflowAsset() && GetDataflowAsset()->Type == EDataflowType::Construction)
			{
				SetConstructionDirty(true);
				SetSimulationDirty(true);
			}
			MarkPackageDirty(); 
		}
	}
}

void UDataflowBaseContent::SetDataflowContext(const TSharedPtr<UE::Dataflow::FEngineContext>& InContext) 
{ 
	DataflowContext = InContext;  
	SetConstructionDirty(true);
	SetSimulationDirty(true);
	MarkPackageDirty();
}

void UDataflowBaseContent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << LastModifiedTimestamp;

	if (!DataflowContext)
	{
		DataflowContext = MakeShared<UE::Dataflow::FEngineContext>(nullptr);
	}
	DataflowContext->Serialize(Ar);
}

void UDataflowBaseContent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UDataflowBaseContent* This = CastChecked<UDataflowBaseContent>(InThis);
	if(This->DataflowContext)
	{
		Collector.AddReferencedObject(This->DataflowContext->Owner);
		Collector.AddReferencedObject(This->DataflowGraph);
	}
	Super::AddReferencedObjects(InThis, Collector);
}

void UDataflowBaseContent::OverrideActorProperty(const TObjectPtr<AActor>& PreviewActor, TObjectPtr<UObject> PropertyValue, const FName& PropertyName)
{
	if(PreviewActor && PropertyValue)
	{
		if(const FProperty* DataflowProperty = PreviewActor->GetClass()->FindPropertyByName(PropertyName))
		{
			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(DataflowProperty))
			{
				if(ObjectProperty->PropertyClass == PropertyValue->GetClass())
				{
					if(UObject** PropertyObject = DataflowProperty->ContainerPtrToValuePtr<UObject*>(PreviewActor))
					{
						(*PropertyObject) = PropertyValue;
					}
				}
			}
		}
	}
}

void UDataflowBaseContent::SetActorProperties(TObjectPtr<AActor>& PreviewActor) const
{
	OverrideActorProperty(PreviewActor, GetDataflowOwner(), TEXT("DataflowAsset"));
}

//
// UDataflowSkeletalContent
//

UDataflowSkeletalContent::UDataflowSkeletalContent() : Super()
{
}

void UDataflowSkeletalContent::SetSkeletalMesh(const TObjectPtr<USkeletalMesh>& SkeletalMeshAsset, const bool bHideAsset)
{
	SkeletalMesh = SkeletalMeshAsset;
	bHideSkeletalMesh = bHideAsset;
	SetConstructionDirty(true);
	SetSimulationDirty(true);
}

void UDataflowSkeletalContent::SetAnimationAsset(const TObjectPtr<UAnimationAsset>& SkeletalAnimationAsset, const bool bHideAsset)
{
	AnimationAsset = SkeletalAnimationAsset;
	bHideAnimationAsset = bHideAsset;
	SetConstructionDirty(true);
	SetSimulationDirty(true);
}

#if WITH_EDITOR

void UDataflowSkeletalContent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) 
{
	const FName PropertyName = PropertyChangedEvent.Property->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDataflowSkeletalContent, SkeletalMesh))
	{
		SetSkeletalMesh(SkeletalMesh);
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDataflowSkeletalContent, AnimationAsset))
	{
		SetAnimationAsset(AnimationAsset);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UDataflowSkeletalContent::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	const FName& Name = InProperty->GetFName();
	if (Name == GET_MEMBER_NAME_CHECKED(ThisClass, SkeletalMesh))
	{
		return bHideSkeletalMesh == false;
	}
	else if(Name == GET_MEMBER_NAME_CHECKED(ThisClass, AnimationAsset))
	{
		return bHideAnimationAsset == false;
	}

	return true;
}

#endif //if WITH_EDITOR

void UDataflowSkeletalContent::AddContentObjects(FReferenceCollector& Collector)
{
	Super::AddContentObjects(Collector);
}

void UDataflowSkeletalContent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UDataflowSkeletalContent* This = CastChecked<UDataflowSkeletalContent>(InThis);
	Collector.AddReferencedObject(This->SkeletalMesh);
	Collector.AddReferencedObject(This->AnimationAsset);
	Super::AddReferencedObjects(InThis, Collector);
}

void UDataflowSkeletalContent::SetActorProperties(TObjectPtr<AActor>& PreviewActor) const
{
	Super::SetActorProperties(PreviewActor);

	OverrideActorProperty(PreviewActor, AnimationAsset, TEXT("AnimationAsset"));
	OverrideActorProperty(PreviewActor, SkeletalMesh, TEXT("SkeletalMesh"));
}

#if WITH_EDITOR
bool UDataflowSkeletalContent::ShouldFilterAnimation(const FAssetData& AssetData) const
{
	if (!SkeletalMesh || !SkeletalMesh->GetSkeleton() || !bListCompatibleAnimationsOnly)
	{
		return false; // nothing to filter against — show everything
	}

	const FString TargetSkeleton = FAssetData(SkeletalMesh->GetSkeleton()).GetExportTextName();
	const FString AssetSkeleton = AssetData.GetTagValueRef<FString>(TEXT("Skeleton"));
	return AssetSkeleton != TargetSkeleton; // true = exclude
}
#endif