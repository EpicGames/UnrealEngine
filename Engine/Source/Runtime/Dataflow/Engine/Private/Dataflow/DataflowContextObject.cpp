// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowContextObject.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowContextEvaluator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowContextObject)


void UDataflowContextObject::EvaluateGraph(UE::Dataflow::FOnPostEvaluationFunction OnPostEvaluation)
{
	if (DataflowEvaluator)
	{
		DataflowEvaluator->Cancel();
	}
	if (DataflowContext)
	{
		DataflowEvaluator = DataflowContext->EvaluateGraph<UE::Dataflow::FDataflowTaskGraphEvaluator>(OnPostEvaluation);
		
	}
}

void UDataflowContextObject::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UDataflowContextObject* This = CastChecked<UDataflowContextObject>(InThis);
	Collector.AddReferencedObject(This->SelectedNode);
	Super::AddReferencedObjects(InThis, Collector);
}

