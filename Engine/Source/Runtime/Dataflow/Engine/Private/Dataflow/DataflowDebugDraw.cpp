// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowDebugDrawComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowDebugDraw)

FDataflowDebugDraw::FDataflowDebugDraw(FDataflowDebugRenderSceneProxy* InDebugRenderSceneProxy, IDataflowDebugDrawInterface::FDataflowElementsType& InDataflowElements) :
	DebugRenderSceneProxy(InDebugRenderSceneProxy), DataflowElements(InDataflowElements)
{
	DebugRenderSceneProxy->DrawAlpha = 1;		// uint Alpha values below this are not rendered (default is 100 but we want to allow more translucency)
	DebugRenderSceneProxy->ClearAll();
	ResetAllState();
}


void FDataflowDebugDraw::SetColor(const FLinearColor& InColor)
{
	Color = InColor;
	if (bTranslucent)
	{
		ColorWithTranslucency = Color.CopyWithNewOpacity(0.25);
	}
	else
	{
		ColorWithTranslucency = Color;
	}
}

void FDataflowDebugDraw::SetPointSize(float InSize)
{
	PointSize = InSize;
}

void FDataflowDebugDraw::SetLineWidth(double InWidth)
{
	LineWidth = InWidth;
}

void FDataflowDebugDraw::SetWireframe(bool bInWireframe)
{
	bWireframe = bInWireframe;

	if (bWireframe && bShaded)
	{
		DrawType = FDebugRenderSceneProxy::EDrawType::SolidAndWireMeshes;
	}
	else if (bWireframe)
	{
		DrawType = FDebugRenderSceneProxy::EDrawType::WireMesh;
	}
	else if (bShaded)
	{
		DrawType = FDebugRenderSceneProxy::EDrawType::SolidMesh;
	}
}

void FDataflowDebugDraw::SetShaded(bool bInShaded)
{
	bShaded = bInShaded;

	if (bWireframe && bShaded)
	{
		DrawType = FDebugRenderSceneProxy::EDrawType::SolidAndWireMeshes;
	}
	else if (bWireframe)
	{
		DrawType = FDebugRenderSceneProxy::EDrawType::WireMesh;
	}
	else if (bShaded)
	{
		DrawType = FDebugRenderSceneProxy::EDrawType::SolidMesh;
	}
}

void FDataflowDebugDraw::SetTranslucent(bool bInTranslucent)
{
	bTranslucent = bInTranslucent;

	if (bTranslucent)
	{
		ColorWithTranslucency = Color.CopyWithNewOpacity(0.25);
	}
	else
	{
		ColorWithTranslucency = Color;
	}
}

void FDataflowDebugDraw::SetForegroundPriority()
{
	// TODO: FDebugRenderSceneProxy currently only renders SDPG_World, it should handle SDPG_Foreground as well
	PriorityGroup = SDPG_Foreground;
}

void FDataflowDebugDraw::SetWorldPriority()
{
	PriorityGroup = SDPG_World;
}

void FDataflowDebugDraw::ResetAllState()
{
	Color = FLinearColor::White;
	LineWidth = 1.0;
	bWireframe = true;
	bShaded = false;
	bTranslucent = false;
	PriorityGroup = SDPG_World;
	ColorWithTranslucency = Color;
}

void FDataflowDebugDraw::ReservePoints(int32 NumAdditionalPoints)
{
	DebugRenderSceneProxy->ReservePoints(NumAdditionalPoints);
}

void FDataflowDebugDraw::DrawObject(const TRefCountPtr<IDataflowDebugDrawObject>& Object)
{
	DebugRenderSceneProxy->AddObject(Object);
}

void FDataflowDebugDraw::DrawPoint(const FVector& Position)
{
	FDataflowDebugRenderSceneProxy::FDebugPoint NewPoint
	{
		.Position = Position,
		.Size = PointSize,
		.Color = ColorWithTranslucency.ToFColor(true),
		.Priority = PriorityGroup,
	};
	DebugRenderSceneProxy->AddPoint(NewPoint);
}

void FDataflowDebugDraw::DrawLine(const FVector& Start, const FVector& End) const
{
	DebugRenderSceneProxy->Lines.Add(FDebugRenderSceneProxy::FDebugLine(Start, End, ColorWithTranslucency.ToFColor(true), LineWidth, PriorityGroup));
}

void FDataflowDebugDraw::DrawMesh(const IDebugDrawMesh& Mesh) const
{
	const FColor MeshColor = ColorWithTranslucency.ToFColor(true);

	if (bWireframe)		// FDebugRenderSceneProxy only renders solid meshes
	{
		DebugRenderSceneProxy->Lines.Reserve(DebugRenderSceneProxy->Lines.Num() + Mesh.GetMaxTriangleIndex() * 3);

		for (int32 TriangleIndex = 0; TriangleIndex < Mesh.GetMaxTriangleIndex(); ++TriangleIndex)
		{
			if (Mesh.IsValidTriangle(TriangleIndex))
			{
				const FIntVector3 Tri = Mesh.GetTriangle(TriangleIndex);
				const FVector A = Mesh.GetVertexPosition(Tri[0]);
				const FVector B = Mesh.GetVertexPosition(Tri[1]);
				const FVector C = Mesh.GetVertexPosition(Tri[2]);
				DrawLine(A, B);
				DrawLine(B, C);
				DrawLine(C, A);
			}
		}
	}
	
	if (bShaded)
	{
		FDebugRenderSceneProxy::FMesh SceneProxyMesh;

		SceneProxyMesh.Box = FBox(EForceInit::ForceInit);

		SceneProxyMesh.Vertices.Reserve(Mesh.GetMaxVertexIndex());

		for (int32 VertexIndex = 0; VertexIndex < Mesh.GetMaxVertexIndex(); ++VertexIndex)
		{
			if (!Mesh.IsValidVertex(VertexIndex))
			{
				SceneProxyMesh.Vertices.Add(FDynamicMeshVertex(FVector3f(0.0f), FVector2f(0.f), FColor(0)));
			}
			else
			{
				const FVector Vertex = Mesh.GetVertexPosition(VertexIndex);
				SceneProxyMesh.Vertices.Add(FDynamicMeshVertex(FVector3f(Vertex), FVector2f(0.f, 0.f), MeshColor));
				SceneProxyMesh.Box += Vertex;
			}
		}

		SceneProxyMesh.Indices.Reserve(3 * Mesh.GetMaxTriangleIndex());
		for (int32 TriangleIndex = 0; TriangleIndex < Mesh.GetMaxTriangleIndex(); ++TriangleIndex)
		{
			if (Mesh.IsValidTriangle(TriangleIndex))
			{
				const FIntVector3 Tri = Mesh.GetTriangle(TriangleIndex);
				SceneProxyMesh.Indices.Add(Tri[0]);
				SceneProxyMesh.Indices.Add(Tri[1]);
				SceneProxyMesh.Indices.Add(Tri[2]);
			}
		}
		
		SceneProxyMesh.Color = MeshColor;

		DebugRenderSceneProxy->Meshes.Add(SceneProxyMesh);
	}
	
}

void FDataflowDebugDraw::DrawBox(const FVector& Extents, const FQuat& Rotation, const FVector& Center, double UniformScale) const
{
	DebugRenderSceneProxy->Boxes.Add(FDebugRenderSceneProxy::FDebugBox(FBox(-Extents, Extents), ColorWithTranslucency.ToFColor(true), FTransform(Rotation, Center, FVector(UniformScale)), DrawType, LineWidth));
}

void FDataflowDebugDraw::DrawSphere(const FVector& Center, double Radius) const
{
	DebugRenderSceneProxy->Spheres.Add(FDebugRenderSceneProxy::FSphere(Radius, Center, ColorWithTranslucency, DrawType));
}

void FDataflowDebugDraw::DrawCapsule(const FVector& Center, const double& Radius, const double& HalfHeight, const FVector& XAxis, const FVector& YAxis, const FVector &ZAxis) const
{
	DebugRenderSceneProxy->Capsules.Add(FDebugRenderSceneProxy::FCapsule(Center, Radius, XAxis, YAxis, ZAxis, HalfHeight, ColorWithTranslucency, DrawType));
}

void FDataflowDebugDraw::DrawText3d(const FString& String, const FVector& Location) const
{
	DebugRenderSceneProxy->Texts.Add(FDebugRenderSceneProxy::FText3d(String, Location, Color));
}

void FDataflowDebugDraw::DrawOverlayText(const FString& InString)
{
	OverlayStrings.Add(InString);
}

FString FDataflowDebugDraw::GetOverlayText() const
{
	if (OverlayStrings.Num() > 0)
	{
		return OverlayStrings[0];
	}

	return {};
}

/* ----------------------------------------------------------------------------------------------------------------------- */

void FDataflowNodeDebugDrawSettings::SetDebugDrawSettings(IDataflowDebugDrawInterface& DataflowRenderingInterface) const
{
	DataflowRenderingInterface.SetWireframe(true);
	DataflowRenderingInterface.SetWorldPriority();
	DataflowRenderingInterface.SetLineWidth(LineWidthMultiplier);
	if (RenderType == EDataflowDebugDrawRenderType::Shaded)
	{
		DataflowRenderingInterface.SetShaded(true);
		DataflowRenderingInterface.SetTranslucent(bTranslucent);
		DataflowRenderingInterface.SetWireframe(true);
	}
	else
	{
		DataflowRenderingInterface.SetShaded(false);
		DataflowRenderingInterface.SetWireframe(true);
	}
	DataflowRenderingInterface.SetWorldPriority();
	DataflowRenderingInterface.SetColor(Color);
}


