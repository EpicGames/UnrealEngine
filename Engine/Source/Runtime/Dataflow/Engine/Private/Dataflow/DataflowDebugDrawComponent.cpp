// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowDebugDrawComponent.h"
#include "MeshElementCollector.h"
#include "PrimitiveDrawInterface.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowDebugDrawObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowDebugDrawComponent)

FDebugRenderSceneProxy* UDataflowDebugDrawComponent::CreateDebugSceneProxy()
{
	return new FDataflowDebugRenderSceneProxy(this);
}

FBoxSphereBounds UDataflowDebugDrawComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBoxSphereBounds::Builder BoundsBuilder;

	BoundsBuilder += LocalToWorld.GetLocation();

	if (FDataflowDebugRenderSceneProxy* DebugSceneProxy = static_cast<FDataflowDebugRenderSceneProxy*>(GetSceneProxy()))
	{	
		// Some of these bounds are pretty loose but whatever

		for (const FDebugRenderSceneProxy::FDebugLine& Line : DebugSceneProxy->Lines)
		{
			BoundsBuilder += Line.Start;
			BoundsBuilder += Line.End;
		}
		for (const FDebugRenderSceneProxy::FDashedLine& DashedLine : DebugSceneProxy->DashedLines)
		{
			BoundsBuilder += DashedLine.Start;
			BoundsBuilder += DashedLine.End;
		}
		for (const FDebugRenderSceneProxy::FArrowLine& ArrowLine : DebugSceneProxy->ArrowLines)
		{
			FBox ArrowBox(TArray<FVector>{ ArrowLine.Start, ArrowLine.End } );
			ArrowBox = ArrowBox.ExpandBy(ArrowLine.Mag);
			BoundsBuilder += ArrowBox;
		}
		for (const FDebugRenderSceneProxy::FCircle& Circle : DebugSceneProxy->Circles)
		{
			BoundsBuilder += FSphere(Circle.Center, Circle.Radius);
		}
		for (const FDebugRenderSceneProxy::FWireCylinder& Cyl : DebugSceneProxy->Cylinders)
		{
			BoundsBuilder += FSphere(Cyl.Base, FMath::Max(Cyl.Radius, Cyl.HalfHeight));
		}
		for (const FDebugRenderSceneProxy::FWireStar& Star : DebugSceneProxy->Stars)
		{
			BoundsBuilder += FSphere(Star.Position, Star.Size);
		}
		for (const FDebugRenderSceneProxy::FDebugBox& Box : DebugSceneProxy->Boxes)
		{
			BoundsBuilder += Box.Box.TransformBy(Box.Transform).ExpandBy(Box.Thickness);
		}
		for (const FDebugRenderSceneProxy::FSphere& Sphere : DebugSceneProxy->Spheres)
		{
			BoundsBuilder += FSphere(Sphere.Location, Sphere.Radius);
		}
		for (const FDebugRenderSceneProxy::FText3d& Text : DebugSceneProxy->Texts)
		{
			BoundsBuilder += FSphere(Text.Location, 100.0f);		// ??
		}
		for (const FDebugRenderSceneProxy::FCone& Cone : DebugSceneProxy->Cones)
		{
			FSphere ConeSphere(FVector(0,0,0), 1.0f);
			ConeSphere = ConeSphere.TransformBy(Cone.ConeToWorld);
			BoundsBuilder += ConeSphere;
		}
		for (const FDebugRenderSceneProxy::FMesh& Mesh : DebugSceneProxy->Meshes)
		{
			BoundsBuilder += Mesh.Box;
		}
		for (const FDebugRenderSceneProxy::FCapsule& Capsule : DebugSceneProxy->Capsules)
		{
			BoundsBuilder += FSphere(Capsule.Base, FMath::Max(Capsule.Radius, Capsule.HalfHeight));
		}
		for (const FDebugRenderSceneProxy::FCoordinateSystem& CoordinateSystem : DebugSceneProxy->CoordinateSystems)
		{
			BoundsBuilder += FSphere(CoordinateSystem.AxisLoc, CoordinateSystem.Scale + CoordinateSystem.Thickness);
		}
		for (const FDataflowDebugRenderSceneProxy::FDebugPoint& Point : DebugSceneProxy->Points)
		{
			BoundsBuilder += Point.Position;
		}
		for (const TRefCountPtr<IDataflowDebugDrawObject>& Object : DebugSceneProxy->Objects)
		{
			if(Object.IsValid() && Object->IsA(FDataflowDebugDrawBaseObject::StaticType()))
			{
				BoundsBuilder += static_cast<FDataflowDebugDrawBaseObject*>(Object.GetReference())->ComputeBoundingBox();
			}
		}
	}

	FBoxSphereBounds ReturnBounds(BoundsBuilder);
	ReturnBounds = ReturnBounds.ExpandBy(5);
	return ReturnBounds;

}


FDataflowDebugRenderSceneProxy::FDataflowDebugRenderSceneProxy(const UPrimitiveComponent* InComponent) :
	FDebugRenderSceneProxy(InComponent)
{
}

void FDataflowDebugRenderSceneProxy::ClearAll()
{
	Lines.Reset();
	DashedLines.Reset();
	ArrowLines.Reset();
	Circles.Reset();
	Cylinders.Reset();
	Stars.Reset();
	Boxes.Reset();
	Spheres.Reset();
	Texts.Reset();
	Cones.Reset();
	Meshes.Reset();
	Capsules.Reset();
	CoordinateSystems.Reset();
	Points.Reset();
	Objects.Reset();
}


FPrimitiveViewRelevance FDataflowDebugRenderSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View);
	Result.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	Result.bSeparateTranslucency = Result.bNormalTranslucency = true;
	return Result;
}

void FDataflowDebugRenderSceneProxy::ReservePoints(int32 NumAdditionalPoints)
{
	Points.Reserve(Points.Num() + NumAdditionalPoints);
}

void FDataflowDebugRenderSceneProxy::AddPoint(const FDebugPoint& Point)
{
	Points.Add(Point);
}

void FDataflowDebugRenderSceneProxy::AddObject(const TRefCountPtr<IDataflowDebugDrawObject>& Object)
{
	Objects.Add(Object);
}

void FDataflowDebugRenderSceneProxy::GetDynamicMeshElementsForView(const FSceneView* View, const int32 ViewIndex, const FSceneViewFamily& ViewFamily, const uint32 VisibilityMap, FMeshElementCollector& Collector, FMaterialCache& DefaultMaterialCache, FMaterialCache& SolidMeshMaterialCache) const
{
	FDebugRenderSceneProxy::GetDynamicMeshElementsForView(View, ViewIndex, ViewFamily, VisibilityMap, Collector, DefaultMaterialCache, SolidMeshMaterialCache);

	FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

	// Draw Points
	for (const FDebugPoint& Point : Points)
	{
		PDI->DrawPoint(Point.Position, Point.Color, Point.Size, Point.Priority);
	}

	// Draw Objects
	for (const TRefCountPtr<IDataflowDebugDrawObject>& Object : Objects)
	{
		if(Object.IsValid() && Object->IsA(FDataflowDebugDrawBaseObject::StaticType()))
		{
			static_cast<FDataflowDebugDrawBaseObject*>(Object.GetReference())->DrawDataflowElements(PDI);
		}
	}
}
