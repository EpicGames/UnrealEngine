// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowDynamicConnections.h"

#include "Dataflow/DataflowObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowDynamicConnections)

FDataflowDynamicConnections::FDataflowDynamicConnections()
	: PinDirection(UE::Dataflow::FPin::EDirection::NONE)
	, OwnerInterface(nullptr)
	, DataflowAssetWeakPtr(nullptr)
{}

FDataflowDynamicConnections::FDataflowDynamicConnections(UE::Dataflow::FPin::EDirection InPinDirection, IOwnerInterface* InOwnerInterface, UDataflow* InDataflowAsset)
	: PinDirection(InPinDirection)
	, OwnerInterface(InOwnerInterface)
	, DataflowAssetWeakPtr(InDataflowAsset)
{}


FDataflowDynamicConnections::FConnectionReference FDataflowDynamicConnections::GetConnectionReference(int32 Index) const
{
	return { &DynamicProperties[Index], Index, &DynamicProperties };
}

void FDataflowDynamicConnections::Refresh()
{
	using namespace UE::Dataflow;

	if (!OwnerInterface)
	{
		return;
	}

	FDataflowNode* OwnerNode = OwnerInterface->GetOwner(this);
	if (!OwnerNode)
	{
		return;
	}

	TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin();
	if (!DataflowAsset.IsValid())
	{
		return;
	}

	TSharedPtr<UE::Dataflow::FGraph, ESPMode::ThreadSafe> DataflowGraph = DataflowAsset->GetDataflow();
	if (!DataflowGraph.IsValid())
	{
		return;
	}

	// prevent re-entrance
	if (bIsRefreshing)
	{
		return;
	}
	bIsRefreshing = true;

	OwnerNode->PauseInvalidations();

	// save the remote connections from the dynamic connections
	using FRemoteConnections = TArray<FDataflowConnection*>;
	TMap<FGuid, FRemoteConnections> RemoteConnectionsByPropertyId;

	for (FDataflowConnection* DynamicConnection : GetDynamicConnections())
	{
		if (DynamicConnection)
		{
			const FGuid* PropertyId = ConnectionNameToPropertyId.Find(DynamicConnection->GetName());
			if (PropertyId)
			{
				FRemoteConnections& RemoteConnections = RemoteConnectionsByPropertyId.Emplace(*PropertyId);
				DynamicConnection->GetConnections(RemoteConnections);
				DataflowGraph->ClearConnections(DynamicConnection);
			}
		}
	}
	// clear all the dynamic connections
	// and first keep the name to id mapping in case we need it for the fallback of not finding the id from the new properties in the bag 
	TMap<FName, FGuid> PreviousConnectionNameToPropertyId = MoveTemp(ConnectionNameToPropertyId);
	ClearDynamicConnections();

	OwnerNode->ResumeInvalidations();

	// Go through all the properties of the property bag recreate the output and reconnect them if needed
	const TConnectionReference<FDataflowAllTypes> NullReference(nullptr);
	const FInstancedPropertyBag& PropertyBag = OwnerInterface->GetPropertyBag(this);
	if (const UPropertyBag* PropertyBagStruct = PropertyBag.GetPropertyBagStruct())
	{
		for (const FPropertyBagPropertyDesc& PropertyDesc : PropertyBagStruct->GetPropertyDescs())
		{
			// we can only add the output if it is a supported type
			if (IsSupportedType(PropertyDesc))
			{
				const FName CppType = GetCppTypeFromPropertyDesc(PropertyDesc);
				if (FDataflowConnection* NewConnection = AddNewConnectionFromPropertyDesc(PropertyDesc))
				{
					FRemoteConnections* RemoteConnections = RemoteConnectionsByPropertyId.Find(PropertyDesc.ID);
					if (!RemoteConnections)
					{
						// Ids may not match anymore ( internal logic of the property bag ) 
						// so let's fall back to the name
						if (const FGuid* PropertyId = PreviousConnectionNameToPropertyId.Find(NewConnection->GetName()))
						{
							RemoteConnections = RemoteConnectionsByPropertyId.Find(*PropertyId);
						}
					}

					// now try to reconnect what was connected before
					if (RemoteConnections)
					{
						for (FDataflowConnection* RemoteConnection : *RemoteConnections)
						{
							DataflowGraph->Connect(NewConnection, RemoteConnection);
						}
					}
				}
			}
		}
	}

	// Todo : need to also remove the one that are no longer necessary
	// ( need to disconnect if needed ? or keep the one that are still connected ?)
	OwnerNode->Invalidate();

	DataflowAsset->RefreshEdNodeByGuid(OwnerNode->GetGuid());

	bIsRefreshing = false;
}

FDataflowConnection* FDataflowDynamicConnections::AddNewConnectionFromPropertyDesc(const FPropertyBagPropertyDesc& PropertyDesc)
{
	using namespace UE::Dataflow;

	if (FDataflowNode* OwnerNode = OwnerInterface ? OwnerInterface->GetOwner(this) : nullptr)
	{
		const int32 Index = DynamicProperties.AddDefaulted();
		TConnectionReference<FDataflowAllTypes> ConnectionReference = GetConnectionReference(Index);
		if (FDataflowConnection* NewConnection = CreateConnection(ConnectionReference))
		{
			SetConnectionTypeFromPropertyDesc(*NewConnection, PropertyDesc);
			if (PinDirection == UE::Dataflow::FPin::EDirection::INPUT)
			{
				OwnerNode->TryRenameInput(ConnectionReference, PropertyDesc.Name);
			}
			else if (PinDirection == UE::Dataflow::FPin::EDirection::OUTPUT)
			{
				OwnerNode->TryRenameOutput(ConnectionReference, PropertyDesc.Name);
			}
			NewConnection->LockType();

			ConnectionNameToPropertyId.Add(NewConnection->GetName(), PropertyDesc.ID);
			return NewConnection;
		}
	}
	return nullptr;
}

bool FDataflowDynamicConnections::IsSupportedType(const FPropertyBagPropertyDesc& Desc)
{
	switch (Desc.ValueType)
	{
	case EPropertyBagPropertyType::Bool:
	case EPropertyBagPropertyType::Byte:
	case EPropertyBagPropertyType::Int32:
	case EPropertyBagPropertyType::Int64:
	case EPropertyBagPropertyType::Float:
	case EPropertyBagPropertyType::Double:
	case EPropertyBagPropertyType::Name:
	case EPropertyBagPropertyType::String:
	case EPropertyBagPropertyType::Text:
	case EPropertyBagPropertyType::Object:
	case EPropertyBagPropertyType::Struct:
		return true;

		// unsupported type for now 
	case EPropertyBagPropertyType::Enum:
	case EPropertyBagPropertyType::SoftObject:
	case EPropertyBagPropertyType::Class:
	case EPropertyBagPropertyType::SoftClass:
	case EPropertyBagPropertyType::UInt32:	// Type not fully supported at UI, will work with restrictions to type editing
	case EPropertyBagPropertyType::UInt64: // Type not fully supported at UI, will work with restrictions to type editing
	default:
		return false;
	}
}

FName FDataflowDynamicConnections::GetCppTypeFromPropertyDesc(const FPropertyBagPropertyDesc& Desc)
{
	return FDataflowConnection::GetTypeNameFromProperty(Desc.CachedProperty);
}

bool FDataflowDynamicConnections::SetConnectionTypeFromPropertyDesc(FDataflowConnection& Connection, const FPropertyBagPropertyDesc& Desc)
{
	if (FDataflowNode* OwnerNode = OwnerInterface ? OwnerInterface->GetOwner(this) : nullptr)
	{
		const FName PropertyCppType(GetCppTypeFromPropertyDesc(Desc));
		const bool bIsArray = (Desc.ContainerTypes.GetFirstContainerType() == EPropertyBagContainerType::Array);

		FName ConcreteType = NAME_None;
		switch (Desc.ValueType)
		{
		case EPropertyBagPropertyType::Double:
		case EPropertyBagPropertyType::Float:
			// the UI shows float but behind the scene set a double property let only deal with float 
			if (bIsArray)
			{
				return Connection.SetConcreteType(UE::Dataflow::GetTypeName<TArray<float>>());
			}
			return Connection.SetConcreteType(UE::Dataflow::GetTypeName<float>());

		case EPropertyBagPropertyType::Bool:
		case EPropertyBagPropertyType::Byte:
		case EPropertyBagPropertyType::Int32:
		case EPropertyBagPropertyType::Int64:
		case EPropertyBagPropertyType::Name:
		case EPropertyBagPropertyType::String:
		case EPropertyBagPropertyType::Text:
		case EPropertyBagPropertyType::Object:
		case EPropertyBagPropertyType::Struct:
			return Connection.SetConcreteType(PropertyCppType);

		// unsupported type for now 
		case EPropertyBagPropertyType::Enum:
		case EPropertyBagPropertyType::SoftObject:
		case EPropertyBagPropertyType::Class:
		case EPropertyBagPropertyType::SoftClass:
		case EPropertyBagPropertyType::UInt32:	// Type not fully supported at UI, will work with restrictions to type editing
		case EPropertyBagPropertyType::UInt64: // Type not fully supported at UI, will work with restrictions to type editing
		default:
			return false;
		}
	}
	return false;
}

FDataflowConnection* FDataflowDynamicConnections::CreateConnection(FDataflowDynamicConnections::FConnectionReference ConnectionReference)
{
	if (FDataflowNode* OwnerNode = OwnerInterface ? OwnerInterface->GetOwner(this) : nullptr)
	{
		if (PinDirection == UE::Dataflow::FPin::EDirection::INPUT)
		{
			return &OwnerNode->RegisterInputArrayConnection(ConnectionReference);
		}
		else if (PinDirection == UE::Dataflow::FPin::EDirection::OUTPUT)
		{
			return &OwnerNode->RegisterOutputArrayConnection(ConnectionReference);
		}
	}
	return nullptr;
}

TArray<FDataflowConnection*> FDataflowDynamicConnections::GetDynamicConnections() const
{
	TArray<FDataflowConnection*> Connections;
	if (FDataflowNode* OwnerNode = OwnerInterface ? OwnerInterface->GetOwner(this) : nullptr)
	{
		if (PinDirection == UE::Dataflow::FPin::EDirection::INPUT)
		{
			Connections.Append(OwnerNode->GetInputs());
		}
		else if (PinDirection == UE::Dataflow::FPin::EDirection::OUTPUT)
		{
			Connections.Append(OwnerNode->GetOutputs());
		}
	}
	return Connections;
}

void FDataflowDynamicConnections::ClearDynamicConnections()
{
	if (FDataflowNode* OwnerNode = OwnerInterface ? OwnerInterface->GetOwner(this) : nullptr)
	{
		for (int32 ConnectionIndex = 0; ConnectionIndex < DynamicProperties.Num(); ++ConnectionIndex)
		{
			if (PinDirection == UE::Dataflow::FPin::EDirection::INPUT)
			{
				OwnerNode->UnregisterInputConnection(GetConnectionReference(ConnectionIndex));
			}
			else if (PinDirection == UE::Dataflow::FPin::EDirection::OUTPUT)
			{
				OwnerNode->UnregisterOutputConnection(GetConnectionReference(ConnectionIndex));
			}
		}
	}
	ConnectionNameToPropertyId.Reset();
	DynamicProperties.Reset();
}
