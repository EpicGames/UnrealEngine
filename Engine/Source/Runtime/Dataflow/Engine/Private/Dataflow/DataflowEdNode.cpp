// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEdNode.h"

#include "Dataflow/DataflowArchive.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowCoreNodes.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Logging/LogMacros.h"
#include "Textures/SlateIcon.h"
#include "Styling/AppStyle.h"
#include "Misc/ScopeExit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowEdNode)

#if WITH_EDITOR
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UIAction.h"
#endif

#define LOCTEXT_NAMESPACE "DataflowEdNode"

DEFINE_LOG_CATEGORY_STATIC(DATAFLOWNODE_LOG, Error, All);

namespace UE::Dataflow::Private
{
	constexpr FLinearColor FrozenTitleColor(0.f, 0.7f, 1.f, 1.f);
	constexpr FLinearColor FrozenBodyTintColor(0.f, 0.7f, 1.f, 0.5f);

	static UE::Dataflow::FPin::EDirection EdPinDirectionToDataflowDirection(EEdGraphPinDirection EdDirection)
	{
		if (EdDirection == EEdGraphPinDirection::EGPD_Input)
		{
			return FPin::EDirection::INPUT;
		}
		if (EdDirection == EEdGraphPinDirection::EGPD_Output)
		{
			return FPin::EDirection::OUTPUT;
		}
		return FPin::EDirection::NONE;
	}

	static EEdGraphPinDirection DataflowDirectionToEdPinDirection(UE::Dataflow::FPin::EDirection Direction)
	{
		if (Direction == FPin::EDirection::INPUT)
		{
			return EEdGraphPinDirection::EGPD_Input;
		}
		if (Direction == FPin::EDirection::OUTPUT)
		{
			return EEdGraphPinDirection::EGPD_Output;
		}
		return EEdGraphPinDirection::EGPD_MAX;
	}
}

UDataflowEdNode::UDataflowEdNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bCanRenameNode = true;
#endif // WITH_EDITOR && !UE_BUILD_SHIPPING
}

void UDataflowEdNode::SetShouldRenderNode(bool bInRender)
{
	bRenderInAssetEditor = bInRender;
	if (IsBound())
	{
#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (UDataflow* DataflowObject = Cast< UDataflow>(GetGraph()))
		{
			if (bRenderInAssetEditor)
			{
				DataflowObject->AddRenderTarget(this);
			}
			else
			{
				DataflowObject->RemoveRenderTarget(this);
			}
		}
#endif
	}
}

void UDataflowEdNode::SetShouldWireframeRenderNode(bool bInRender)
{
	bRenderWireframeInAssetEditor = bInRender;
	if (IsBound())
	{
#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (UDataflow* const DataflowObject = UDataflow::GetDataflowAssetFromEdGraph(GetGraph()))
		{
			if (bRenderWireframeInAssetEditor)
			{
				DataflowObject->AddWireframeRenderTarget(this);
			}
			else
			{
				DataflowObject->RemoveWireframeRenderTarget(this);
			}
		}
#endif
	}
}

void UDataflowEdNode::SetCanEnableWireframeRenderNode(bool bInCanEnable)
{
	bCanEnableRenderWireframe = bInCanEnable;
}

bool UDataflowEdNode::CanEnableWireframeRenderNode() const
{
	return bCanEnableRenderWireframe;
}

TSharedPtr<FDataflowNode> UDataflowEdNode::GetDataflowNode()
{
	if(TSharedPtr<UE::Dataflow::FGraph> Dataflow = GetDataflowGraph())
	{
		return Dataflow->FindBaseNode(GetDataflowNodeGuid());
	}
	return TSharedPtr<FDataflowNode>(nullptr);
}

TSharedPtr<const FDataflowNode> UDataflowEdNode::GetDataflowNode() const
{
	if (TSharedPtr<const UE::Dataflow::FGraph> Dataflow = GetDataflowGraph())
	{
		return Dataflow->FindBaseNode(GetDataflowNodeGuid());
	}
	return TSharedPtr<FDataflowNode>(nullptr);
}

#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
UEdGraphPin* UDataflowEdNode::CreateEdPin(const UE::Dataflow::FPin& Pin)
{
	UEdGraphPin* const EdPin = CreatePin(UE::Dataflow::Private::DataflowDirectionToEdPinDirection(Pin.Direction), Pin.Type, Pin.Name);
	EdPin->bHidden = Pin.bHidden;
	if (FDataflowArrayTypePolicy::SupportsTypeStatic(Pin.Type))
	{
		EdPin->PinType.ContainerType = EPinContainerType::Array;
	}
	return EdPin;
}
#endif



void UDataflowEdNode::UpdatePinsFromDataflowNode()
{
	UE_LOGF(DATAFLOWNODE_LOG, Verbose, "UDataflowEdNode::UpdatePinsFromDataflowNode()");
	// called on node creation from UI. 

#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (DataflowGraph)
	{
		if (DataflowNodeGuid.IsValid())
		{
			if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(DataflowNodeGuid))
			{
				// remove pins that do not match inputs / outputs anymore
				TArray<UEdGraphPin*> PinsToRemove;
				for (UEdGraphPin* Pin : GetAllPins())
				{
					if (Pin)
					{
						if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
						{
							const FDataflowInput* DataflowInput = DataflowNode->FindInput(Pin->GetFName());
							if (!DataflowInput)
							{
								PinsToRemove.Add(Pin);
							}
							else if (DataflowInput->GetType() != Pin->PinType.PinCategory)
							{
								Pin->PinType = FEdGraphPinType();
								Pin->PinType.bIsReference = false;
								Pin->PinType.bIsConst = false;
								Pin->PinType.PinCategory = DataflowInput->GetType();
								Pin->PinType.PinSubCategory = NAME_None;
								Pin->PinType.PinSubCategoryObject = nullptr;
								if (FDataflowArrayTypePolicy::SupportsTypeStatic(DataflowInput->GetType()))
								{
									Pin->PinType.ContainerType = EPinContainerType::Array;
								}
							}
						}
						else if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
						{
							const FDataflowOutput* DataflowOutput = DataflowNode->FindOutput(Pin->GetFName());
							if (!DataflowOutput)
							{
								PinsToRemove.Add(Pin);
							}
							else if (DataflowOutput->GetType() != Pin->PinType.PinCategory)
							{
								Pin->PinType = FEdGraphPinType();
								Pin->PinType.bIsReference = false;
								Pin->PinType.bIsConst = false;
								Pin->PinType.PinCategory = DataflowOutput->GetType();
								Pin->PinType.PinSubCategory = NAME_None;
								Pin->PinType.PinSubCategoryObject = nullptr;
								if (FDataflowArrayTypePolicy::SupportsTypeStatic(DataflowOutput->GetType()))
								{
									Pin->PinType.ContainerType = EPinContainerType::Array;
								}
							}
						}
					}
				}
				for (UEdGraphPin* PinToRemove : PinsToRemove)
				{
					RemovePin(PinToRemove);
				}
				PinsToRemove.Reset();

				for (const UE::Dataflow::FPin& Pin : DataflowNode->GetPins())
				{
					const EEdGraphPinDirection EdDirection = UE::Dataflow::Private::DataflowDirectionToEdPinDirection(Pin.Direction);
					if (nullptr == FindPin(Pin.Name, EdDirection))
					{
						CreateEdPin(Pin);
					}
				}

				// reorder the pins 
				TArray<UEdGraphPin*> OrderedPins;
				OrderedPins.Reserve(GetAllPins().Num());
				for (const FDataflowInput* DataflowInput : DataflowNode->GetInputs())
				{
					if (DataflowInput)
					{
						if (UEdGraphPin* EdPin = FindPin(DataflowInput->GetName(), EEdGraphPinDirection::EGPD_Input))
						{
							OrderedPins.Add(EdPin);
						}
					}
				}
				for (const FDataflowOutput* DataflowOutput : DataflowNode->GetOutputs())
				{
					if (DataflowOutput)
					{
						if (UEdGraphPin* EdPin = FindPin(DataflowOutput->GetName(), EEdGraphPinDirection::EGPD_Output))
						{
							OrderedPins.Add(EdPin);
						}
					}
				}
				Pins = MoveTemp(OrderedPins);
			}
		}
	}

	UpdatePinsDefaultValuesFromNode();

#endif // WITH_EDITOR && !UE_BUILD_SHIPPING
}

void UDataflowEdNode::UpdatePinsConnectionsFromDataflowNode()
{
	UE_LOGF(DATAFLOWNODE_LOG, Verbose, "UDataflowEdNode::UpdatePinsConnectionsFromDataflowNode()");
	// called on node creation from UI. 

#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (DataflowGraph && DataflowNodeGuid.IsValid())
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(DataflowNodeGuid))
		{
			for (UEdGraphPin* Pin : GetAllPins())
			{
				if (ensure(Pin))
				{
					if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
					{
						if (const FDataflowInput* DataflowInput = DataflowNode->FindInput(Pin->GetFName()))
						{
							// first break all connection 
							Pin->BreakAllPinLinks();

							// then regenerate them 
							if (const FDataflowOutput* ConnectedDataflowOutput = DataflowInput->GetConnection())
							{
								const FGuid OutputOwnerGuid = ConnectedDataflowOutput->GetOwningNodeGuid();
								auto SearchByNodeGuidPredicate = [&OutputOwnerGuid](UEdGraphNode* EdNode)
									{
										if (UDataflowEdNode* DataflowEdnode = Cast<UDataflowEdNode>(EdNode))
										{
											return (DataflowEdnode->DataflowNodeGuid == OutputOwnerGuid);
										}
										return false;
									};

								if (TObjectPtr<UEdGraphNode>* EdNodeToConnect = GetGraph()->Nodes.FindByPredicate(SearchByNodeGuidPredicate))
								{
									if (UDataflowEdNode* EdDataflowNodeToConnect = Cast<UDataflowEdNode>(*EdNodeToConnect))
									{
										if (UEdGraphPin* PinToConnect = EdDataflowNodeToConnect->FindPin(ConnectedDataflowOutput->GetName(), EEdGraphPinDirection::EGPD_Output))
										{
											PinToConnect->MakeLinkTo(Pin);
											EdDataflowNodeToConnect->UpdatePinsFromDataflowNode();
											GetGraph()->NotifyNodeChanged(EdDataflowNodeToConnect);
										}
									}
								}
							}
						}
					}
					else if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
					{
						if (const FDataflowOutput* DataflowOutput = DataflowNode->FindOutput(Pin->GetFName()))
						{
							// first break all connection 
							Pin->BreakAllPinLinks();

							// then regenerate them 
							const TArray<const FDataflowInput*> ConnectedDataflowInputs = DataflowOutput->GetConnectedInputs();
							for (const FDataflowInput* ConnectedDataflowInput : ConnectedDataflowInputs)
							{
								if (ConnectedDataflowInput)
								{
									const FGuid InputOwnerGuid = ConnectedDataflowInput->GetOwningNodeGuid();
									auto SearchByNodeGuidPredicate = [&InputOwnerGuid](UEdGraphNode* EdNode)
										{
											if (UDataflowEdNode* DataflowEdnode = Cast<UDataflowEdNode>(EdNode))
											{
												return (DataflowEdnode->DataflowNodeGuid == InputOwnerGuid);
											}
											return false;
										};

									if (TObjectPtr<UEdGraphNode>* EdNodeToConnect = GetGraph()->Nodes.FindByPredicate(SearchByNodeGuidPredicate))
									{
										if (UDataflowEdNode* EdDataflowNodeToConnect = Cast<UDataflowEdNode>(*EdNodeToConnect))
										{
											if (UEdGraphPin* PinToConnect = EdDataflowNodeToConnect->FindPin(ConnectedDataflowInput->GetName(), EEdGraphPinDirection::EGPD_Input))
											{
												Pin->MakeLinkTo(PinToConnect);
												EdDataflowNodeToConnect->UpdatePinsFromDataflowNode();
												GetGraph()->NotifyNodeChanged(EdDataflowNodeToConnect);
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
#endif // WITH_EDITOR && !UE_BUILD_SHIPPING
}

#if WITH_EDITOR

void UDataflowEdNode::AllocateDefaultPins()
{
	UE_LOGF(DATAFLOWNODE_LOG, Verbose, "UDataflowEdNode::AllocateDefaultPins()");
	// called on node creation from UI. 

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (DataflowGraph)
	{
		if (DataflowNodeGuid.IsValid())
		{
			if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(DataflowNodeGuid))
			{
				for (const UE::Dataflow::FPin& Pin : DataflowNode->GetPins())
				{
					CreateEdPin(Pin);
				}
			}
		}
	}

	UpdatePinsDefaultValuesFromNode();
#endif // WITH_EDITOR && !UE_BUILD_SHIPPING
}

bool UDataflowEdNode::SupportsEditablePinType(const UEdGraphPin& Pin)
{
	if (const FDataflowConnection* Connection = GetConnectionFromPin(&Pin))
	{
		if (const FProperty* Property = Connection->GetProperty())
		{
			// We do not handle EditConditions
			static const FName Name_EditCondition("EditCondition");
			if (Property->HasMetaData(Name_EditCondition))
			{
				return false;
			}

			// only for direct properties of the node
			if (UStruct* OwnerStruct = Property->GetOwnerStruct())
			{
				if (!OwnerStruct->IsChildOf<FDataflowNode>())
				{
					return false;
				}
			}
		}

		// todo(dataflow): Add support for anytype in the future 
		if (!Connection->IsAnyType())
		{
			const FName Pintype = Pin.PinType.PinCategory;
			if (Pin.PinType.IsArray())
			{
				return false;
			}
			return (Pintype == TDataflowPolicyTypeName<bool>::GetName())
				|| (Pintype == TDataflowPolicyTypeName<int>::GetName())
				|| (Pintype == TDataflowPolicyTypeName<float>::GetName())
				|| (Pintype == TDataflowPolicyTypeName<FString>::GetName())
				;
		}
	}
	return false;
}

TSharedPtr<UE::Dataflow::FGraph> UDataflowEdNode::GetDataflowGraphFromPin(const UEdGraphPin& Pin)
{
	if (UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(Pin.GetOwningNode()))
	{
		return EdNode->DataflowGraph;
	}
	return {};
}

TSharedPtr<const FDataflowNode> UDataflowEdNode::GetDataflowNodeFromPin(const UEdGraphPin& Pin)
{
	if (UDataflowEdNode* EdNode = Cast<UDataflowEdNode>(Pin.GetOwningNode()))
	{
		return EdNode->GetDataflowNode();
	}
	return {};
}

const FDataflowInput* UDataflowEdNode::GetDataflowInputFromPin(const UEdGraphPin& Pin)
{
	if (TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNodeFromPin(Pin))
	{
		if (Pin.Direction == EEdGraphPinDirection::EGPD_Input)
		{
			return DataflowNode->FindInput(Pin.PinName);
		}
	}
	return nullptr;
}

const FDataflowOutput* UDataflowEdNode::GetDataflowOutputFromPin(const UEdGraphPin& Pin)
{
	if (TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNodeFromPin(Pin))
	{
		if (Pin.Direction == EEdGraphPinDirection::EGPD_Output)
		{
			return DataflowNode->FindOutput(Pin.PinName);
		}
	}
	return nullptr;
}

bool UDataflowEdNode::CanConnectPins(const UEdGraphPin& OutputPin, const UEdGraphPin& InputPin)
{
	ensure(OutputPin.Direction == EEdGraphPinDirection::EGPD_Output && InputPin.Direction == EEdGraphPinDirection::EGPD_Input);

	const TSharedPtr<UE::Dataflow::FGraph> Graph = GetDataflowGraphFromPin(OutputPin);
	const FDataflowOutput* Output = GetDataflowOutputFromPin(OutputPin);
	const FDataflowInput* Input = GetDataflowInputFromPin(InputPin);
	if (Graph && Output && Input)
	{
		return Graph->CanConnect(*Output, *Input);
	}
	return false;
}

bool UDataflowEdNode::CanConvertPins(const UEdGraphPin& OutputPin, const UEdGraphPin& InputPin)
{
	ensure(OutputPin.Direction == EEdGraphPinDirection::EGPD_Output && InputPin.Direction == EEdGraphPinDirection::EGPD_Input);

	const FDataflowOutput* Output = GetDataflowOutputFromPin(OutputPin);
	const FDataflowInput* Input = GetDataflowInputFromPin(InputPin);
	if (Output && Input)
	{
		const FName AutoConvertNodeName = UE::Dataflow::FAnyTypesRegistry::GetAutoConvertNodeTypeStatic(Output->GetType(), Input->GetType());
		return AutoConvertNodeName != NAME_None;
	}
	return false;
}
#endif // WITH_EDITOR

void UDataflowEdNode::UpdatePinsDefaultValuesFromNode()
{
#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!DataflowGraph || !DataflowNodeGuid.IsValid())
	{
		return;
	}
	TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(DataflowNodeGuid);
	if (!DataflowNode)
	{
		return;
	}

	if (bEditablePinReentranceGuard)
	{
		return; // 
	}

	bEditablePinReentranceGuard = true;

	for (UEdGraphPin* Pin : GetAllPins())
	{
		if (ensure(Pin) && Pin->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			if (SupportsEditablePinType(*Pin))
			{
				if (const FDataflowInput* DataflowInput = DataflowNode->FindInput(Pin->GetFName()))
				{
					if (const FProperty* Property = DataflowInput->GetProperty())
					{
						Pin->DefaultValue.Empty();
						const void* SourceValue = DataflowInput->RealAddress();
						//const void* SourceValue = Property->ContainerPtrToValuePtr<void>(DataflowNode.Get());
						Property->ExportText_Direct(Pin->DefaultValue, SourceValue, SourceValue, nullptr, PPF_None);
					}
				}
			}
		}
	}

	bEditablePinReentranceGuard = false;
#endif
}

void UDataflowEdNode::AddOptionPin()
{
#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (DataflowGraph && DataflowNodeGuid.IsValid())
	{
		// Modify();  // TODO: How do we modify a DataflowNode

		if (const TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(DataflowNodeGuid))
		{
			const TArray<UE::Dataflow::FPin> AddedPins = DataflowNode->AddPins();
			for (const UE::Dataflow::FPin& Pin : AddedPins)
			{
				switch (Pin.Direction)
				{
				case UE::Dataflow::FPin::EDirection::INPUT:
				case UE::Dataflow::FPin::EDirection::OUTPUT:
					CreateEdPin(Pin);
					ReconstructNode();
					break;
				default:
					break;  // Add pin isn't implemented on this node
				}
			}
		}

		// Refresh the current graph, so the pins can be updated
		if (UEdGraph* const ParentGraph = GetGraph())
		{
			ParentGraph->NotifyGraphChanged();
		}
	}
#endif // WITH_EDITOR && !UE_BUILD_SHIPPING
}

void UDataflowEdNode::RemoveOptionPin()
{
#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (DataflowGraph && DataflowNodeGuid.IsValid())
	{
		// Modify();  // TODO: How do we modify a DataflowNode

		if (const TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(DataflowNodeGuid))
		{
			const TArray<UE::Dataflow::FPin> RemovePins = DataflowNode->GetPinsToRemove();
			UE::Dataflow::FDataflowNodePauseInvalidationScope PauseInvalidationScope(DataflowNode.Get()); // Don't call invalidations per pin. Nodes may not evaluate correctly until all pins have been removed.
			for (const UE::Dataflow::FPin& Pin : RemovePins)
			{
				switch (Pin.Direction)
				{
				case UE::Dataflow::FPin::EDirection::INPUT:
				case UE::Dataflow::FPin::EDirection::OUTPUT:
					if (UEdGraphPin* const EdPin = FindPin(Pin.Name, UE::Dataflow::Private::DataflowDirectionToEdPinDirection(Pin.Direction)))
					{
						constexpr bool bNotifyNodes = true;
						EdPin->BreakAllPinLinks(bNotifyNodes);
						RemovePin(EdPin);
						ReconstructNode();
					}
					break;
				default:
					break;  // Add pin isn't implemented on this node
				}
			}
		}

		// Refresh the current graph, so the pins can be updated
		if (UEdGraph* const ParentGraph = GetGraph())
		{
			ParentGraph->NotifyGraphChanged();
		}
	}
#endif // WITH_EDITOR && !UE_BUILD_SHIPPING
}

bool UDataflowEdNode::PinIsCompatibleWithType(const UEdGraphPin& Pin, const FEdGraphPinType& PinType) const
{
#if WITH_EDITOR
	check(Pin.GetOwningNode() == this);
	if (TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
	{
		if (Pin.Direction == EEdGraphPinDirection::EGPD_Input)
		{
			return DataflowNode->InputSupportsType(Pin.GetFName(), PinType.PinCategory);
		}
		if (Pin.Direction == EEdGraphPinDirection::EGPD_Output)
		{
			return DataflowNode->OutputSupportsType(Pin.GetFName(), PinType.PinCategory);
		}
	}
#endif // WITH_EDITOR
	return false;
}

bool UDataflowEdNode::HasAnyWatchedConnection() const
{
	return (WatchedConnections.Num() > 0);
}

bool UDataflowEdNode::IsConnectionWatched(const FDataflowConnection& Connection) const
{
	return WatchedConnections.Contains(Connection.GetGuid());
}

void UDataflowEdNode::WatchConnection(const FDataflowConnection& Connection, bool Value)
{
	if (Value)
	{
		WatchedConnections.AddUnique(Connection.GetGuid());
	}
	else
	{
		WatchedConnections.Remove(Connection.GetGuid());
	}
}

bool UDataflowEdNode::IsPinWatched(const UEdGraphPin* Pin) const
{
	if (FDataflowConnection* Connection = UDataflowEdNode::GetConnectionFromPin(Pin))
	{
		return IsConnectionWatched(*Connection);
	}
	return false;
}

void UDataflowEdNode::WatchPin(const UEdGraphPin* Pin, bool bWatch)
{
	if (FDataflowConnection* Connection = UDataflowEdNode::GetConnectionFromPin(Pin))
	{
		WatchConnection(*Connection, bWatch);
	}
}



#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void UDataflowEdNode::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (ensure(IsBound()))
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(DataflowNodeGuid))
		{
			check(Pin);

			// Return whether a pin in the UEdGraph matches the specified connection in the Dataflow::FGraph
			auto MatchesConnection = [this](const UEdGraphPin* const Pin, const FDataflowConnection* const Connection) -> bool
				{
					if (const UDataflowEdNode* const LinkedNode = Cast<UDataflowEdNode>(Pin->GetOwningNode()))
					{
						if (ensure(LinkedNode->IsBound()))
						{
							if (const TSharedPtr<FDataflowNode> LinkedDataflowNode = DataflowGraph->FindBaseNode(LinkedNode->GetDataflowNodeGuid()))
							{
								return
									(Pin->Direction == EEdGraphPinDirection::EGPD_Input &&
										Connection == LinkedDataflowNode->FindInput(FName(Pin->GetName()))) ||
									(Pin->Direction == EEdGraphPinDirection::EGPD_Output &&
										Connection == LinkedDataflowNode->FindOutput(FName(Pin->GetName())));
							}
						}
					}
					return false;
				};

			if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
			{
				if (FDataflowInput* const ConnectionInput = DataflowNode->FindInput(FName(Pin->GetName())))
				{
					// Add any newly added connections
					for (const UEdGraphPin* const LinkedCon : Pin->LinkedTo)
					{
						UDataflowEdNode* const LinkedNode = Cast<UDataflowEdNode>(LinkedCon->GetOwningNode());
						if (ensure(LinkedNode && LinkedNode->IsBound()))
						{
							if (const TSharedPtr<FDataflowNode> LinkedDataflowNode = DataflowGraph->FindBaseNode(LinkedNode->GetDataflowNodeGuid()))
							{
								if (FDataflowOutput* const LinkedConOutput = LinkedDataflowNode->FindOutput(FName(LinkedCon->GetName())))
								{
									if (ConnectionInput->GetConnectedOutputs().Find(LinkedConOutput) == INDEX_NONE)
									{
										DataflowGraph->Connect(LinkedConOutput, ConnectionInput);
										UpdatePinsFromDataflowNode();
										LinkedNode->UpdatePinsFromDataflowNode();
									}
								}
							}
						}
					}
					// Clear any defunct connection
					if (FDataflowOutput* const ConnectedOutput = ConnectionInput->GetConnection())
					{
						if (!Pin->LinkedTo.FindByPredicate(
							[this, ConnectedOutput, &MatchesConnection](const UEdGraphPin* const LinkedCon) -> bool
							{
								return MatchesConnection(LinkedCon, ConnectedOutput);
							}))
						{
							DataflowGraph->Disconnect(ConnectedOutput, ConnectionInput);
						}
					}
				}
			}
			else if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
			{
				// Update newly added connections from the input pins
				for (UEdGraphPin* const LinkedPin : Pin->LinkedTo)
				{
					LinkedPin->GetOwningNode()->PinConnectionListChanged(LinkedPin);
				}

				// Remove any remaining defunct connections
				if (FDataflowOutput* const ConnectionOutput = DataflowNode->FindOutput(FName(Pin->GetName())))
				{
					const TArray<FDataflowInput*> InputsToDisconnect =
						ConnectionOutput->GetConnections().FilterByPredicate(
							[this, &Pin, &MatchesConnection](FDataflowInput* const ConnectedInput) -> bool
							{
								return ensure(ConnectedInput) && !Pin->LinkedTo.FindByPredicate(
									[this, ConnectedInput, &MatchesConnection](const UEdGraphPin* const LinkedCon) -> bool
									{
										return MatchesConnection(LinkedCon, ConnectedInput);
									});
							});
					UDataflow* const DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(GetGraph());
					for (FDataflowInput* const ConnectedInput : InputsToDisconnect)
					{
						const TObjectPtr<UDataflowEdNode> InputEdNode = DataflowAsset->FindEdNodeByDataflowNodeGuid(ConnectedInput->GetOwningNodeGuid());
						if (ensure(InputEdNode))
						{
							UEdGraphPin* const InputPin = InputEdNode->FindPin(ConnectedInput->GetName(), EEdGraphPinDirection::EGPD_Input);
							if (ensure(InputPin))
							{
								// To avoid double invalidations, instead of disconnecting, update connections by calling PinConnectionListChanged on the input pin
								// This means PinConnectionListChanged might be called twice on the input, with the second time resulting in no actions
								InputEdNode->PinConnectionListChanged(InputPin);
							}
						}
					}
				}
			}
		}
	}

	Super::PinConnectionListChanged(Pin);
}

#endif // WITH_EDITOR && !UE_BUILD_SHIPPING

void UDataflowEdNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << DataflowNodeGuid;
#if WITH_EDITOR
	if (Ar.IsTransacting())
	{
		const UDataflow* const DataflowObject = UDataflow::GetDataflowAssetFromEdGraph(GetGraph());
		bool bCanSerializeNode = !DataflowObject || DataflowObject->IsPerNodeTransactionSerializationEnabled();

		TSharedPtr<FDataflowNode> DataflowNode = GetDataflowNode();

		// the DataflowNode may not be always valid so we need to serialize that part first so that Svaing and Loading behave exactly the same way 
		const bool bNodeSerializable = bCanSerializeNode && DataflowNode.IsValid();
		if (Ar.IsLoading())
		{
			DATAFLOW_OPTIONAL_BLOCK_READ_BEGIN(bNodeSerializable)
			{
				DataflowNode->SerializeInternal(Ar);
			}
			DATAFLOW_OPTIONAL_BLOCK_READ_ELSE()
			DATAFLOW_OPTIONAL_BLOCK_READ_END()
		}
		else if (Ar.IsSaving())
		{
			DATAFLOW_OPTIONAL_BLOCK_WRITE_BEGIN()
			{
				if (bNodeSerializable)
				{
					DataflowNode->SerializeInternal(Ar);
				}
			}
			DATAFLOW_OPTIONAL_BLOCK_WRITE_END()
		}
	}
	// some double level template type have unwanted spaces 
	// now that the dataflow connection are no longer having spaces in their type name we need to fix that up on the pins
	RemoveSpacesInAllPinTypes();
#endif
}

void UDataflowEdNode::RemoveSpacesInAllPinTypes()
{
#if WITH_EDITOR
	for (UEdGraphPin* Pin : this->GetAllPins())
	{
		if (Pin)
		{
			FString PinTypeString{ Pin->PinType.PinCategory.ToString() };
			if (PinTypeString.Contains(TEXT(" ")))
			{
				PinTypeString.RemoveSpacesInline();
				Pin->PinType.PinCategory = FName(PinTypeString);
			}
		}
	}
#endif
}

#if WITH_EDITOR

FText UDataflowEdNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString(GetName());
}

/*static*/ FSlateIcon UDataflowEdNode::GetIconFromNode(const FDataflowNode& DataflowNode)
{
	FSlateIcon Icon;
	if (const FString* IconName = DataflowNode.TypedScriptStruct()->FindMetaData("Icon"))
	{
		FName StyleSheetName = FAppStyle::GetAppStyleSetName();
		if (const FString* StyleName = DataflowNode.TypedScriptStruct()->FindMetaData("Style"))
		{
			StyleSheetName = FName(*StyleName);
		}
		Icon = FSlateIcon(StyleSheetName, FName(*IconName));
	}
	else
	{
		// no icon defined by the node look for one in the category
		Icon = UE::Dataflow::FIconCategoryRegistry::Get().GetCategoryIcon(DataflowNode.GetCategory());
	}
	return Icon;
}

FSlateIcon UDataflowEdNode::GetIconAndTint(FLinearColor& OutColor) const
{
	FSlateIcon Icon;
	if (TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
	{
		Icon = UDataflowEdNode::GetIconFromNode(*DataflowNode);
	}
	return Icon;
}

bool UDataflowEdNode::ShowPaletteIconOnNode() const
{
	return true;
}

FLinearColor UDataflowEdNode::GetNodeTitleColor() const
{
	if (const TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
	{
		if (DataflowNode->IsFrozen())
		{
			return UE::Dataflow::Private::FrozenTitleColor;
		}
		else if (DataflowNode->IsColorOverriden())
		{
			return DataflowNode->GetOverrideColor();
		}
		return UE::Dataflow::FNodeColorsRegistry::Get().GetNodeTitleColor(DataflowNode->GetCategory());
	}
	return FDataflowNode::DefaultNodeTitleColor;
}

FLinearColor UDataflowEdNode::GetNodeBodyTintColor() const
{
	if (const TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
	{
		if (DataflowNode->IsFrozen())
		{
			return UE::Dataflow::Private::FrozenBodyTintColor;
		}
		return UE::Dataflow::FNodeColorsRegistry::Get().GetNodeBodyTintColor(DataflowNode->GetCategory());
	}
	return FDataflowNode::DefaultNodeBodyTintColor;
}

FText UDataflowEdNode::GetTooltipText() const
{
	if (const TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode()) 
	{
		return FText::FromString(DataflowNode->GetToolTip());
	}

	return FText::FromString("");

}

FText UDataflowEdNode::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	if (Pin)
	{
		if (const TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
		{
			const FText DisplayName = DataflowNode->GetPinDisplayName(Pin->PinName, UE::Dataflow::Private::EdPinDirectionToDataflowDirection(Pin->Direction));
			if (!DisplayName.IsEmpty())
			{
				return DisplayName;
			}
		}
	}
	return Super::GetPinDisplayName(Pin);
}

void UDataflowEdNode::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	if (const TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
	{
		const UE::Dataflow::FPin::EDirection PinDirection = UE::Dataflow::Private::EdPinDirectionToDataflowDirection(Pin.Direction);

		FString MetaDataStr;
		TArray<FString> PinMetaData = DataflowNode->GetPinMetaData(Pin.PinName, PinDirection);
			
		if (Pin.Direction == EGPD_Input && PinMetaData.Contains(FDataflowNode::DataflowIntrinsic.ToString()))
		{
			MetaDataStr = "[Intrinsic]";
		}
		if (Pin.Direction == EGPD_Output && PinMetaData.Contains(FDataflowNode::DataflowPassthrough.ToString()))
		{
			MetaDataStr = "[Passthrough]";
		}

		FString NameStr = Pin.PinName.ToString();
		if (MetaDataStr.Len() > 0)
		{
			NameStr.Appendf(TEXT(" %s"), *MetaDataStr);
		}

		// find type information 
		FString TypeNameStr = Pin.PinType.PinCategory.ToString();

		const FDataflowConnection* Connection = nullptr;
		if (Pin.Direction == EGPD_Input)
		{
			Connection = DataflowNode->FindInput(Pin.PinName);
		}
		else if (Pin.Direction == EGPD_Output)
		{
			Connection = DataflowNode->FindOutput(Pin.PinName);
		}
		if (Connection)
		{
			TypeNameStr = Connection->GetPropertyTypeNameTooltip();
		}

		const FString PropertyTooltip = DataflowNode->GetPinToolTip(Pin.PinName, PinDirection);

		// put it all together 
		if (PropertyTooltip.IsEmpty())
		{
			HoverTextOut.Appendf(TEXT("%s\n%s"), *NameStr, *TypeNameStr);
		}
		else
		{
			HoverTextOut.Appendf(TEXT("%s\n%s\n\n%s"), *NameStr, *TypeNameStr, *PropertyTooltip);
		}
		
}
}

void UDataflowEdNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	const UEdGraph* EdGraph = this->GetGraph();
	if (EdGraph == nullptr)
	{
		return;
	}

	if (!DataflowGraph || !FromPin)
	{
		return;
	}

	if (UEdGraphNode* FromGraphNode = FromPin->GetOwningNode())
	{
		if (FromPin->Direction == EEdGraphPinDirection::EGPD_Output)
		{
			for (UEdGraphPin* InputPin : this->GetAllPins())
			{
				if (InputPin && InputPin->Direction == EEdGraphPinDirection::EGPD_Input)
				{
					if (UDataflowEdNode::CanConnectPins(*FromPin, *InputPin))
					{
						if (EdGraph->GetSchema()->TryCreateConnection(FromPin, InputPin))
						{
							FromGraphNode->NodeConnectionListChanged();
							this->NodeConnectionListChanged();
							return;
						}
					}
				}
			}
		}
		if (FromPin->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			for (UEdGraphPin* OutputPin : this->GetAllPins())
			{
				if (OutputPin && OutputPin->Direction == EEdGraphPinDirection::EGPD_Output)
				{
					if (UDataflowEdNode::CanConnectPins(*OutputPin, *FromPin))
					{
						if (EdGraph->GetSchema()->TryCreateConnection(FromPin, OutputPin))
						{
							FromGraphNode->NodeConnectionListChanged();
							this->NodeConnectionListChanged();
							return;
						}
					}
				}
			}
		}
	}
}

void UDataflowEdNode::OnPinRemoved(UEdGraphPin* InRemovedPin)
{
	if (DataflowGraph && DataflowNodeGuid.IsValid())
	{
		if (const TSharedPtr<FDataflowNode> DataflowNode = DataflowGraph->FindBaseNode(DataflowNodeGuid))
		{
			if (InRemovedPin->Direction == EEdGraphPinDirection::EGPD_Input)
			{
				if (FDataflowInput* Con = DataflowNode->FindInput(FName(InRemovedPin->GetName())))
				{
					const UE::Dataflow::FPin Pin = { UE::Dataflow::FPin::EDirection::INPUT, Con->GetType(), Con->GetName() };
					DataflowNode->OnPinRemoved(Pin);
					DataflowNode->UnregisterPinConnection(Pin);
				}
			}
			else if (InRemovedPin->Direction == EEdGraphPinDirection::EGPD_Output)
			{
				if (FDataflowOutput* Con = DataflowNode->FindOutput(FName(InRemovedPin->GetName())))
				{
					const UE::Dataflow::FPin Pin = { UE::Dataflow::FPin::EDirection::OUTPUT, Con->GetType(), Con->GetName() };
					DataflowNode->OnPinRemoved(Pin);
					DataflowNode->UnregisterPinConnection(Pin);
				}
			}
		}
	}
}

bool UDataflowEdNode::ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const
{
	UEdGraphNode::ShouldDrawNodeAsControlPointOnly(OutInputPinIndex, OutOutputPinIndex);
	if (const TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
	{
		if (DataflowNode->GetType() == FDataflowReRouteNode::StaticType())
		{
			OutInputPinIndex = 0;
			OutOutputPinIndex = 1;
			return true;
		}
	}
	return false;
}

void UDataflowEdNode::PostEditUndo()
{
	Super::PostEditUndo();

	UnRegisterDelegateHandle();

	// Refresh the current graph, so the pins or whatever happened to this object can be reflected to the graph
	if (UEdGraph* const ParentGraph = GetGraph())
	{
		if (IsValid(ParentGraph) && ParentGraph->Schema)
		{
			ParentGraph->NotifyGraphChanged();
		}
	}

	// Make sure to re-sync the Dataflow connections
	for (UEdGraphPin* const Pin : GetAllPins())
	{
		PinConnectionListChanged(Pin);
	}

	RegisterDelegateHandle();

	ScheduleNodeInvalidation();
}

void UDataflowEdNode::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	if (bEditablePinReentranceGuard)
	{
		return; // we are setting the values from UpdatePinsDefaultValuesFromNode
	}

	if (Pin && SupportsEditablePinType(*Pin))
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = GetDataflowNode())
		{
			if (const FDataflowInput* Input = DataflowNode->FindInput(Pin->PinName))
			{
				if (FProperty* Property = const_cast<FProperty*>(Input->GetProperty()))
				{
					bEditablePinReentranceGuard = true;
					ON_SCOPE_EXIT{ bEditablePinReentranceGuard = false; };

					const FName PropertyType = FDataflowConnection::GetTypeNameFromProperty(Property);
					const FString PinStringValue = Pin->GetDefaultAsString();
					if (PropertyType == Pin->PinType.PinCategory)
					{
						//void* TargetAddress = Property->ContainerPtrToValuePtr<void>(DataflowNode.Get());
						void* TargetAddress = Input->RealAddress();
						if (PropertyType == TDataflowPolicyTypeName<bool>::GetName())
						{
							const bool bValue = PinStringValue.ToBool();
							Property->CopyCompleteValue(TargetAddress, &bValue);
						}
						else if (PropertyType == TDataflowPolicyTypeName<int>::GetName())
						{
							const int32 IntValue = (PinStringValue.IsNumeric()) ? FCString::Atoi(*PinStringValue) : 0;
							Property->CopyCompleteValue(TargetAddress, &IntValue);
						}
						else if (PropertyType == TDataflowPolicyTypeName<float>::GetName())
						{
							const float FloatValue = (PinStringValue.IsNumeric()) ? FCString::Atof(*PinStringValue) : 0;
							Property->CopyCompleteValue(TargetAddress, &FloatValue);
						}
						else if (PropertyType == TDataflowPolicyTypeName<FString>::GetName())
						{
							Property->CopyCompleteValue(TargetAddress, &PinStringValue);
						}

						// we cannot invalidate the node right away because invalidation cause delegate to invalidate the pin 
						// we are already receiving a notification from causing ensure in the EdGraphPin code 
						// so we schedule an invalidation for later 
						ScheduleNodeInvalidation();
					}
				}
			}
		}
	}
}

void UDataflowEdNode::ScheduleNodeInvalidation()
{
	if (TSharedPtr<FDataflowNode> DataflowNode = GetDataflowNode())
	{
		AsyncTask(ENamedThreads::GameThread, [WeakNode = DataflowNode.ToWeakPtr()]()
			{
				if (TSharedPtr<FDataflowNode> DataflowNode = WeakNode.Pin())
				{
					DataflowNode->Invalidate();
				}
			});
	}
}

void UDataflowEdNode::HideAllInputPins()
{
	bool bAnyHidden = false;
	if (TSharedPtr<FDataflowNode> DataflowNode = GetDataflowNode())
	{
		TArray<FDataflowInput*> Inputs = DataflowNode->GetInputs();
		for (FDataflowInput* const Input : Inputs)
		{
			if (Input->GetCanHidePin() && !Input->GetPinIsHidden())
			{
				Input->SetPinIsHidden(true);
				if (!bAnyHidden)
				{
					Modify();
					bAnyHidden = true;
				}
				UEdGraphPin* const EdPin = FindPin(Input->GetName(), EEdGraphPinDirection::EGPD_Input);
				check(EdPin);
				EdPin->Modify();
				EdPin->bHidden = true;
			}
		}
	}

	if (bAnyHidden)
	{
		GetGraph()->NotifyGraphChanged();
	}
}

void UDataflowEdNode::ShowAllInputPins()
{
	bool bAnyUnhidden = false;
	if (TSharedPtr<FDataflowNode> DataflowNode = GetDataflowNode())
	{
		TArray<FDataflowInput*> Inputs = DataflowNode->GetInputs();
		for (FDataflowInput* const Input : Inputs)
		{
			if (Input->GetCanHidePin() && Input->GetPinIsHidden())
			{
				Input->SetPinIsHidden(false);
				if (!bAnyUnhidden)
				{
					Modify();
					bAnyUnhidden = true;
				}
				UEdGraphPin* const EdPin = FindPin(Input->GetName(), EEdGraphPinDirection::EGPD_Input);
				check(EdPin);
				EdPin->Modify();
				EdPin->bHidden = false;
			}
		}
	}

	if (bAnyUnhidden)
	{
		GetGraph()->NotifyGraphChanged();
	}
}

void UDataflowEdNode::ToggleHideInputPin(FName PinName)
{
	if (TSharedPtr<FDataflowNode> DataflowNode = GetDataflowNode())
	{
		if (FDataflowInput* const Input = DataflowNode->FindInput(PinName))
		{
			if (ensure(Input->GetCanHidePin()))
			{
				const bool bWasHidden = Input->GetPinIsHidden();
				Input->SetPinIsHidden(!bWasHidden);
				Modify();
				UEdGraphPin* const EdPin = FindPin(Input->GetName(), EEdGraphPinDirection::EGPD_Input);
				check(EdPin);
				EdPin->Modify();
				EdPin->bHidden = !bWasHidden;
				GetGraph()->NotifyGraphChanged();
			}
		}
	}
}

bool UDataflowEdNode::CanToggleHideInputPin(FName PinName) const
{
	if (TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
	{
		if (const FDataflowInput* const Input = DataflowNode->FindInput(PinName))
		{
			if (Input->GetCanHidePin() && !Input->HasAnyConnections())
			{
				return true;
			}
		}
	}
	return false;
}

bool UDataflowEdNode::IsInputPinShown(FName PinName) const
{
	if (TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
	{
		if (const FDataflowInput* const Input = DataflowNode->FindInput(PinName))
		{
			return !Input->GetPinIsHidden();
		}
	}
	return false;
}
#endif //WITH_EDITOR


TArray<UE::Dataflow::FRenderingParameter> UDataflowEdNode::GetRenderParameters() const
{
	if (TSharedPtr<const FDataflowNode> DataflowNode = GetDataflowNode())
	{
		return DataflowNode->GetRenderParameters();
	}
	return 	TArray<UE::Dataflow::FRenderingParameter>();
}

void UDataflowEdNode::RegisterDelegateHandle()
{
#if WITH_EDITOR
	ensure(!OnNodeInvalidatedDelegateHandle.IsValid());
	OnNodeInvalidatedDelegateHandle = FDataflowNodeDelegates::OnNodeInvalidated.AddWeakLambda(this, [this](FDataflowNode& InDataflowNode)
		{
			if (IsInGameThread() || IsInSlateThread())
			{
				if (InDataflowNode.GetGuid() == GetDataflowNodeGuid())
				{
					if (UEdGraph* EdGraph = GetGraph())
					{
						EdGraph->NotifyNodeChanged(this);
						if (UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(EdGraph))
						{
							FDataflowAssetDelegates::OnNodeInvalidated.Broadcast(*DataflowAsset, InDataflowNode);
						}
					}
				}
			}
		});
#endif
}

void UDataflowEdNode::UnRegisterDelegateHandle()
{
#if WITH_EDITOR
	if (OnNodeInvalidatedDelegateHandle.IsValid())
	{
		FDataflowNodeDelegates::OnNodeInvalidated.Remove(OnNodeInvalidatedDelegateHandle);
		OnNodeInvalidatedDelegateHandle.Reset();
	}
#endif
}

void UDataflowEdNode::SetDataflowNodeGuid(FGuid InGuid)
{
	UnRegisterDelegateHandle();

	DataflowNodeGuid = InGuid;

	RegisterDelegateHandle();
}

TSharedPtr<FDataflowNode> UDataflowEdNode::GetDataflowNodeFromEdNode(UEdGraphNode* EdNode)
{
	if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
	{
		return DataflowEdNode->GetDataflowNode();
	}
	return {};
}

TSharedPtr<const FDataflowNode> UDataflowEdNode::GetDataflowNodeFromEdNode(const UEdGraphNode* EdNode)
{
	if (const UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
	{
		return DataflowEdNode->GetDataflowNode();
	}
	return {};
}

FDataflowConnection* UDataflowEdNode::GetConnectionFromPin(const UEdGraphPin* Pin)
{
#if WITH_EDITOR
	FDataflowConnection* DataflowConnection = nullptr;
	if (Pin)
	{
		if (TSharedPtr<FDataflowNode> DataflowNode = GetDataflowNodeFromEdNode(Pin->GetOwningNode()))
		{
			if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
			{
				DataflowConnection = DataflowNode->FindInput(Pin->PinName);
			}
			else if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
			{
				DataflowConnection = DataflowNode->FindOutput(Pin->PinName);
			}
		}
	}
	return DataflowConnection;
#else
	ensure(false); // should always be called in editor
	return nullptr;
#endif
}

#undef LOCTEXT_NAMESPACE

