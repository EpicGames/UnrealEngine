// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "DataflowEngineLogPrivate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "Dataflow/DataflowTextureAssetNodes.h"
#include "Dataflow/DataflowAssetPathNodes.h"
#include "Dataflow/DataflowVariableNodes.h"
#include "Dataflow/DataflowSubGraphNodes.h"
#include "Dataflow/DataflowSkeletonNodes.h"

/**
 * The public interface to this module
 */
class FDataflowEngineModule : public IModuleInterface
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		UE::Dataflow::RegisterTextureAssetNodes();
		UE::Dataflow::RegisterAssetPathNodes();
		UE::Dataflow::RegisterVariableNodes();
		UE::Dataflow::RegisterSubGraphNodes();
		UE::Dataflow::RegisterSkeletonNodes();
	}

	virtual void ShutdownModule() override { }

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
};

IMPLEMENT_MODULE( FDataflowEngineModule, DataflowEngine )



