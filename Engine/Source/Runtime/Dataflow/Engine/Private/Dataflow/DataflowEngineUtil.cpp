// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowEngineUtil.h"

#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowConnection.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "ReferenceSkeleton.h"
#include "UObject/UnrealType.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

namespace UE::Dataflow
{

	namespace Animation
	{
		void GlobalTransformsInternal(int32 Index, const FReferenceSkeleton& Ref, TArray<FTransform>& Mat, TArray<bool>& Visited)
		{
			if (!Visited[Index])
			{
				const TArray<FTransform>& RefMat = Ref.GetRefBonePose();

				int32 ParentIndex = Ref.GetParentIndex(Index);
				if (ParentIndex != INDEX_NONE && ParentIndex != Index) // why self check?
				{
					GlobalTransformsInternal(ParentIndex, Ref, Mat, Visited);
					Mat[Index].SetFromMatrix(RefMat[Index].ToMatrixWithScale() * Mat[ParentIndex].ToMatrixWithScale());
				}
				else
				{
					Mat[Index] = RefMat[Index];
				}

				Visited[Index] = true;
			}
		}

		void GlobalTransforms(const FReferenceSkeleton& Ref, TArray<FTransform>& Mat)
		{
			TArray<bool> Visited;
			Visited.Init(false, Ref.GetNum());
			Mat.SetNum(Ref.GetNum());

			int32 Index = Ref.GetNum() - 1;
			while (Index >= 0)
			{
				GlobalTransformsInternal(Index, Ref, Mat, Visited);
				Index--;
			}
		}
	}

	namespace Color
	{
		FLinearColor GetRandomColor(const int32 RandomSeed, int32 Idx)
		{
			FRandomStream RandomStream(RandomSeed * 7 + Idx * 41);

			const uint8 R = static_cast<uint8>(RandomStream.FRandRange(128, 255));
			const uint8 G = static_cast<uint8>(RandomStream.FRandRange(128, 255));
			const uint8 B = static_cast<uint8>(RandomStream.FRandRange(128, 255));

			return FLinearColor(FColor(R, G, B, 255));
		}
	}

	namespace Type
	{
		FPropertyBagPropertyDesc GetPropertyBagPropertyDescFromDataflowType(const FName Name, const FName Type)
		{
			if (Type == UE::Dataflow::GetTypeName<bool>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Bool);
			}
			else if (Type == UE::Dataflow::GetTypeName<int32>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Int32);
			}
			else if (Type == UE::Dataflow::GetTypeName<int64>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Int64);
			}
			else if (Type == UE::Dataflow::GetTypeName<uint32>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::UInt32);
			}
			else if (Type == UE::Dataflow::GetTypeName<uint64>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::UInt64);
			}
			else if (Type == UE::Dataflow::GetTypeName<float>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Float);
			}
			else if (Type == UE::Dataflow::GetTypeName<double>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Double);
			}
			else if (Type == UE::Dataflow::GetTypeName<FName>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Name);
			}
			else if (Type == UE::Dataflow::GetTypeName<FString>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::String);
			}
			else if (Type == UE::Dataflow::GetTypeName<FText>())
			{
				return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Text);
			}
			else
			{
				const FString TypeStr(Type.ToString());

				// Make sure to remove the TObjectPtr
				FString ObjectPtrInnerType;
				if (FDataflowUObjectConvertibleTypePolicy::GetObjectPtrInnerType(TypeStr, ObjectPtrInnerType))
				{
					if (const UClass* ObjectClass = FindFirstObjectSafe<UClass>(*ObjectPtrInnerType))
					{
						return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Object, ObjectClass);
					}
				}
				if (TypeStr.StartsWith("U"))
				{
					const FString ShortTypeName = TypeStr.RightChop(1);
					if (const UClass* ObjectClass = FindFirstObjectSafe<UClass>(*ShortTypeName))
					{
						return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Object, ObjectClass);
					}
				}
				else if (TypeStr.StartsWith("F"))
				{
					const FString ShortTypeName = TypeStr.RightChop(1);
					if (const UScriptStruct* ScriptStruct = FindFirstObjectSafe<UScriptStruct>(*ShortTypeName))
					{
						return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Struct, ScriptStruct);
					}
				}
				else if (FDataflowArrayTypePolicy::SupportsTypeStatic(Type))
				{
					const FName ElementType = FDataflowArrayTypePolicy::GetElementType(Type);
					FPropertyBagPropertyDesc PropertyDesc = GetPropertyBagPropertyDescFromDataflowType(Name, ElementType);
					PropertyDesc.ContainerTypes = FPropertyBagContainerTypes{ EPropertyBagContainerType::Array };
					return PropertyDesc;
				}
			}
			// invalid value 
			return FPropertyBagPropertyDesc(Name, EPropertyBagPropertyType::Count);
		}

		FPropertyBagPropertyDesc GetPropertyBagPropertyDescFromDataflowConnection(const FDataflowConnection& Connection)
		{
			const FProperty* ConnectionProperty = Connection.GetProperty();
			check(ConnectionProperty);

			// AnyType require special treatment to make sure we give a concrete type
			if (Connection.IsAnyType())
			{
				if (Connection.HasConcreteType())
				{
					const FPropertyBagPropertyDesc Desc = GetPropertyBagPropertyDescFromDataflowType(Connection.GetName(), Connection.GetType());
					if (Desc.ValueType != EPropertyBagPropertyType::Count)
					{
						return Desc;
					}
				}
				// Fallback let's use the AnyType default (Value) type
				if (ConnectionProperty->GetClass()->IsChildOf(FStructProperty::StaticClass()))
				{
					if (const FStructProperty* StructProperty = CastField<FStructProperty>(ConnectionProperty))
					{
						if (const bool bInheritsFromAnyType = StructProperty->Struct->IsChildOf<FDataflowAnyType>())
						{
							for (TFieldIterator<FProperty> FieldIt(StructProperty->Struct); FieldIt; ++FieldIt)
							{
								FProperty* Prop = *FieldIt;
								if (Prop->GetFName() == "Value")
								{
									return FPropertyBagPropertyDesc(Connection.GetName(), Prop);
								}
							}
						}
					}
				}
			}
			// default simply set property desc from the property
			return FPropertyBagPropertyDesc(Connection.GetName(), Connection.GetProperty());
		}

		FName MakeUniqueNameForPropertyBag(FName OriginalName, const FInstancedPropertyBag& PropertyBag)
		{
			FName NewName = OriginalName;
			int32 SuffixNumber = 1;
			while (PropertyBag.FindPropertyDescByName(NewName))
			{
				NewName.SetNumber(SuffixNumber++);
			}
			return NewName;
		}
	}

	namespace RenderGeometry
	{
		UStaticMesh* GetDataflowBox()
		{
			const FStringView Path = TEXT("/Engine/EditorMeshes/Dataflow/SM_Dataflow_Box.SM_Dataflow_Box");

			return LoadObject<UStaticMesh>(nullptr, Path);
		}

		UStaticMesh* GetDataflowSphere()
		{
			const FStringView Path = TEXT("/Engine/EditorMeshes/Dataflow/SM_Dataflow_Sphere.SM_Dataflow_Sphere");
		
			return LoadObject<UStaticMesh>(nullptr, Path);
		}

		UStaticMesh* GetDataflowPlane()
		{
			const FStringView Path = TEXT("/Engine/EditorMeshes/Dataflow/SM_Dataflow_Plane.SM_Dataflow_Plane");

			return LoadObject<UStaticMesh>(nullptr, Path);
		}
	}

	namespace RenderMaterial
	{
		UMaterialInterface* GetDataflowColorMaterial(const bool bTwoSided)
		{
			const FStringView Path = bTwoSided
				? TEXT("/Engine/EditorMaterials/Dataflow/M_Dataflow_Color_TwoSided.M_Dataflow_Color_TwoSided")
				: TEXT("/Engine/EditorMaterials/Dataflow/M_Dataflow_Color.M_Dataflow_Color");

			return LoadObject<UMaterialInterface>(nullptr, Path);
		}

		UMaterialInterface* GetDataflowColorWireframeMaterial(const bool bTwoSided)
		{
			const FStringView Path = bTwoSided
				? TEXT("/Engine/EditorMaterials/Dataflow/M_Dataflow_Color_Wireframe_TwoSided.M_Dataflow_Color_Wireframe_TwoSided")
				: TEXT("/Engine/EditorMaterials/Dataflow/M_Dataflow_Color_Wireframe.M_Dataflow_Color_Wireframe");

			return LoadObject<UMaterialInterface>(nullptr, Path);
		}

		UMaterialInterface* GetDataflowCheckerBoardMaterial(const bool bUseVertexColor)
		{
			const FStringView Path = bUseVertexColor
				? TEXT("/Engine/EditorMaterials/Dataflow/M_Dataflow_Vertex_CheckerBoard.M_Dataflow_Vertex_CheckerBoard")
				: TEXT("/Engine/EditorMaterials/Dataflow/M_Dataflow_CheckerBoard.M_Dataflow_CheckerBoard");

			return LoadObject<UMaterialInterface>(nullptr, Path);
		}

		UMaterialInterface* GetDataflowVertexColorMaterial()
		{
			const FStringView Path = TEXT("/Engine/EditorMaterials/Dataflow/DataflowVertexMaterial.DataflowVertexMaterial");

			return LoadObject<UMaterialInterface>(nullptr, Path);
		}

		UMaterialInterface* GetDataflowPointsMaterial()
		{
			const FStringView Path = TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial");

			return LoadObject<UMaterialInterface>(nullptr, Path);
		}

		UMaterialInterface* GetDataflowLinesMaterial()
		{
			const FStringView Path = TEXT("/MeshModelingToolsetExp/Materials/LineSetOverlaidComponentMaterial");

			return LoadObject<UMaterialInterface>(nullptr, Path);
		}
	}

	namespace Mesh
	{
		bool GetMeshVertices(const UE::Geometry::FDynamicMesh3* InMeshPtr, TArray<FVector3f>& OutVertices)
		{
			OutVertices.Reset();

			if (InMeshPtr)
			{
				const int32 NumVertices = InMeshPtr->VertexCount();

				OutVertices.Reserve(NumVertices);

				for (const int32 VertexID : InMeshPtr->VertexIndicesItr())
				{
					if (InMeshPtr->IsVertex(VertexID))
					{
						OutVertices.Add((FVector3f)InMeshPtr->GetVertex(VertexID));
					}
				}

				return true;
			}

			return false;
		}

		bool GetMeshVertexNormals(const UE::Geometry::FDynamicMesh3* InMeshPtr, TArray<FVector3f>& OutVertexNormals)
		{
			OutVertexNormals.Reset();

			if (InMeshPtr)
			{
				const int32 NumVertices = InMeshPtr->VertexCount();

				OutVertexNormals.Reserve(NumVertices);

				if (InMeshPtr->HasVertexNormals())
				{
					for (const int32 VertexId : InMeshPtr->VertexIndicesItr())
					{
						if (InMeshPtr->IsVertex(VertexId))
						{
							OutVertexNormals.Add(InMeshPtr->GetVertexNormal(VertexId));
						}
					}
				}
				else if (InMeshPtr->HasAttributes() && InMeshPtr->Attributes()->PrimaryNormals())
				{
					const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay = InMeshPtr->Attributes()->PrimaryNormals();

					TArray<int> OverlayElements;
					for (const int32 VertexId : InMeshPtr->VertexIndicesItr())
					{
						if (InMeshPtr->IsVertex(VertexId))
						{
							OverlayElements.Reset();
							NormalOverlay->GetVertexElements(VertexId, OverlayElements);

							FVector3f AvgNormal(0.0f, 0.0f, 0.0f);
							if (OverlayElements.Num() > 0)
							{
								for (int32 ElementID : OverlayElements)
								{
									AvgNormal += NormalOverlay->GetElement(ElementID);
								}
								AvgNormal /= (float)OverlayElements.Num();

								OutVertexNormals.Add(AvgNormal);
							}
						}
					}
				}

				return true;
			}

			return false;
		}

		bool GetMeshUVs(const UE::Geometry::FDynamicMesh3* InDynMeshPtr, TArray<FVector2f>& OutUVs, const int32 InUVLayer)
		{
			if (InDynMeshPtr && InDynMeshPtr->HasAttributes() && InDynMeshPtr->Attributes()->NumUVLayers() > 0)
			{
				OutUVs.Init(FVector2f::ZeroVector, InDynMeshPtr->MaxVertexID()); // RenderFacade->DynamicMesh code expects all vertices to have full UV sets.

				int32 NumUVLayers = InDynMeshPtr->Attributes()->NumUVLayers();

				if (InUVLayer >= NumUVLayers)
				{
					return false;
				}

				TArray<const UE::Geometry::FDynamicMeshUVOverlay*> UVLayers;
				UVLayers.SetNum(NumUVLayers);

				for (int32 Layer = 0; Layer < NumUVLayers; ++Layer)
				{
					UVLayers[Layer] = InDynMeshPtr->Attributes()->GetUVLayer(Layer);
				}

				for (const int32 TriangleIndex : InDynMeshPtr->TriangleIndicesItr())
				{
					const UE::Geometry::FIndex3i Tri = InDynMeshPtr->GetTriangle(TriangleIndex);

					for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
					{
						const int32 VertexIndex = Tri[TriangleVertexIndex];
						if (InDynMeshPtr->IsVertex(VertexIndex))
						{
							OutUVs[VertexIndex] = UVLayers[InUVLayer]->GetElementAtVertex(TriangleIndex, VertexIndex);
						}
					}
				}

				return true;
			}

			return false;
		}
	}
}
