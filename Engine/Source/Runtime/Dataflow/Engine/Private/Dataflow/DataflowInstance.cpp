// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowInstance.h"

#include "Dataflow/DataflowAttachment.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Interfaces/Interface_AssetUserData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowInstance)

namespace UE::Dataflow::InstanceUtils
{
	static const FName DataflowAssetPropertyName("DataflowAsset");
	static const FName DataflowTerminalPropertyName("DataflowTerminal");

	UDataflowAttachment* GetDataflowAttachment(const UObject* InAsset)
	{
		if (const IInterface_AssetUserData* UserDataInterface = Cast<IInterface_AssetUserData>(InAsset))
		{
			// The asset user interface does not provide a const API, we need to work around it 
			IInterface_AssetUserData* MutableUserDataInterface = const_cast<IInterface_AssetUserData*>(UserDataInterface);
			return MutableUserDataInterface->GetAssetUserData<UDataflowAttachment>();
		}
		return nullptr;
	}

	void AddDataflowAttachmentIfNeeded(UObject* InAsset)
	{
		if (const IDataflowInstanceInterface* DataflowInstanceInterface = Cast<IDataflowInstanceInterface>(InAsset))
		{
			// no attachement needed
			return;
		}

		if (IInterface_AssetUserData* UserDataInterface = Cast<IInterface_AssetUserData>(InAsset))
		{
			if (!UserDataInterface->HasAssetUserDataOfClass(UDataflowAttachment::StaticClass()))
			{
				if (UDataflowAttachment* Attachement = FDataflowAttachmentFactory::Get().Create(InAsset))
				{
					UserDataInterface->AddAssetUserData(Attachement);
				}
			}
		}
	}

	UDataflowAttachment* GetDataflowAttachmentFromObject(const UObject* Obj)
	{
		return GetDataflowAttachment(Obj);
	}

	IDataflowInstanceInterface* GetDataflowInterfaceFromObject(UObject* Obj)
	{
		// is the owning asset itself implement the dataflow instance interface
		if (IDataflowInstanceInterface* Interface = Cast<IDataflowInstanceInterface>(Obj))
		{
			return Interface;
		}
		// or is that using a dataflow attachement ? 
		if (UDataflowAttachment* Attachment = GetDataflowAttachment(Obj))
		{
			return Attachment;
		}
		return nullptr;
	}

	const IDataflowInstanceInterface* GetDataflowInterfaceFromObject(const UObject* Obj)
	{
		// is the owning asset itself implement the dataflow instance interface
		if (const IDataflowInstanceInterface* Interface = Cast<const IDataflowInstanceInterface>(Obj))
		{
			return Interface;
		}
		// or is that using a dataflow attachement ? 
		if (const UDataflowAttachment* Attachment = GetDataflowAttachment(Obj))
		{
			return Attachment;
		}
		return nullptr;
	}

	bool HasValidDataflowAsset(UObject* Obj)
	{
		if (Obj)
		{
			// first check if the object implements IDataflowGeneratedAssetInterface
			if (const IDataflowInstanceInterface* Interface = GetDataflowInterfaceFromObject(Obj))
			{
				return true;
			}
			// Legacy - check if the class contains a dataflow asset and terminal node name properties
			if (UClass* Class = Obj->GetClass())
			{
				return Class->FindPropertyByName(DataflowAssetPropertyName)
					&& Class->FindPropertyByName(DataflowTerminalPropertyName);
			}
		}
		return false;

	}

	template<typename TDataflowType, typename TObjectType>
	TDataflowType* GetDataflowAssetFromObjectTemplate(TObjectType* Obj)
	{
		// first check if the object is itself the dataflow asset
		TDataflowType* DataflowObject = Cast<TDataflowType>(Obj);

		if (!DataflowObject)
		{
			// check if the object implements IDataflowGeneratedAssetInterface
			if (const IDataflowInstanceInterface* Interface = GetDataflowInterfaceFromObject(Obj))
			{
				DataflowObject = Interface->GetDataflowInstance().GetDataflowAsset();
			}
		}

		// legacy checks for properties
		if (!DataflowObject && Obj)
		{
			// last search if the object has a property named DataflowAsset
			// @TODO(dataflow)  we should retire this code path eventually in favor of using the interface solution 
			if (const UClass* Class = Obj->GetClass())
			{
				if (FProperty* Property = Class->FindPropertyByName(DataflowAssetPropertyName))
				{
					DataflowObject = *Property->ContainerPtrToValuePtr<TDataflowType*>(Obj);
				}
			}
		}
		return DataflowObject;
	}

	UDataflow* GetDataflowAssetFromObject(UObject* Obj)
	{
		return GetDataflowAssetFromObjectTemplate<UDataflow, UObject>(Obj);
	}

	const UDataflow* GetDataflowAssetFromObject(const UObject* Obj)
	{
		return GetDataflowAssetFromObjectTemplate<const UDataflow, const UObject>(Obj);
	}

	bool SetDataflowAssetOnObject(UDataflow* DataflowAsset, UObject* Obj)
	{
		AddDataflowAttachmentIfNeeded(Obj);

		// check if the object implements IDataflowGeneratedAssetInterface
		if (IDataflowInstanceInterface* Interface = GetDataflowInterfaceFromObject(Obj))
		{
			Interface->GetDataflowInstance().SetDataflowAsset(DataflowAsset);
			return true;
		}
		return false;
	}

	FName GetTerminalNodeNameFromObject(UObject* Obj)
	{
		if (Obj)
		{
			// check if the object implements IDataflowGeneratedAssetInterface
			if (IDataflowInstanceInterface* Interface = GetDataflowInterfaceFromObject(Obj))
			{
				return Interface->GetDataflowInstance().GetDataflowTerminal();
			}

			// last search if the object has a property named DataflowTerminal
			// @TODO(dataflow)  we should retire this code path eventually in favor of using the interface solution 
			if (UClass* Class = Obj->GetClass())
			{
				if (FProperty* Property = Class->FindPropertyByName(DataflowTerminalPropertyName))
				{
					return *Property->ContainerPtrToValuePtr<FName>(Obj);
				}
			}
		}
		return NAME_None;
	}

	TArray<FName> GetTerminalNodeNames(const UDataflow* DataflowAsset)
	{
		TArray<FName> TerminalNodeNames;
		if (DataflowAsset)
		{
			if (const TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = DataflowAsset->Dataflow)
			{
				for (const TSharedPtr<FDataflowNode>& Node : DataflowGraph->GetNodes())
				{
					if (Node->AsType<FDataflowTerminalNode>())
					{
						TerminalNodeNames.Emplace(Node->GetName());
					}
				}
			}
		}
		return TerminalNodeNames;
	}

	void GetAssetRegistryTags(const UObject* BoundAsset, FAssetRegistryTagsContext& Context)
	{
		const UDataflow* DataflowObject = UE::Dataflow::InstanceUtils::GetDataflowAssetFromObject(BoundAsset);

		const bool bHasDataflow = (DataflowObject != nullptr);
		Context.AddTag(UObject::FAssetRegistryTag(TEXT("HasDataflow"), bHasDataflow ? TEXT("True") : TEXT("False"), UObject::FAssetRegistryTag::TT_Alphabetical));

		const bool bIsDataflowEmbedded = DataflowObject ? (!DataflowObject->IsAsset()) : false;
		Context.AddTag(UObject::FAssetRegistryTag(TEXT("IsDataflowEmbedded"), bIsDataflowEmbedded ? TEXT("True") : TEXT("False"), UObject::FAssetRegistryTag::TT_Alphabetical));
	}

}


//---------------------------------------------------------------------------
// FDataflowVariableOverrides
//---------------------------------------------------------------------------
FDataflowVariableOverrides::FDataflowVariableOverrides(FDataflowInstance* InOwner)
	: Owner(InOwner)
{
}

FDataflowVariableOverrides& FDataflowVariableOverrides::operator=(const FDataflowVariableOverrides& Other)
{
	if (this != &Other)
	{
		Variables = Other.Variables;

		// transfer the overriden GUIDs that may have changed with the copy 
		OverriddenVariableGuids.Empty();
		for (const FGuid OtherOverrideGuid : Other.OverriddenVariableGuids)
		{
			if (const FPropertyBagPropertyDesc* Desc = Other.Variables.FindPropertyDescByID(OtherOverrideGuid))
			{
				OverriddenVariableGuids.Add(Desc->ID);
			}
		}

		// IMPORTANT : do not copy the owner 
	}

	return *this;
}

void FDataflowVariableOverrides::SetVariableOverrideAndNotify(const FGuid& PropertyID, bool bOverrideState)
{
	if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByID(PropertyID))
	{
		if (bOverrideState)
		{
			OverriddenVariableGuids.AddUnique(Desc->ID);
		}
		else
		{
			OverriddenVariableGuids.Remove(Desc->ID);
		}
		if (FDataflowAssetDelegates::OnVariablesOverrideStateChanged.IsBound())
		{
			const UDataflow* DataflowAsset = Owner ? Owner->GetDataflowAsset() : nullptr;
			FDataflowAssetDelegates::OnVariablesOverrideStateChanged.Broadcast(DataflowAsset, Desc->Name, bOverrideState);
		}
	}
}

bool FDataflowVariableOverrides::OverrideVariableBool(FName VariableName, bool bValue)
{
	if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByName(VariableName))
	{
		if (Variables.SetValueBool(*Desc, bValue) == EPropertyBagResult::Success)
		{
			SetVariableOverrideAndNotify(Desc->ID, true);
			return true;
		}
	}
	return false;
}

bool FDataflowVariableOverrides::OverrideVariableBoolArray(FName VariableName, const TArray<bool>& Values)
{
	using namespace UE::Dataflow::InstanceUtils::Private;

	auto WriteBoolValue = [](FPropertyBagArrayRef& ArrayRef, int32 Idx, bool Value)
		{
			return ArrayRef.SetValueBool(Idx, Value);
		};

	return OverrideVariableArrayAndNotify(Variables, VariableName, Values, WriteBoolValue);
}

bool FDataflowVariableOverrides::OverrideVariableInt(FName VariableName, int64 Value)
{
	if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByName(VariableName))
	{
		if (Variables.SetValueInt64(*Desc, Value) == EPropertyBagResult::Success)
		{
			SetVariableOverrideAndNotify(Desc->ID, true);
			return true;
		}
	}
	return false;
}

bool FDataflowVariableOverrides::OverrideVariableInt32Array(FName VariableName, const TArray<int32>& Values)
{
	using namespace UE::Dataflow::InstanceUtils::Private;

	auto WriteInt32Value = [](FPropertyBagArrayRef& ArrayRef, int32 Idx, int32 Value)
		{
			return ArrayRef.SetValueInt32(Idx, Value);
		};

	return OverrideVariableArrayAndNotify(Variables, VariableName, Values, WriteInt32Value);
}

bool FDataflowVariableOverrides::OverrideVariableInt64Array(FName VariableName, const TArray<int64>& Values)
{
	using namespace UE::Dataflow::InstanceUtils::Private;

	auto WriteInt64Value = [](FPropertyBagArrayRef& ArrayRef, int32 Idx, int64 Value)
		{
			return ArrayRef.SetValueInt64(Idx, Value);
		};

	return OverrideVariableArrayAndNotify(Variables, VariableName, Values, WriteInt64Value);
}

bool FDataflowVariableOverrides::OverrideVariableFloat(FName VariableName, float Value)
{
	if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByName(VariableName))
	{
		if (Variables.SetValueFloat(*Desc, Value) == EPropertyBagResult::Success)
		{
			SetVariableOverrideAndNotify(Desc->ID, true);
			return true;
		}
	}
	return false;
}

bool FDataflowVariableOverrides::OverrideVariableFloatArray(FName VariableName, const TArray<float>& Values)
{
	using namespace UE::Dataflow::InstanceUtils::Private;

	auto WriteFloatValue = [](FPropertyBagArrayRef& ArrayRef, int32 Idx, float Value)
		{
			return ArrayRef.SetValueFloat(Idx, Value);
		};

	return OverrideVariableArrayAndNotify(Variables, VariableName, Values, WriteFloatValue);
}

bool FDataflowVariableOverrides::OverrideVariableObject(FName VariableName, const UObject* Value)
{
	if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByName(VariableName))
	{
		if (Variables.SetValueObject(*Desc, Value) == EPropertyBagResult::Success)
		{
			SetVariableOverrideAndNotify(Desc->ID, true);
			return true;
		}
	}
	return false;
}

bool FDataflowVariableOverrides::OverrideVariableObjectArray(FName VariableName, const TArray<TObjectPtr<UObject>>& Values)
{
	using namespace UE::Dataflow::InstanceUtils::Private;

	auto WriteObjectValue = [](FPropertyBagArrayRef& ArrayRef, int32 Idx, const TObjectPtr<UObject>& Value)
		{
			return ArrayRef.SetValueObject(Idx, Value);
		};

	return OverrideVariableArrayAndNotify(Variables, VariableName, Values, WriteObjectValue);
}

bool FDataflowVariableOverrides::OverrideVariableObjectArray(FName VariableName, const TArray<UObject*>& Values)
{
	using namespace UE::Dataflow::InstanceUtils::Private;

	auto WriteObjectValue = [](FPropertyBagArrayRef& ArrayRef, int32 Idx, UObject* Value)
		{
			return ArrayRef.SetValueObject(Idx, Value);
		};

	return OverrideVariableArrayAndNotify(Variables, VariableName, Values, WriteObjectValue);
}

bool FDataflowVariableOverrides::OverrideVariableName(FName VariableName, FName Value)
{
	if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByName(VariableName))
	{
		if (Variables.SetValueName(*Desc, Value) == EPropertyBagResult::Success)
		{
			SetVariableOverrideAndNotify(Desc->ID, true);
			return true;
		}
	}
	return false;
}

bool FDataflowVariableOverrides::OverrideVariableName(FName VariableName, const TArray<FName>& Values)
{
	using namespace UE::Dataflow::InstanceUtils::Private;

	auto WriteNameValue = [](FPropertyBagArrayRef& ArrayRef, int32 Idx, const FName& Value)
		{
			return ArrayRef.SetValueName(Idx, Value);
		};

	return OverrideVariableArrayAndNotify(Variables, VariableName, Values, WriteNameValue);
}

bool FDataflowVariableOverrides::OverrideVariableString(FName VariableName, FString Value)
{
	if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByName(VariableName))
	{
		if (Variables.SetValueString(*Desc, Value) == EPropertyBagResult::Success)
		{
			SetVariableOverrideAndNotify(Desc->ID, true);
			return true;
		}
	}
	return false;
}

bool FDataflowVariableOverrides::OverrideVariableString(FName VariableName, const TArray<FString>& Values)
{
	using namespace UE::Dataflow::InstanceUtils::Private;

	auto WriteStringValue = [](FPropertyBagArrayRef& ArrayRef, int32 Idx, const FString& Value)
		{
			return ArrayRef.SetValueString(Idx, Value);
		};

	return OverrideVariableArrayAndNotify(Variables, VariableName, Values, WriteStringValue);
}

const FInstancedPropertyBag* FDataflowVariableOverrides::GetDefaultVariablesFromAsset() const 
{
	if (Owner)
	{
		if (const UDataflow* DataflowAsset = Owner->GetDataflowAsset())
		{
			return &DataflowAsset->Variables;
		}
	}
	return nullptr;
}

FInstancedPropertyBag& FDataflowVariableOverrides::GetOverridenVariables()
{
	return Variables;
}

void FDataflowVariableOverrides::RemoveAllVariables()
{
	Variables.Reset();
	OverriddenVariableGuids.Reset();
}

void FDataflowVariableOverrides::SyncVariables()
{
	if (const FInstancedPropertyBag* DefaultVariables = GetDefaultVariablesFromAsset())
	{
		// In editor builds, sync with overrides.
		Variables.MigrateToNewBagInstanceWithOverrides(*DefaultVariables, OverriddenVariableGuids);

		// clean up differences
		RemoveOverridenVariablesNotInDataflowAsset();
	}
	else
	{
		RemoveAllVariables();
	}
}

void FDataflowVariableOverrides::RemoveOverridenVariablesNotInDataflowAsset()
{
	// Remove overriden variables that do not exists in the dataflow asset
	if (!OverriddenVariableGuids.IsEmpty())
	{
		if (const UPropertyBag* Bag = Variables.GetPropertyBagStruct())
		{
			for (TArray<FGuid>::TIterator It = OverriddenVariableGuids.CreateIterator(); It; ++It)
			{
				if (!Bag->FindPropertyDescByID(*It))
				{
					It.RemoveCurrentSwap();
				}
			}
		}
		else
		{
			OverriddenVariableGuids.Reset();
		}
	}
}

bool FDataflowVariableOverrides::HasVariable(FName VariableName) const
{
	return (Variables.FindPropertyDescByName(VariableName) != nullptr);
}

const FInstancedPropertyBag& FDataflowVariableOverrides::GetVariables() const
{
	return Variables;
}

FInstancedPropertyBag& FDataflowVariableOverrides::GetVariables()
{
	return Variables;
}

bool FDataflowVariableOverrides::IsVariableOverridden(const FGuid PropertyID) const
{
	return OverriddenVariableGuids.Contains(PropertyID);
}

bool FDataflowVariableOverrides::IsVariableOverridden(FName VariableName) const
{
	if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByName(VariableName))
	{
		return IsVariableOverridden(Desc->ID);
	}
	return false;
}

void FDataflowVariableOverrides::SetVariableOverridden(const FGuid PropertyID, const bool bIsOverridden)
{
	SetVariableOverrideAndNotify(PropertyID, bIsOverridden);
	SyncVariables();
}


FName FDataflowVariableOverrides::GetVariablePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(FDataflowVariableOverrides, Variables);
}

#if WITH_EDITOR
void FDataflowVariableOverrides::OnOwnerPostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	const FName VariablePropertyName = GetVariablePropertyName();

	if (PropertyName == VariablePropertyName || MemberPropertyName == VariablePropertyName)
	{
		// Add / remove / move properties / change type
		// resync variable from the dataflow asset
		SyncVariables();
	}
	else if (HasVariable(MemberPropertyName))
	{
		// Specific variable change , nothing to sync 
		// nothing to do now ( potential may need to re-evaluate the dataflow )
		SyncVariables();
	}
}

void FDataflowVariableOverrides::OnDataflowVariablesChanged(const UDataflow* DataflowAsset, FName VariableName)
{
	SyncVariables();
}
#endif

//---------------------------------------------------------------------------
// FDataflowInstance
//---------------------------------------------------------------------------

FDataflowInstance::FDataflowInstance(UObject* InOwner, UDataflow* InDataflowAsset, FName InTerminalNodeName)
	: DataflowAsset(InDataflowAsset)
	, DataflowTerminal(InTerminalNodeName)
	, VariableOverrides(this)
	, Owner(InOwner)
{
#if WITH_EDITOR
	// If we are in the process of loading, we can't be certain that we aren't already inside one of these delegate's broadcasts
	if (!IsInAsyncLoadingThread())
	{
		BindEditorDelegates();
	}
#endif
}

FDataflowInstance& FDataflowInstance::operator=(const FDataflowInstance& Other)
{
	DataflowAsset = Other.DataflowAsset;
	DataflowTerminal = Other.DataflowTerminal;
	VariableOverrides = Other.VariableOverrides;
#if WITH_EDITORONLY_DATA
	PreviewBlueprintVariableOverrides = Other.PreviewBlueprintVariableOverrides;
#endif

	// Note that Owner is deliberately not copied, since it should be set to the owning object of 
	// this instance.
	//
	// OnOwnerPropertyChangedHandle should also not be copied, as each instance of 
	// FDataflowInstance needs to have its own unique OnOwnerPropertyChangedHandle to ensure that
	// the delegate is unregistered properly when the instance is destroyed.

	return *this;
}

FDataflowInstance::~FDataflowInstance()
{
#if WITH_EDITOR
	UnbindEditorDelegates();
#endif
}

void FDataflowInstance::PostLoad()
{
#if WITH_EDITOR
	BindEditorDelegates();
#endif
}

#if WITH_EDITOR
void FDataflowInstance::BindEditorDelegates()
{
	// let listen to property change on the owner so we can listen to our own changes ( because we are a struct )
	if (Owner)
	{
		if (!OnOwnerPropertyChangedHandle.IsValid())
		{
			FDataflowAssetDelegates::OnVariablesChanged.AddWeakLambda(Owner.Get(),
				[this](const UDataflow* InDataflowAsset, FName InVariableName)
				{
					if (InDataflowAsset == DataflowAsset)
					{
						VariableOverrides.OnDataflowVariablesChanged(InDataflowAsset, InVariableName);
					}
				});

			OnOwnerPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FDataflowInstance::OnOwnerPostEditChangeProperty);
		}
	}
}

void FDataflowInstance::UnbindEditorDelegates()
{
	if (Owner)
	{
		FDataflowAssetDelegates::OnVariablesChanged.RemoveAll(Owner.Get());
	}
	// make sure we remove this outside of the Owner check in case of the Owner changed value since construction
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnOwnerPropertyChangedHandle);
	OnOwnerPropertyChangedHandle.Reset();
}

void FDataflowInstance::OnOwnerPostEditChangeProperty(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (IsValid(Owner) && IsValid(InObject) && Owner == InObject)
	{
		VariableOverrides.OnOwnerPostEditChangeProperty(InPropertyChangedEvent);

		// if the Dataflow asset has changed we need to resync the variables
		if (InPropertyChangedEvent.GetPropertyName() == GetDataflowAssetPropertyName())
		{
			SyncVariables();
		}
	}
	// should check for terminal node / dataflow asset changes ?
}

TSharedPtr<FStructOnScope> FDataflowInstance::MakeStructOnScope() const
{
	return MakeShared<FStructOnScope>(FDataflowInstance::StaticStruct(), (uint8*)(this));
}
#endif

FName FDataflowInstance::GetDataflowTerminalPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(FDataflowInstance, DataflowTerminal);
}

/** Get the dataflow asset member property name */
FName FDataflowInstance::GetDataflowAssetPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(FDataflowInstance, DataflowAsset);
}

FName FDataflowInstance::GetVariableOverridesPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(FDataflowInstance, VariableOverrides);
}

void FDataflowInstance::SetDataflowAsset(UDataflow* InDataflowAsset)
{
	if (DataflowAsset != InDataflowAsset)
	{
		DataflowAsset = InDataflowAsset;
		VariableOverrides.SyncVariables();
	}
}

UDataflow* FDataflowInstance::GetDataflowAsset() const
{
	return DataflowAsset;
}

void FDataflowInstance::SetDataflowTerminal(FName TerminalNodeName)
{
	// todo(ccaillaud)  : in the future we should check that the terminal node is part of the assigned dataflow 
	DataflowTerminal = TerminalNodeName;
}

FName FDataflowInstance::GetDataflowTerminal() const
{
	return DataflowTerminal;
}

const FInstancedPropertyBag& FDataflowInstance::GetVariables() const
{
	return VariableOverrides.GetVariables();
}

FInstancedPropertyBag& FDataflowInstance::GetVariables()
{
	return VariableOverrides.GetVariables();
}

const FDataflowVariableOverrides& FDataflowInstance::GetVariableOverrides() const
{
	return VariableOverrides;
}

FDataflowVariableOverrides& FDataflowInstance::GetVariableOverrides()
{
	return VariableOverrides;
}

void FDataflowInstance::SyncVariables()
{
	VariableOverrides.SyncVariables();
}

bool FDataflowInstance::UpdateOwnerAsset(bool bUpdateDependentAssets) const
{
	bool bSuccess = false;
	if (DataflowAsset && DataflowAsset->Dataflow)
	{
		TArray<TSharedPtr<FDataflowNode>> NodesToEvaluate;

		if (bUpdateDependentAssets)
		{
			// find all terminal nodes
			NodesToEvaluate = DataflowAsset->Dataflow->GetFilteredNodes(FDataflowTerminalNode::StaticType());
		}
		else if (const TSharedPtr<FDataflowNode> Node = DataflowAsset->Dataflow->FindFilteredNode(FDataflowTerminalNode::StaticType(), DataflowTerminal))
		{
			// find only the specified one 
			NodesToEvaluate.Add(Node);
		}

		UE::Dataflow::FEngineContext Context(Owner.Get());
		for (TSharedPtr<FDataflowNode> Node: NodesToEvaluate)
		{
			if (const FDataflowTerminalNode* TerminalNode = Node? Node->AsType<const FDataflowTerminalNode>(): nullptr)
			{
				// Note: If the node is deactivated and has any outputs, then these outputs might still need to be forwarded.
				//       Therefore the Evaluate method has to be called for whichever value of bActive.
				//       This however isn't the case of SetAssetValue() for which the active state needs to be checked before the call.
				TerminalNode->Evaluate(Context);
				if (TerminalNode->IsActive())
				{
					TerminalNode->SetAssetValue(Owner, Context);
					bSuccess = true;
				}
			}
		}
	}
	return bSuccess;
}
