// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowMesh.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowMesh)

UDataflowMesh::UDataflowMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DynamicMesh(MakeUnique<UE::Geometry::FDynamicMesh3>())
{
}

const UE::Geometry::FDynamicMesh3* UDataflowMesh::GetDynamicMesh() const
{
	return DynamicMesh.Get();
}

const UE::Geometry::FDynamicMesh3& UDataflowMesh::GetDynamicMeshRef() const
{
	return *DynamicMesh;
}

const TArray<TObjectPtr<UMaterialInterface>>& UDataflowMesh::GetMaterials() const
{
	return Materials;
}

void UDataflowMesh::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsSaving())
	{
		bool bValidDynamicMesh = DynamicMesh.IsValid();
		Ar << bValidDynamicMesh;
		if (bValidDynamicMesh)
		{
			Ar << *DynamicMesh;
		}
	} 
	else if (Ar.IsLoading())
	{
		bool bValidDynamicMesh = false;
		Ar << bValidDynamicMesh;
		if (bValidDynamicMesh)
		{
			DynamicMesh = MakeUnique<UE::Geometry::FDynamicMesh3>();
			Ar << *DynamicMesh;
		}
		else
		{
			DynamicMesh.Reset();
		}
	}

	Ar << Materials;
}

