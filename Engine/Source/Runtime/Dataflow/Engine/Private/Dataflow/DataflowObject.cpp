// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowSubGraph.h"
#include "Dataflow/DataflowCompiler.h"
#include "Dataflow/DataflowCompiledGraph.h"
#include "EdGraph/EdGraphSchema.h"

#if WITH_EDITOR
#include "EdGraph/EdGraphPin.h"
#endif
#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowObject)

#define LOCTEXT_NAMESPACE "UDataflow"

FDataflowAssetDelegates::FOnVariablesChanged FDataflowAssetDelegates::OnVariablesChanged;
FDataflowAssetDelegates::FOnSubGraphsChanged FDataflowAssetDelegates::OnSubGraphsChanged;
FDataflowAssetDelegates::FOnNodeInvalidated FDataflowAssetDelegates::OnNodeInvalidated;
#if WITH_EDITOR
FDataflowAssetDelegates::FOnDataflowPostEditUndo FDataflowAssetDelegates::OnDataflowPostEditUndo;
#endif
FDataflowAssetDelegates::FOnVariablesOverrideStateChanged FDataflowAssetDelegates::OnVariablesOverrideStateChanged;

namespace UE::Dataflow::CVars
{
	/** Enable the simulation dataflow (for now WIP) */
	TAutoConsoleVariable<bool> CVarEnableSimulationDataflow(
			TEXT("p.Dataflow.EnableSimulation"),
			false,
			TEXT("If true enable the use of simulation dataflow (WIP)"),
			ECVF_Default);
}

namespace UE::Dataflow::Private
{
	// EdNode visitor helper function (non-const version)
	template<typename ActionType>
	bool ForEachEdGraphNodeInEdGraph(UEdGraph* EdGraph, ActionType Action)
	{
		if (!EdGraph)
		{
			return false;
		}
		for (UEdGraphNode* EdNode : EdGraph->Nodes)
		{
			if (EdNode)
			{
				if (!Action(*EdGraph, *EdNode))
				{
					return false;
				}
			}
		}
		return true;
	}

	// EdNode visitor helper function (const version)
	template<typename ActionType>
	bool ForEachEdGraphNodeInEdGraph(const UEdGraph* EdGraph, ActionType Action)
	{
		if (!EdGraph)
		{
			return false;
		}
		for (const UEdGraphNode* EdNode : EdGraph->Nodes)
		{
			if (EdNode)
			{
				if (!Action(*EdGraph, *EdNode))
				{
					return false;
				}
			}
		}
		return true;
	}

	// EdNode visitor helper function (non-const version)
	// this explore the subgraphs as well
	template<typename ActionType>
	void ForEachEdGraphNodeInDataflowAsset(UDataflow* DataflowAsset, ActionType Action)
	{
		if (ForEachEdGraphNodeInEdGraph(DataflowAsset, Action))
		{
			for (UEdGraph* SubGraph : DataflowAsset->GetSubGraphs())
			{
				if (!ForEachEdGraphNodeInEdGraph(SubGraph, Action))
				{
					return;
				}
			}
		}
	}

	// EdNode visitor helper function (const version)
	// this explore the subgraphs as well
	template<typename ActionType>
	void ForEachEdGraphNodeInDataflowAsset(const UDataflow* DataflowAsset, ActionType Action)
	{
		if (ForEachEdGraphNodeInEdGraph(DataflowAsset, Action))
		{
			for (const UEdGraph* SubGraph : DataflowAsset->GetSubGraphs())
			{
				if (!ForEachEdGraphNodeInEdGraph(SubGraph, Action))
				{
					return;
				}
			}
		}
	}
}

FDataflowAssetEdit::FDataflowAssetEdit(UDataflow* InAsset, FPostEditFunctionCallback InCallback)
	: PostEditCallback(InCallback)
	, Asset(InAsset)
{
}

FDataflowAssetEdit::~FDataflowAssetEdit()
{
	PostEditCallback();
}

UE::Dataflow::FGraph* FDataflowAssetEdit::GetGraph()
{
	if (Asset)
	{
		return Asset->Dataflow.Get();
	}
	return nullptr;
}

const FString UDataflow::SimulationTag = TEXT("DataflowSimulationTag");

UDataflow::UDataflow(const FObjectInitializer& ObjectInitializer)
	: UEdGraph(ObjectInitializer)
	, Dataflow(new UE::Dataflow::FGraph())
{}

void UDataflow::BeginDestroy()
{
	BeginDestroyEvent.Broadcast(this);
	BeginDestroyEvent.Clear();
	
	Super::BeginDestroy();
}

void UDataflow::EvaluateTerminalNodeByName(FName NodeName, UObject* Asset)
{
	ensureAlwaysMsgf(false, TEXT("Deprecated use the dataflow blueprint library from now on"));
}

void UDataflow::PostEditCallback()
{
	// mark as dirty for the UObject
}

void UDataflow::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UDataflow* const This = CastChecked<UDataflow>(InThis);

	for(TObjectPtr<const UDataflowEdNode> Target : This->GetRenderTargets())
	{
		Collector.AddReferencedObject(Target);
	}

	if (This->Dataflow)
	{
		This->Dataflow->AddReferencedObjects(Collector);
	}
	Super::AddReferencedObjects(InThis, Collector);
}

#if WITH_EDITOR

void UDataflow::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UDataflow::PostEditUndo()
{
	Super::PostEditUndo();

	// Refresh the graph panel so widgets for nodes restored by the transaction
	// system (e.g. comment nodes whose own PostEditUndo doesn't notify) get rebuilt.
	if (Schema)
	{
		NotifyGraphChanged();
	}

	FDataflowAssetDelegates::OnDataflowPostEditUndo.Broadcast(*this);
}

#endif

void UDataflow::PostLoad()
{
#if WITH_EDITOR
	const TSet<FName>& DisabledNodes = Dataflow->GetDisabledNodes();

	UE::Dataflow::Private::ForEachEdGraphNodeInDataflowAsset(this, 
		[this, &DisabledNodes](UEdGraph& EdGraph, UEdGraphNode& EdNode)
		{
			// Not all nodes are UDataflowEdNode (There is now UDataflowEdNodeComment)
			if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(&EdNode))
			{
				DataflowEdNode->SetDataflowGraph(Dataflow);
				DataflowEdNode->SetDataflowNodeGuid(DataflowEdNode->DataflowNodeGuid);
				DataflowEdNode->UpdatePinsFromDataflowNode();
			}

			if (DisabledNodes.Contains(FName(EdNode.GetName())))
			{
				EdNode.SetEnabledState(ENodeEnabledState::Disabled);
			}
			return true; // visit all nodes
		});

	// Resync connections (nodes might have redirected connections
	for (const UE::Dataflow::FLink& Link : Dataflow->GetConnections())
	{
		TSharedPtr<const FDataflowNode> OutputNode = Dataflow->FindBaseNode(Link.OutputNode);
		TSharedPtr<const FDataflowNode> InputNode = Dataflow->FindBaseNode(Link.InputNode);
		if (ensure(OutputNode && InputNode))
		{
			const FDataflowOutput* const Output = OutputNode->FindOutput(Link.Output);
			const FDataflowInput* const Input = InputNode->FindInput(Link.Input);
			if (Output && Input)
			{
				TObjectPtr<UDataflowEdNode> OutputEdNode = FindEdNodeByDataflowNodeGuid(Link.OutputNode);
				TObjectPtr<UDataflowEdNode> InputEdNode = FindEdNodeByDataflowNodeGuid(Link.InputNode);

				if (ensure(OutputEdNode && InputEdNode))
				{
					UEdGraphPin* const OutputPin = OutputEdNode->FindPin(Output->GetName(), EEdGraphPinDirection::EGPD_Output);
					UEdGraphPin* const InputPin = InputEdNode->FindPin(Input->GetName(), EEdGraphPinDirection::EGPD_Input);

					if (ensure(OutputPin && InputPin))
					{
						if (OutputPin->LinkedTo.Find(InputPin) == INDEX_NONE)
						{
							OutputPin->MakeLinkTo(InputPin);
						}
					}
				}
			}
		}
	}
#endif

	LastModifiedRenderTarget = UE::Dataflow::FTimestamp::Current();
	UObject::PostLoad();
}

void UDataflow::AddRenderTarget(TObjectPtr<const UDataflowEdNode> InNode)
{
	LastModifiedRenderTarget = UE::Dataflow::FTimestamp::Current();
	check(InNode->ShouldRenderNode());
	RenderTargets.AddUnique(InNode);
}

void UDataflow::RemoveRenderTarget(TObjectPtr<const UDataflowEdNode> InNode)
{
	LastModifiedRenderTarget = UE::Dataflow::FTimestamp::Current();
	check(!InNode->ShouldRenderNode());
	RenderTargets.Remove(InNode);
}

void UDataflow::AddWireframeRenderTarget(TObjectPtr<const UDataflowEdNode> InNode)
{
	LastModifiedRenderTarget = UE::Dataflow::FTimestamp::Current();
	check(InNode->ShouldWireframeRenderNode());
	WireframeRenderTargets.AddUnique(InNode);
}

void UDataflow::RemoveWireframeRenderTarget(TObjectPtr<const UDataflowEdNode> InNode)
{
	LastModifiedRenderTarget = UE::Dataflow::FTimestamp::Current();
	check(!InNode->ShouldWireframeRenderNode());
	WireframeRenderTargets.Remove(InNode);
}


void UDataflow::Serialize(FArchive& Ar)
{
#if WITH_EDITOR
	// Disable per-node serialization (used for transactions, i.e., undo/redo) when serializing the whole graph.
	bEnablePerNodeTransactionSerialization = false;
#endif

	Super::Serialize(Ar);
	Dataflow->Serialize(Ar, this);

#if WITH_EDITOR
	bEnablePerNodeTransactionSerialization = true;
#endif
}

TObjectPtr<const UDataflowEdNode> UDataflow::FindEdNodeByDataflowNodeGuid(const FGuid& Guid) const
{
	TObjectPtr<const UDataflowEdNode> FoundNode(nullptr);

	UE::Dataflow::Private::ForEachEdGraphNodeInDataflowAsset(this, 
		[&Guid, &FoundNode](const UEdGraph& EdGraph, const UEdGraphNode& EdNode)
		{
			if (const UDataflowEdNode* const DataflowEdNode = Cast<UDataflowEdNode>(&EdNode))
			{
				if (DataflowEdNode->GetDataflowNodeGuid() == Guid)
				{
					FoundNode = DataflowEdNode;
					return false; // early exit
				}
			}
			return true; // continue visiting 
		});

	return FoundNode;
}

TObjectPtr<UDataflowEdNode> UDataflow::FindEdNodeByDataflowNodeGuid(const FGuid& Guid)
{
	TObjectPtr<UDataflowEdNode> FoundNode(nullptr);

	UE::Dataflow::Private::ForEachEdGraphNodeInDataflowAsset(this,
		[&Guid, &FoundNode](UEdGraph& EdGraph, UEdGraphNode& EdNode)
		{
			if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(&EdNode))
			{
				if (DataflowEdNode->GetDataflowNodeGuid() == Guid)
				{
					FoundNode = DataflowEdNode;
					return false; // early exit
				}
			}
			return true; // continue visiting 
		});

	return FoundNode;
}

UDataflow* UDataflow::GetDataflowAssetFromEdGraph(UEdGraph* EdGraph)
{
	// main graph is the dataflow asset
	UDataflow* DataflowAsset = Cast<UDataflow>(EdGraph);
	if (!DataflowAsset && EdGraph)
	{
		UEdGraph* EdParentGraph = Cast<UEdGraph>(EdGraph->GetOuter());
		DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(EdParentGraph);
	}
	return DataflowAsset;
}

const UDataflow* UDataflow::GetDataflowAssetFromEdGraph(const UEdGraph* EdGraph)
{
	// main graph is the dataflow asset
	const UDataflow* DataflowAsset = Cast<const UDataflow>(EdGraph);
	if (!DataflowAsset && EdGraph)
	{
		const UEdGraph* EdParentGraph = Cast<const UEdGraph>(EdGraph->GetOuter());
		DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(EdParentGraph);
	}
	return DataflowAsset;
}

const UDataflowSubGraph* UDataflow::FindSubGraphByName(FName Name) const
{
	for (const UDataflowSubGraph* SubGraph : DataflowSubGraphs)
	{
		if (SubGraph->GetFName() == Name)
		{
			return SubGraph;
		}
	}
	return nullptr;
}

UDataflowSubGraph* UDataflow::FindSubGraphByName(FName Name)
{
	for (UDataflowSubGraph* SubGraph : DataflowSubGraphs)
	{
		if (SubGraph->GetFName() == Name)
		{
			return SubGraph;
		}
	}
	return nullptr;
}

const UDataflowSubGraph* UDataflow::FindSubGraphByGuid(const FGuid& SubGraphGuid) const
{
	for (const UDataflowSubGraph* SubGraph : DataflowSubGraphs)
	{
		if (SubGraph->GetSubGraphGuid() == SubGraphGuid)
		{
			return SubGraph;
		}
	}
	return nullptr;
}

UDataflowSubGraph* UDataflow::FindSubGraphByGuid(const FGuid& SubGraphGuid)
{
	for (UDataflowSubGraph* SubGraph : DataflowSubGraphs)
	{
		if (SubGraph->GetSubGraphGuid() == SubGraphGuid)
		{
			return SubGraph;
		}
	}
	return nullptr;
}

void UDataflow::AddSubGraph(UDataflowSubGraph* SubGraph)
{
	if (ensure(SubGraph))
	{
		if (ensure(SubGraph->IsInOuter(this)))
		{
#if WITH_EDITORONLY_DATA
			SubGraphs.AddUnique(SubGraph);
#endif
			DataflowSubGraphs.AddUnique(SubGraph);
			Modify();
		}
	}	
}

void UDataflow::RemoveSubGraph(UDataflowSubGraph* SubGraph)
{
	if (ensure(SubGraph))
	{
#if WITH_EDITORONLY_DATA
			SubGraphs.Remove(SubGraph);
#endif
			DataflowSubGraphs.Remove(SubGraph);
			Modify();
	}
}

const TArray<TObjectPtr<UDataflowSubGraph>>& UDataflow::GetSubGraphs() const
{
	return DataflowSubGraphs;
}

void UDataflow::RefreshEdNodeByGuid(const FGuid NodeGuid)
{
	RefreshEdNode(FindEdNodeByDataflowNodeGuid(NodeGuid));
}

void UDataflow::RefreshEdNode(TObjectPtr<UDataflowEdNode> EdNode)
{
	if (EdNode)
	{
		EdNode->UpdatePinsFromDataflowNode();
		EdNode->UpdatePinsConnectionsFromDataflowNode();
		if (Nodes.Contains(EdNode))
		{
			NotifyNodeChanged(EdNode);
			//NotifyGraphChanged();
			return;
		}
		for (TObjectPtr<UDataflowSubGraph> Subgraph : DataflowSubGraphs)
		{
			if (Subgraph && Subgraph->Nodes.Contains(EdNode))
			{
				Subgraph->NotifyNodeChanged(EdNode);
				//Subgraph->NotifyGraphChanged();
				return;
			}
		}
	}
}

TSharedPtr<const UE::Dataflow::FCompiledGraph> UDataflow::CompileGraphIfNeeded()
{
	const FGuid GraphTopolyGuid = Dataflow ? Dataflow->GetGraphTopologyGuid() : FGuid();
	if (LastCompiledTopologyGUID != GraphTopolyGuid)
	{
		CompiledGraphs.Reset();

		UE::Dataflow::FCompilerParameters Params;
		Params.DataflowObjectToCompile = this;

		UE::Dataflow::FCompilerResults Results;
		if (UE::Dataflow::FCompiler::Compile(Params, Results))
		{
			CompiledGraphs = Results.GetCompiledGraphs();
			UE_LOGF(LogTemp, Warning, "Dataflow Compiled : time = %.3f ms (%d compiled graphs)", (float)Results.GetCompilationTimeMs(), CompiledGraphs.Num());
		}
		else
		{
			ensureMsgf(false, TEXT("Graph compilation failed, check the result error log"));
		}
		// store the guid even if the compilation was not successful so that we don't keep on trying to recompile
		LastCompiledTopologyGUID = GraphTopolyGuid;
	}
	const TSharedRef<UE::Dataflow::FCompiledGraph>* OutCompiledGraph = CompiledGraphs.Find(this);
	return OutCompiledGraph 
		? OutCompiledGraph->ToSharedPtr()
		: nullptr;
}

#if WITH_EDITOR
bool UDataflow::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	const FName& Name = InProperty->GetFName();

	if (Name == GET_MEMBER_NAME_CHECKED(ThisClass, Type))
	{
		return UE::Dataflow::CVars::CVarEnableSimulationDataflow.GetValueOnGameThread();
	}

	return true;
}

#endif

#undef LOCTEXT_NAMESPACE

