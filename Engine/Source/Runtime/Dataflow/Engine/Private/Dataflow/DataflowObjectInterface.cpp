// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowObjectInterface.h"

#include "Dataflow/DataflowAttachment.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowCompiler.h"
#include "Dataflow/DataflowInstance.h"

namespace UE::Dataflow
{
	template class TEngineContext<FContextSingle>;
	template class TEngineContext<FContextThreaded>;

	namespace Private
	{
		void UpdateAssetFromNode(FContext& Context, const FDataflowNode& Node, UObject* Asset)
		{
			if (const FDataflowTerminalNode* TerminalNode = Node.AsType<const FDataflowTerminalNode>())
			{
				// Note: If the node is deactivated and has any outputs, then these outputs might still need to be forwarded.
				//       Therefore the Evaluate method has to be called for whichever value of bActive.
				//       This however isn't the case of SetAssetValue() for which the active state needs to be checked before the call.
				if (TerminalNode->IsActive())
				{
					TerminalNode->SetAssetValue(Asset, Context);
				}
			}
		}

		bool EvaluateGraphInternal(TSharedRef<IDataflowEvaluator> Evaluator, UDataflow* DataflowObject, UObject* ObjectToUpdate, FOnPostEvaluationFunction OnPostEvaluation)
		{
			if (DataflowObject)
			{
				if (TSharedPtr<FGraph> Graph = DataflowObject->GetDataflow())
				{
					if (TSharedPtr<const UE::Dataflow::FCompiledGraph> CompiledGraph = DataflowObject->CompileGraphIfNeeded())
					{
						TWeakObjectPtr<UObject> WeakObjectToUpdate = ObjectToUpdate;

						FDataflowEvaluatorParameters Params(CompiledGraph.ToSharedRef());
						Params.OnPostEvaluation = OnPostEvaluation;
						Params.OnTerminalNodeEvaluated = [WeakObjectToUpdate](FContext& Context, const FDataflowNode& Node)
							{
								TStrongObjectPtr<UObject> ObjectToUpdate = WeakObjectToUpdate.Pin();
								UpdateAssetFromNode(Context, Node, ObjectToUpdate.Get());
							};
						Evaluator->Start(Params);
						return true;
					}
				}
			}
			return false;
		}

		bool EvaluateGraphInternal(TSharedRef<IDataflowEvaluator> Evaluator, UObject* ObjectToUpdate, FOnPostEvaluationFunction OnPostEvaluation)
		{
			if (UDataflow* DataflowObject = InstanceUtils::GetDataflowAssetFromObject(ObjectToUpdate))
			{
				return EvaluateGraphInternal(Evaluator, DataflowObject, ObjectToUpdate, OnPostEvaluation);
			}
			return false;
		}

		UObject* FindBoundAsset(UObject* Object)
		{
			if (UDataflowAttachment* DataflowAttachment = Cast<UDataflowAttachment>(Object))
			{
				return DataflowAttachment->GetOuter();
			}
			return Object;
		}
	}
}
