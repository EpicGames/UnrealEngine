// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowOverlay.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowSelection.h"

namespace UE::Dataflow::Overlay
{
	static const FString COverlaySeparator = TEXT("─────────────────────────\n");

	FString BuildOverlayNodeInfoString(const FDataflowNode* InNode)
	{
		FString Path = "/Construction/"; // TODO: Get this from Toolkit
		FString NodeName = InNode->GetName().ToString();
		FString NodeType = InNode->GetType().ToString();

		FString OutStr;
		OutStr += FString::Printf(TEXT("%s\n%s\n%s\n"), *Path, *NodeName, *NodeType);
		return OutStr;
	}

	FString BuildOverlaySelectionEvaluateResultString(FDataflowSelection& InSelection)
	{
		FString OutStr;
		OutStr += FString::Printf(TEXT("%s\n"), *InSelection.ToString());
		return OutStr;
	}

	FString BuildOverlayCollectionInfoString(const FManagedArrayCollection& InCollection)
	{
		FString NumVertsStr = FString::FormatAsNumber(InCollection.NumElements(FGeometryCollection::VerticesGroup));
		FString NumFacesStr = FString::FormatAsNumber(InCollection.NumElements(FGeometryCollection::FacesGroup));
		FString NumTransformsStr = FString::FormatAsNumber(InCollection.NumElements(FGeometryCollection::TransformGroup));
		FString NumGeometryStr = FString::FormatAsNumber(InCollection.NumElements(FGeometryCollection::GeometryGroup));

		FString OutStr;
		OutStr += FString::Printf(TEXT("Vertices: %s\nFaces: %s\nTransforms: %s\nGeometry: %s\n"), *NumVertsStr, *NumFacesStr, *NumTransformsStr, *NumTransformsStr);
		return OutStr;
	}

	FString BuildOverlayBoundsInfoString(const FManagedArrayCollection& InCollection)
	{
		GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
		const FBox BBox = BoundsFacade.GetBoundingBoxInCollectionSpace();

		FString BBoxCenterStr = FString::Printf(TEXT("%3.3f, %3.3f, %3.3f"), BBox.GetCenter().X, BBox.GetCenter().Y, BBox.GetCenter().Z);
		FString BBoxMinStr = FString::Printf(TEXT("%3.3f, %3.3f, %3.3f"), BBox.Min.X, BBox.Min.Y, BBox.Min.Z);
		FString BBoxMaxStr = FString::Printf(TEXT("%3.3f, %3.3f, %3.3f"), BBox.Max.X, BBox.Max.Y, BBox.Max.Z);
		FString BBoxSizeStr = FString::Printf(TEXT("%3.3f, %3.3f, %3.3f"), BBox.GetSize().X, BBox.GetSize().Y, BBox.GetSize().Z);

		FString OutStr;
		OutStr += FString::Printf(TEXT("Center: %s\nMin: %s\nMax: %s\nSize: %s\n"), *BBoxCenterStr, *BBoxMinStr, *BBoxMaxStr, *BBoxSizeStr);
		return OutStr;
	}

	FString BuildOverlayMemInfoString(const FDataflowNode* InNode)
	{
		FString OutStr;

		int32 SumMem = 0;
		for (FPropertyValueIterator PropertyIt(FProperty::StaticClass(), InNode->TypedScriptStruct(), InNode); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = PropertyIt.Key();

			if (const FEnumProperty* PropAsEnumProp = CastField<const FEnumProperty>(Property))
			{
				SumMem += PropAsEnumProp->GetUnderlyingProperty()->GetElementSize();
			}
			else if (Property)
			{
				SumMem += Property->GetElementSize();
			}
		}

		float MemResult = (float)SumMem / 1024.f;
		OutStr += FString::Printf(TEXT("Memory: %03fKB\n"), MemResult);

		return OutStr;
	}

	FString BuildOverlayFinalString(const TArray<FString>& InStringArr)
	{
		FString OutStr;

		int32 Idx = 0;
		for (const FString& Str : InStringArr)
		{
			OutStr += Str;

			if (Idx < InStringArr.Num() - 1)
			{
				OutStr += COverlaySeparator;
			}

			Idx++;
		}

		return OutStr;
	}
}

