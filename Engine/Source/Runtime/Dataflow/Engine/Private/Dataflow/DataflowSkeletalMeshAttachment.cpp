// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSkeletalMeshAttachment.h"
#include "Dataflow/DataflowContent.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Engine/SkeletalMesh.h"

void UDataflowSkeletalMeshAttachment::WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const
{
	if (const TObjectPtr<UDataflowSkeletalContent> Content = Cast<UDataflowSkeletalContent>(DataflowContent))
	{
		Content->SetDataflowAsset(GetDataflowInstance().GetDataflowAsset());
		Content->SetDataflowTerminal(GetDataflowInstance().GetDataflowTerminal().ToString());

		if (USkeletalMesh* SkmAsset = GetTypedOuter<USkeletalMesh>())
		{
			Content->SetSkeletalMesh(SkmAsset);
		}
	}
}

void UDataflowSkeletalMeshAttachment::ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent)
{

}

TObjectPtr<UDataflowBaseContent> UDataflowSkeletalMeshAttachment::CreateDataflowContent()
{
	TObjectPtr<UDataflowBaseContent> BaseContent = NewObject<UDataflowSkeletalContent>();

	BaseContent->SetDataflowOwner(this);
	BaseContent->SetTerminalAsset(this);

	WriteDataflowContent(BaseContent);

	return BaseContent;
}

FString UDataflowSkeletalMeshAttachment::GetPreviewActorPath() const
{
	return TEXT("/Dataflow/BP_SkeletalMeshPreview.BP_SkeletalMeshPreview_C");
}