// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSkeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSkeleton)

const FReferenceSkeleton& FDataflowSkeleton::GetRefSkeleton() const
{
	return RefSkeleton;
}

FReferenceSkeletonModifier FDataflowSkeleton::ModifySkeleton()
{
	return FReferenceSkeletonModifier(RefSkeleton, nullptr);
}

void FDataflowSkeleton::Empty()
{
	RefSkeleton.Empty();
}

void FDataflowSkeleton::Copy(const FReferenceSkeleton& ToCopy)
{
	RefSkeleton = ToCopy;
}

bool FDataflowSkeleton::Serialize(FArchive& Ar)
{
	Ar << RefSkeleton;
	return true;
}
