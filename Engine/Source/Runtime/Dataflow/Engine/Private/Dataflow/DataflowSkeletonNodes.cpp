// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSkeletonNodes.h"

#include "BoneIndices.h"
#include "ReferenceSkeleton.h"
#include "Dataflow/DataflowNodeFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSkeletonNodes)

namespace UE::Dataflow
{
	void RegisterSkeletonNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMergeSkeletonsDataflowNode);
	}
}

FMergeSkeletonsDataflowNode::FMergeSkeletonsDataflowNode(
	const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Skeletons);
	RegisterInputConnection(&SharedRootPosition).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bPreserveBoneNames).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&MergedSkeleton);
}

void FMergeSkeletonsDataflowNode::Evaluate(
	UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MergedSkeleton))
	{
		const TArray<FDataflowSkeleton>& InSkeletons = GetValue(Context, &Skeletons);
		const FVector InRootPos = GetValue(Context, &SharedRootPosition);
		const bool bInPreserveBoneNames = GetValue(Context, &bPreserveBoneNames);

		FDataflowSkeleton OutSkeleton;
		
		{ // Note: this scope is to ensure Modifier destructor runs before OutSkeleton is assigned to the dataflow output
			FReferenceSkeletonModifier Modifier = OutSkeleton.ModifySkeleton();
			const FTransform SharedRootTransform(InRootPos);
			FName RootName("Root");
			Modifier.Add(FMeshBoneInfo(RootName, RootName.ToString(), INDEX_NONE),
				SharedRootTransform);
			TSet<FName> UsedBoneNames;
			UsedBoneNames.Add(RootName);
			auto SetUniqueBoneName = [&UsedBoneNames](FName& NameToMakeUnique) -> bool
			{
				int32 NameInt = NameToMakeUnique.GetNumber();
				bool bWasModified = false;
				while (UsedBoneNames.Contains(NameToMakeUnique))
				{
					bWasModified = true;
					NameToMakeUnique.SetNumber(++NameInt);
				}
				UsedBoneNames.Add(NameToMakeUnique);
				return bWasModified;
			};

			int32 BoneCount = 0;
			for (const FDataflowSkeleton& SrcSkeleton : InSkeletons)
			{
				const FReferenceSkeleton& SrcRef = SrcSkeleton.GetRefSkeleton();
				const TArray<FMeshBoneInfo>& SrcInfos = SrcRef.GetRawRefBoneInfo();
				const TArray<FTransform>& SrcPoses = SrcRef.GetRawRefBonePose();

				const int32 Offset = Modifier.GetReferenceSkeleton().GetRawBoneNum();

				for (int32 BoneIdx = 0; BoneIdx < SrcInfos.Num(); ++BoneIdx)
				{
					FMeshBoneInfo BoneInfo = SrcInfos[BoneIdx];
					bool bNameChanged = !bInPreserveBoneNames;
					if (bInPreserveBoneNames)
					{
						bNameChanged = SetUniqueBoneName(BoneInfo.Name);
					}
					else
					{
						BoneInfo.Name = FName("Bone", BoneCount++);
					}
					if (bNameChanged)
					{
#if WITH_EDITORONLY_DATA
						BoneInfo.ExportName = BoneInfo.Name.ToString();
#endif
					}
					BoneInfo.ParentIndex = (BoneInfo.ParentIndex == INDEX_NONE) ? 0 : BoneInfo.ParentIndex + Offset;

					Modifier.Add(BoneInfo, SrcPoses[BoneIdx]);
				}
			}
		}

		SetValue(Context, OutSkeleton, &MergedSkeleton);
	}
}
