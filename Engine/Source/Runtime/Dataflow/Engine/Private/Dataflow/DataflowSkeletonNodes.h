// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowSkeleton.h"

#include "DataflowSkeletonNodes.generated.h"

/**
 * Merges an array of skeletons into a single skeleton
 */
USTRUCT()
struct FMergeSkeletonsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMergeSkeletonsDataflowNode, "MergeSkeletons", "Skeleton", "Append Merge Combine")

private:
	/** Skeletons to merge together */
	UPROPERTY(EditAnywhere, Category = Skeleton, meta = (DataflowInput))
	TArray<FDataflowSkeleton> Skeletons;

	/** World-space position of the shared root bone added to the merged skeleton */
	UPROPERTY(EditAnywhere, Category = Skeleton, meta = (DataflowInput))
	FVector SharedRootPosition = FVector::ZeroVector;

	/** Whether to attempt to preserve the names of the bones from the input skeleton. Otherwise, bones will be simply numbered. */
	UPROPERTY(EditAnywhere, Category = Skeleton, meta = (DataflowInput))
	bool bPreserveBoneNames = true;

	/** The merged skeleton */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSkeleton MergedSkeleton;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMergeSkeletonsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow
{
	void RegisterSkeletonNodes();
}
