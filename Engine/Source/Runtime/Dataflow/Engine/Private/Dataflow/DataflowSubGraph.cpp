// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSubGraph.h"

#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSubGraphNodes.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditAction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSubGraph)

namespace UE::Dataflow::DataflowSubGraph::Private
{
	template<typename T>
	T* GetNodeByType(const UEdGraph& EdGraph)
	{
		for (UEdGraphNode* EdNode : EdGraph.Nodes)
		{
			if (UDataflowEdNode* DataflowEdNode = Cast<UDataflowEdNode>(EdNode))
			{
				if (TSharedPtr<FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
				{
					if (T* TypedDataflowNode = DataflowNode->AsType<T>())
					{
						return TypedDataflowNode;
					}
				}
			}
		}
		return nullptr;
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowSubGraphDelegates::FOnDataflowSubGraphLoaded FDataflowSubGraphDelegates::OnSubGraphLoaded;

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

UDataflowSubGraph::UDataflowSubGraph(class FObjectInitializer const& ObjectInitializer)
	: UEdGraph(ObjectInitializer)
	, SubGraphGuid(IsTemplate() ? FGuid() : FGuid::NewGuid())
	, bIsLoaded(false)
{}

FDataflowSubGraphInputNode* UDataflowSubGraph::GetInputNode() const
{
	// TODO : we should probably cache the node Guid or even a weakPtr on the EdNode upon loading for better perf in the future
	return UE::Dataflow::DataflowSubGraph::Private::GetNodeByType<FDataflowSubGraphInputNode>(*this);
}

FDataflowSubGraphOutputNode* UDataflowSubGraph::GetOutputNode() const
{
	// TODO : we should probably cache the node Guid or even a weakPtr on the EdNode upon loading for better perf in the future
	return UE::Dataflow::DataflowSubGraph::Private::GetNodeByType<FDataflowSubGraphOutputNode>(*this);
}

bool UDataflowSubGraph::IsForEachSubGraph() const
{
	return bIsForEach;
}

void UDataflowSubGraph::SetForEachSubGraph(bool bValue)
{
	if (bValue != bIsForEach)
	{
		bIsForEach = bValue;

		Modify();
		if (UDataflow* DataflowAsset = UDataflow::GetDataflowAssetFromEdGraph(this))
		{
			DataflowAsset->Modify();
			FDataflowAssetDelegates::OnSubGraphsChanged.Broadcast(DataflowAsset, SubGraphGuid, UE::Dataflow::ESubGraphChangedReason::ChangedType);
		}
	}
}

void UDataflowSubGraph::PostLoad()
{
	Super::PostLoad();

	bIsLoaded = true;

	if (FDataflowSubGraphDelegates::OnSubGraphLoaded.IsBound())
	{
		FDataflowSubGraphDelegates::OnSubGraphLoaded.Broadcast(*this);
	}
}

#if WITH_EDITOR
void UDataflowSubGraph::PostEditUndo()
{
	Super::PostEditUndo();

	// Refresh the graph panel so widgets for nodes restored by the transaction
	// system (e.g. comment nodes whose own PostEditUndo doesn't notify) get rebuilt.
	if (Schema)
	{
		NotifyGraphChanged();
	}
}
#endif
