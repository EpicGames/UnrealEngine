// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSubGraphNodes.h"

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowSubGraph.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "GraphEditAction.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "DataflowSubGraphNodes"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSubGraphNodes)

namespace UE::Dataflow::SubGraphNodes::Private
{
	class FContextSubGraph: public FEngineContext
	{
		static constexpr int32 MaxSubGraphNestingDepth = 20;

	public:
		DATAFLOW_CONTEXT_INTERNAL(FEngineContext, FContextSubGraph);

		FContextSubGraph(FContext& InParentContext, UObject* DataflowAsset, const ISubGraphContextCallback& InSubGraphCallback)
			: FEngineContext(DataflowAsset)
			, ParentContext(InParentContext)
			, SubGraphCallback(InSubGraphCallback)
		{
			// make sure to match the threaded mode
			SetThreaded(ParentContext.IsThreaded());

			// Record the subgraph nesting depth
			if (const FContextSubGraph* ParentSubGraphContext = ParentContext.AsType<FContextSubGraph>())
			{
				SubGraphNestingDepth = ParentSubGraphContext->SubGraphNestingDepth + 1;
			}
		}

		virtual void SetDataImpl(FContextCacheKey Key, TUniquePtr<FContextCacheElementBase>&& DataStoreEntry) override
		{
			DataStore.Emplace(Key, MoveTemp(DataStoreEntry));
		}

		virtual const TUniquePtr<FContextCacheElementBase>* GetDataImpl(FContextCacheKey Key) const override
		{
			return DataStore.Find(Key);
		}

		virtual bool HasDataImpl(FContextCacheKey Key, FTimestamp InTimestamp = FTimestamp::Invalid) const override
		{
			return DataStore.Contains(Key) && DataStore[Key]->GetTimestamp() >= InTimestamp;
		}

		virtual bool IsEmptyImpl() const override
		{
			return DataStore.IsEmpty();
		}

		virtual void BroadCastLog(EMessageSeverity::Type Severity, const FText& InMessage, const FDataflowNode* InNode, const FDataflowOutput* InOutput) override
		{
			// make sure to forward to the parent context
			if (Severity == EMessageSeverity::Error)
			{
				ParentContext.Error(InMessage, InNode, InOutput);
			}
			else if (Severity == EMessageSeverity::Warning)
			{
				ParentContext.Warning(InMessage, InNode, InOutput);
			}
			else
			{
				ParentContext.Info(InMessage, InNode, InOutput);
			}

			FEngineContext::BroadCastLog(Severity, InMessage, InNode, InOutput);
		}

		virtual void Evaluate(const FDataflowNode* Node, const FDataflowOutput* Output) override
		{
			BeginContextEvaluation(Node, Output);
		}

		virtual bool Evaluate(const FDataflowOutput& Connection) override
		{
			return Connection.EvaluateImpl(*this);
		}

		bool EvaluateSubGraph(const FDataflowSubGraphOutputNode& SubGraphOutputNode, const FDataflowNode& CallerNode, const FDataflowOutput& CallerOutput)
		{
			if (SubGraphNestingDepth >= MaxSubGraphNestingDepth)
			{
				ParentContext.Error("Maximum nesting depth reached - skipping evaluation (Potential recursive subgraph call)", &CallerNode, &CallerOutput);
				CallerOutput.SetNullValue(*this);
				return false;
			}

			bool bEvaluationSuccess = false;
			if (const FDataflowInput* InputToPull = SubGraphOutputNode.FindInput(CallerOutput.GetName()))
			{
				if (const FDataflowOutput* OutputToEvaluate = InputToPull->GetConnection())
				{
					// we need to make sure the graph output and caller output are of compatible type
					if (UE::Dataflow::AreTypesCompatible(OutputToEvaluate->GetType(), CallerOutput.GetType()))
					{
						TAtomic<bool> bHasWarning = false;
						TAtomic<bool> bHasError = false;
						FDelegateHandle OnLogHandle = ParentContext.GetOnContextLogMulticast().AddLambda(
							[&bHasWarning, &bHasError](const FLogMessage& InMessage)
							{
								if (InMessage.Severity == EMessageSeverity::Warning)
								{
									bHasWarning = true;
								}
								else if (InMessage.Severity == EMessageSeverity::Error)
								{
									bHasError = true;
								}
							});

						if (Evaluate(*OutputToEvaluate))
						{
							// copy the cache value from this cachestore to the parent one 
							bEvaluationSuccess = CopyDataToAnotherContext(OutputToEvaluate->CacheKey(), ParentContext, CallerOutput.CacheKey(), 
								CallerOutput.GetProperty(), CallerOutput.GetOwningNodeGuid(), CallerOutput.GetOwningNodeValueHash(), CallerOutput.GetOwningNodeTimestamp()
							);
						}

						ParentContext.GetOnContextLogMulticast().Remove(OnLogHandle);

						if (bHasError)
						{
							const FText ErrorMessage(LOCTEXT("SubgraphContext_EvaluationError", "Evaluation of SubGraph returned errors : see the details above"));
							ParentContext.Error(ErrorMessage, &CallerNode, &CallerOutput);
						}
						if (bHasWarning)
						{
							const FText WarningMessage(LOCTEXT("SubgraphContext_EvaluationWarning", "Evaluation of SubGraph returned warnings: see the details above"));
							ParentContext.Warning(WarningMessage, &CallerNode, &CallerOutput);
						}
					}
				}
			}
			return bEvaluationSuccess;
		}

		void EvaluateParentContext(const FDataflowOutput& SubGraphOutput)
		{
			SubGraphCallback.EvaluateParentContext(ParentContext, *this, SubGraphOutput);
		}

		void SetCurrentArrayIndex(int32 Index) { CurrentArrayIndex = Index; }
		int32 GetCurrentArrayIndex() const { return CurrentArrayIndex; }

		void TransferPerfData()
		{
#if DATAFLOW_EDITOR_EVALUATION
			ParentContext.AddExternalPerfData(GetPerfData());
#endif
		}

		virtual UObject* AddAsset(const FString& AssetPath, const UClass* AssetClass) override
		{
			return ParentContext.AddAsset(AssetPath, AssetClass);
		}

		virtual UObject* CommitAsset(const FString& AssetPath) override
		{
			return ParentContext.CommitAsset(AssetPath);
		}

		virtual void ClearAssets() override
		{
			ParentContext.ClearAssets();
		}

	private:
		int32 CurrentArrayIndex = 0;
		int32 SubGraphNestingDepth = 0;
		FContext& ParentContext;
		const UE::Dataflow::ISubGraphContextCallback& SubGraphCallback;
	};
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow
{
	void RegisterSubGraphNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSubGraphInputNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSubGraphOutputNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowSubGraphGetCurrentIndexNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowCallSubGraphNode);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowSubGraphInputNode::FDataflowSubGraphInputNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, DynamicConnections(UE::Dataflow::FPin::EDirection::OUTPUT, this, Cast<UDataflow>(InParam.OwningObject))
{
}

void FDataflowSubGraphInputNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Dataflow::SubGraphNodes::Private;

	if (FContextSubGraph* SubGraphContext = Context.AsType<FContextSubGraph>())
	{
		SubGraphContext->EvaluateParentContext(*Out);
	}
	else if (Out)
	{
		Out->SetNullValue(Context);
	}
}

void FDataflowSubGraphInputNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		// make sure the node is up to date with the serialized data
		DynamicConnections.Refresh();
	}
}

void FDataflowSubGraphInputNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	DynamicConnections.Refresh();
}

bool FDataflowSubGraphInputNode::SupportsDropConnectionOnNode(FName TypeName, UE::Dataflow::FPin::EDirection Direction) const
{ 
	return (Direction == UE::Dataflow::FPin::EDirection::INPUT);
}

const FDataflowConnection* FDataflowSubGraphInputNode::OnDropConnectionOnNode(const FDataflowConnection& DroppedConnection)
{
	using namespace  UE::Dataflow::SubGraphNodes;
	using namespace UE::Dataflow::Type;
	if (SupportsDropConnectionOnNode(DroppedConnection.GetType(), DroppedConnection.GetDirection()))
	{
		FPropertyBagPropertyDesc Desc = GetPropertyBagPropertyDescFromDataflowConnection(DroppedConnection);
		Desc.Name = MakeUniqueNameForPropertyBag(Desc.Name, PropertyBag);
		PropertyBag.AddProperties({ Desc });
		DynamicConnections.Refresh();
		return FindOutput(Desc.Name);
	}
	return nullptr;
}

void FDataflowSubGraphInputNode::AddConnectionsTo(TArrayView<FDataflowConnection*> Connections)
{
	using namespace  UE::Dataflow::SubGraphNodes;
	using namespace UE::Dataflow::Type;
	for (const FDataflowConnection* Connection : Connections)
	{
		if (Connection && SupportsDropConnectionOnNode(Connection->GetType(), Connection->GetDirection()))
		{
			FPropertyBagPropertyDesc Desc = GetPropertyBagPropertyDescFromDataflowConnection(*Connection);
			Desc.Name = MakeUniqueNameForPropertyBag(Desc.Name, PropertyBag);
			PropertyBag.AddProperties({ Desc });
		}
	}
	DynamicConnections.Refresh();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowSubGraphOutputNode::FDataflowSubGraphOutputNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, DynamicConnections(UE::Dataflow::FPin::EDirection::INPUT, this, Cast<UDataflow>(InParam.OwningObject))
{
}

void FDataflowSubGraphOutputNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	// nothing to do, the subgraph call node will be pull the inputs directly 
	// and this node has no outputs
}

void FDataflowSubGraphOutputNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		// make sure the node is up to date with the serialized data
		DynamicConnections.Refresh();
	}
}

void FDataflowSubGraphOutputNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	DynamicConnections.Refresh();
}

bool FDataflowSubGraphOutputNode::SupportsDropConnectionOnNode(FName TypeName, UE::Dataflow::FPin::EDirection Direction) const
{
	return (Direction == UE::Dataflow::FPin::EDirection::OUTPUT);
}


const FDataflowConnection* FDataflowSubGraphOutputNode::OnDropConnectionOnNode(const FDataflowConnection& DroppedConnection)
{
	using namespace  UE::Dataflow::SubGraphNodes;
	using namespace UE::Dataflow::Type;
	if (SupportsDropConnectionOnNode(DroppedConnection.GetType(), DroppedConnection.GetDirection()))
	{
		FPropertyBagPropertyDesc Desc = GetPropertyBagPropertyDescFromDataflowConnection(DroppedConnection);
		Desc.Name = MakeUniqueNameForPropertyBag(Desc.Name, PropertyBag);
		PropertyBag.AddProperties({ Desc });
		DynamicConnections.Refresh();
		return FindInput(Desc.Name);
	}
	return nullptr;
}

void FDataflowSubGraphOutputNode::AddConnectionsTo(TArrayView<FDataflowConnection*> Connections)
{
	using namespace  UE::Dataflow::SubGraphNodes;
	using namespace UE::Dataflow::Type;
	for (const FDataflowConnection* Connection : Connections)
	{
		if (Connection && SupportsDropConnectionOnNode(Connection->GetType(), Connection->GetDirection()))
		{
			FPropertyBagPropertyDesc Desc = GetPropertyBagPropertyDescFromDataflowConnection(*Connection);
			Desc.Name = MakeUniqueNameForPropertyBag(Desc.Name, PropertyBag);
			PropertyBag.AddProperties({ Desc });
		}
	}
	DynamicConnections.Refresh();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////
FDataflowSubGraphGetCurrentIndexNode::FDataflowSubGraphGetCurrentIndexNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid)
	: FDataflowNode(Param, InGuid)
{
	RegisterOutputConnection(&Index);
}

void FDataflowSubGraphGetCurrentIndexNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Dataflow::SubGraphNodes::Private;

	if (FContextSubGraph* SubGraphContext = Context.AsType<FContextSubGraph>())
	{
		SetValue(Context, SubGraphContext->GetCurrentArrayIndex(), &Index);
	}
	else
	{
		SetValue(Context, 0, &Index);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowCallSubGraphNode::FDataflowCallSubGraphNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, DynamicInputs(UE::Dataflow::FPin::EDirection::INPUT, this, Cast<UDataflow>(InParam.OwningObject))
	, DynamicOutputs(UE::Dataflow::FPin::EDirection::OUTPUT, this, Cast<UDataflow>(InParam.OwningObject))
{
	DataflowAssetWeakPtr = Cast<UDataflow>(InParam.OwningObject);
}

FDataflowCallSubGraphNode::~FDataflowCallSubGraphNode()
{
	UDataflowSubGraph* SubGraph = nullptr;
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		SubGraph = DataflowAsset->FindSubGraphByGuid(SubGraphGuid);
	}
	UnregisterHandlers(SubGraph);
}

void FDataflowCallSubGraphNode::RegisterHandlers(UDataflowSubGraph* SubGraph)
{
	if (IsValid())
	{
		if (SubGraph)
		{
			OnGraphChangedHandle = SubGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateRaw(this, &FDataflowCallSubGraphNode::OnSubGraphChanged));
		}

		FDataflowSubGraphDelegates::OnSubGraphLoaded.AddRaw(this, &FDataflowCallSubGraphNode::OnSubGraphLoaded);
		FDataflowAssetDelegates::OnSubGraphsChanged.AddRaw(this, &FDataflowCallSubGraphNode::OnSomeSubGraphsChanged);
	}
}

void FDataflowCallSubGraphNode::UnregisterHandlers(UDataflowSubGraph* SubGraph)
{
	FDataflowAssetDelegates::OnSubGraphsChanged.RemoveAll(this);
	FDataflowSubGraphDelegates::OnSubGraphLoaded.RemoveAll(this);

	if (SubGraph)
	{
		SubGraph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
	}
}

void FDataflowCallSubGraphNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		SetSubGraphGuid(SubGraphGuid);
	}
}

void FDataflowCallSubGraphNode::SetSubGraphGuid(const FGuid& InSubGraphGuid)
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		UnregisterHandlers(DataflowAsset->FindSubGraphByGuid(SubGraphGuid));

		SubGraphGuid = InSubGraphGuid;

		UDataflowSubGraph* NewSubGraph = DataflowAsset->FindSubGraphByGuid(SubGraphGuid);
		RegisterHandlers(NewSubGraph);
		SubGraphName = NewSubGraph? NewSubGraph->GetFName(): NAME_None;

		SyncInputsPropertyBagWithSubGraph();
		SyncOutputsPropertyBagWithSubGraph();
	}
}

void FDataflowCallSubGraphNode::RefreshSubGraphName()
{
	SubGraphName = NAME_None;
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (UDataflowSubGraph* NewSubGraph = DataflowAsset->FindSubGraphByGuid(SubGraphGuid))
		{
			SubGraphName = NewSubGraph->GetFName();
		}
	}
}

bool FDataflowCallSubGraphNode::IsValid() const
{
	return SubGraphGuid.IsValid();
}

void FDataflowCallSubGraphNode::OnSubGraphChanged(const struct FEdGraphEditAction& InAction)
{
	bool bInputsChanged = false;
	bool bOutputsChanged = false;

	for (const UEdGraphNode* EdNode : InAction.Nodes)
	{
		// Todo : use the template function in SubGraph code ? 
		if (const UDataflowEdNode* DataflowEdNode = Cast<const UDataflowEdNode>(EdNode))
		{
			if (TSharedPtr<const FDataflowNode> DataflowNode = DataflowEdNode->GetDataflowNode())
			{
				if (const FDataflowSubGraphInputNode* SubGraphInputNode = DataflowNode->AsType<FDataflowSubGraphInputNode>())
				{
					bInputsChanged = true;
				}
				else if (const FDataflowSubGraphOutputNode* SubGraphOutputNode = DataflowNode->AsType<FDataflowSubGraphOutputNode>())
				{
					bOutputsChanged = true;
				}
			}
		}
	}
	if (bInputsChanged)
	{
		SyncInputsPropertyBagWithSubGraph();
	}
	if (bOutputsChanged)
	{
		SyncOutputsPropertyBagWithSubGraph();
	}
}

void FDataflowCallSubGraphNode::OnSomeSubGraphsChanged(const UDataflow* InDataflowAsset, const FGuid& InSubGraphGuid, UE::Dataflow::ESubGraphChangedReason InReason)
{
	if (InSubGraphGuid == SubGraphGuid && InDataflowAsset == DataflowAssetWeakPtr)
	{
		switch (InReason)
		{
		case UE::Dataflow::ESubGraphChangedReason::Created:
			// nothing to do, we should not have a newly created guid
			break;

		case UE::Dataflow::ESubGraphChangedReason::Renamed:
			// Todo(Dataflow) : we do not care about the name right now but we may if the name is displayed 
			break;

		case UE::Dataflow::ESubGraphChangedReason::Deleting:
			SetSubGraphGuid(FGuid());
			break;

		case UE::Dataflow::ESubGraphChangedReason::Deleted:
			break;

		case UE::Dataflow::ESubGraphChangedReason::ChangedType:
			break;
		}
	}
}

void FDataflowCallSubGraphNode::OnSubGraphLoaded(const UDataflowSubGraph& InSubGraph)
{
	if (InSubGraph.GetOuter() == DataflowAssetWeakPtr.Get())
	{
		if (InSubGraph.GetSubGraphGuid() == SubGraphGuid)
		{
			SetSubGraphGuid(SubGraphGuid);
		}
	}
}

void FDataflowCallSubGraphNode::SyncInputsPropertyBagWithSubGraph()
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (UDataflowSubGraph* SubGraph = DataflowAsset->FindSubGraphByGuid(SubGraphGuid))
		{
			InputsPropertyBag.Reset();
			if (FDataflowSubGraphInputNode* SubGraphInputNode = SubGraph->GetInputNode())
			{
				InputsPropertyBag = SubGraphInputNode->GetPropertyBag(&DynamicInputs);
			}
		}
	}
	DynamicInputs.Refresh();
}

void FDataflowCallSubGraphNode::SyncOutputsPropertyBagWithSubGraph()
{
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (UDataflowSubGraph* SubGraph = DataflowAsset->FindSubGraphByGuid(SubGraphGuid))
		{
			OutputsPropertyBag.Reset();
			if (FDataflowSubGraphOutputNode* SubGraphOutputNode = SubGraph->GetOutputNode())
			{
				OutputsPropertyBag = SubGraphOutputNode->GetPropertyBag(&DynamicOutputs);
			}
		}
	}
	DynamicOutputs.Refresh();
}

const FInstancedPropertyBag& FDataflowCallSubGraphNode::GetPropertyBag(const FDataflowDynamicConnections* Caller)
{
	static FInstancedPropertyBag DefaultPropertyBag;
	if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
	{
		if (Caller == &DynamicInputs)
		{
			return InputsPropertyBag;
		}
		else if (Caller == &DynamicOutputs)
		{
			return OutputsPropertyBag;
		}
	}
	return DefaultPropertyBag;
}

void FDataflowCallSubGraphNode::EvaluateParentContext(UE::Dataflow::FContext& Context, UE::Dataflow::FContext& SubGraphContext, const FDataflowOutput& SubGraphOutput) const
{
	using namespace UE::Dataflow::SubGraphNodes::Private;

	bool bIsFirstIteration = true;
	if (FContextSubGraph* SubGraphContextPtr = SubGraphContext.AsType<FContextSubGraph>())
	{
		bIsFirstIteration = (SubGraphContextPtr->GetCurrentArrayIndex() == 0);
	}

	// Do we have a matching internal output?
	const FDataflowOutput* InternalOutput = FindOutput(SubGraphOutput.GetName());

	const bool bGetDataFromExternalContext = bIsFirstIteration || (InternalOutput == nullptr);

	bool bDataCopied = false;
	// First iteration : we pull from the external/caller graph
	if (bGetDataFromExternalContext)
	{
		if (const FDataflowInput* Input = FindInput(SubGraphOutput.GetName()))
		{
			Input->PullValue(Context);
			if (const FDataflowOutput* Output = Input->GetConnection())
			{
				bDataCopied = Context.CopyDataToAnotherContext(Output->CacheKey(), SubGraphContext, SubGraphOutput.CacheKey(),
					SubGraphOutput.GetProperty(), SubGraphOutput.GetOwningNodeGuid(), SubGraphOutput.GetOwningNodeValueHash(), SubGraphOutput.GetOwningNodeTimestamp()
				);
			}
		}
	}
	else if (InternalOutput) // Second and subsequent iterations : copy from the previous outputs
	{
		// No need to pull the data since the output has already been evaluated
		// todo(dataflow) : this is a fair assumption but feels there could be special cases
		//					but also pulling the data here could result in weird infinite loops
		bDataCopied = Context.CopyDataToAnotherContext(InternalOutput->CacheKey(), SubGraphContext, SubGraphOutput.CacheKey(),
			SubGraphOutput.GetProperty(), SubGraphOutput.GetOwningNodeGuid(), SubGraphOutput.GetOwningNodeValueHash(), SubGraphOutput.GetOwningNodeTimestamp()
		);
	}

	if (!bDataCopied)
	{
		// Default value when everything else failed
		SubGraphOutput.SetNullValue(SubGraphContext);
	}
}

void FDataflowCallSubGraphNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out)
	{
		if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
		{
			if (UDataflowSubGraph* SubGraphToCall = DataflowAsset->FindSubGraphByGuid(SubGraphGuid))
			{
				if (const FDataflowSubGraphOutputNode* SubGraphOutputNode = SubGraphToCall->GetOutputNode())
				{
					UObject* ContextOwner = nullptr;
					if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
					{
						ContextOwner = EngineContext->Owner;
					}

					// pulling the outputs and when the graph evaluate the remote left end of its graph OnInputValueRequested will be called
					UE::Dataflow::SubGraphNodes::Private::FContextSubGraph SubGraphContext(Context, ContextOwner, *this);
					int32 NumIterations = 1;
					if (SubGraphToCall->IsForEachSubGraph())
					{
						FDataflowInput* FirstArrayInput = nullptr;
						for (FDataflowInput* Input : GetInputs())
						{
							if (FDataflowArrayTypePolicy::SupportsTypeStatic(Input->GetType()))
							{
								FirstArrayInput = Input;
								break;
							}
						}
						if (FirstArrayInput)
						{
							if (!FirstArrayInput->IsConnected())
							{
								NumIterations = 0;
							}
							else
							{
								FirstArrayInput->PullValue(Context);
								check(FirstArrayInput->GetConnection());
								const TUniquePtr<UE::Dataflow::FContextCacheElementBase>* CacheEntryToClone = Context.GetDataImpl(FirstArrayInput->GetConnection()->CacheKey());
								if (CacheEntryToClone && CacheEntryToClone->IsValid())
								{
									NumIterations = (*CacheEntryToClone)->GetNumArrayElements(Context);
								}
							}
						}
					}

					TArray<FDataflowOutput*> OutputsToEvaluate = GetOutputs();

					if (NumIterations > 0)
					{
						// Even if the evaluate function is for a single output 
						// We are pulling all outputs otherwise this may force us to evaluate the loops multiple time 
						// whioch may be more expensive
						for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
						{
							SubGraphContext.SetCurrentArrayIndex(Iteration);
							for (const FDataflowOutput* OutToEvaluate : OutputsToEvaluate)
							{
								if (OutToEvaluate)
								{
									SubGraphContext.EvaluateSubGraph(*SubGraphOutputNode, *this, *OutToEvaluate);
								}
							}
							SubGraphContext.ClearAllData();
						}
						SubGraphContext.TransferPerfData();
					}
					else
					{
						// no iteration , mean we need to set default values to the output
						for (const FDataflowOutput* OutToEvaluate : OutputsToEvaluate)
						{
							if (OutToEvaluate)
							{
								OutToEvaluate->SetNullValue(Context);
							}
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE