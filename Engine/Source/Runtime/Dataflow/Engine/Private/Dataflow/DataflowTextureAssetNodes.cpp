// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowTextureAssetNodes.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "Engine/Texture2D.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowTextureAssetNodes)


#define LOCTEXT_NAMESPACE "DataflowTextureAssetNodes"

namespace UE::Dataflow
{
	void RegisterTextureAssetNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowTextureTerminalNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowTexture2DTerminalNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowGetTextureAssetNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowGetTextureAssetFromPathNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowTextureToImageNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowImageToTextureNode);

		DATAFLOW_NODE_REGISTER_GETTER_FOR_ASSET(UTexture2D, FDataflowGetTextureAssetNode);
	}
}

namespace DataflowTextureAssetNodes::Private
{
	void UpdateTexture2DFromImage(UTexture2D& InTexture, const FImage& InImage)
	{
		// convert to the BGRA8
		FImage ConvertedImage(InImage);
		ConvertedImage.ChangeFormat(ERawImageFormat::Type::BGRA8, EGammaSpace::Linear);

#if WITH_EDITOR
		InTexture.PreEditChange(nullptr);
#endif

#if WITH_EDITORONLY_DATA
		InTexture.Source.Init(ConvertedImage);
#endif
		InTexture.UpdateResource();

#if WITH_EDITOR
		InTexture.PostEditChange();
#endif
	}
}


////////////////////////////////////////////////////////////////////////////////////////////

FDataflowTextureTerminalNode::FDataflowTextureTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Image);
	RegisterInputConnection(&TextureAsset);
	RegisterOutputConnection(&Image, &Image);
	RegisterOutputConnection(&TextureAsset, &TextureAsset);
}

void FDataflowTextureTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	SafeForwardInput(Context, &Image, &Image);
	TObjectPtr<UTexture2D> OutTexture = GetValue(Context, &TextureAsset);
	SetValue(Context, OutTexture, &TextureAsset);
}

void FDataflowTextureTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	UTexture2D* AssetToSet = Cast<UTexture2D>(Asset.Get());
	if (!AssetToSet)
	{
		// use the input instead
		AssetToSet = GetValue(Context, &TextureAsset);
	}

	if (AssetToSet)
	{
		const FDataflowImage& InImage = GetValue(Context, &Image);
		if (InImage.GetWidth() > 0 && InImage.GetHeight() > 0)
		{
			DataflowTextureAssetNodes::Private::UpdateTexture2DFromImage(*AssetToSet, InImage.GetImage());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FDataflowTexture2DTerminalNode::FDataflowTexture2DTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Image);
	RegisterInputConnection(&AssetPath);
	RegisterOutputConnection(&TextureAsset);
}

void FDataflowTexture2DTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	// Asset is created and stored by the SetAssetValue
	// Evaluate only return an existing one or nullptr
	const FString& InAssetPath = GetValue(Context, &AssetPath);
	TObjectPtr<UTexture2D> OutTexture = ::LoadObject<UTexture2D>(nullptr, InAssetPath);
	SetValue(Context, OutTexture, &TextureAsset);
}

void FDataflowTexture2DTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	TObjectPtr<UTexture2D> NullTexturePtr = nullptr;

#if WITH_EDITOR
	const FString& InAssetPath = GetValue(Context, &AssetPath);
	if (!FPackageName::IsValidObjectPath(InAssetPath))
	{
		Context.Error(FString::Format(TEXT("Asset Path input is not a valid path : {0}"), { InAssetPath }), this);
		SetValue(Context, NullTexturePtr, &TextureAsset);
		return;
	}
	const FString PackageName = FPackageName::ObjectPathToPackageName(InAssetPath);

	UPackage* Package = Cast<UPackage>(FindPackage(nullptr, *PackageName));
	if (Package == nullptr)
	{
		Package = CreatePackage(*PackageName);
	}
	if (Package == nullptr)
	{
		Context.Error(FString::Format(TEXT("Failed to find or create package {0}"), { PackageName }), this);
		SetValue(Context, NullTexturePtr, &TextureAsset);
		return;
	}

	const FName AssetName = FName(FPackageName::GetLongPackageAssetName(PackageName));
	UObject* ExistingObject = StaticFindObjectFastInternal( /*Class=*/ NULL, Package, AssetName, EFindObjectFlags::ExactClass);
	if (ExistingObject && !ExistingObject->GetClass()->IsChildOf<UTexture2D>())
	{
		Context.Error(FString::Format(TEXT("Asset {0} already exists but is not a Texture2D compatible type"), { InAssetPath }), this);
		SetValue(Context, NullTexturePtr, &TextureAsset);
		return;
	}
	// It is an error if we are trying to replace an object of a different class
	TObjectPtr<UTexture2D> OutTexture2D = Cast<UTexture2D>(ExistingObject);
	if (!OutTexture2D)
	{
		OutTexture2D = NewObject<UTexture2D>(Package, UTexture2D::StaticClass(), AssetName, RF_Public | RF_Standalone | RF_Transactional);
		if (!OutTexture2D)
		{
			Context.Error(FString::Format(TEXT("Failed to create texture asset {0}"), { InAssetPath }), this);
			SetValue(Context, NullTexturePtr, &TextureAsset);
			return;
		}
	}

	// make sure the asset is set properly
	OutTexture2D->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(OutTexture2D.Get());

	// update with image
	const FDataflowImage& InImage = GetValue(Context, &Image);
	DataflowTextureAssetNodes::Private::UpdateTexture2DFromImage(*OutTexture2D, InImage.GetImage());

	// finally set the texture to the output
	SetValue(Context, OutTexture2D, &TextureAsset);

#else
	Context.Error(TEXT("Creation of texture asset only supported in Editor"), this);
	SetValue(Context, NullTexturePtr, &TextureAsset);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////

FDataflowGetTextureAssetNode::FDataflowGetTextureAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&TextureAsset);
}

void FDataflowGetTextureAssetNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TextureAsset))
	{
		SetValue(Context, TextureAsset, &TextureAsset);
	}
}

bool FDataflowGetTextureAssetNode::SupportsAssetProperty(UObject* Asset) const
{
	return (Cast<UTexture2D>(Asset) != nullptr);
}

void FDataflowGetTextureAssetNode::SetAssetProperty(UObject* Asset)
{
	if (UTexture2D* InMaterialAsset = Cast<UTexture2D>(Asset))
	{
		TextureAsset = InMaterialAsset;
	}
}


////////////////////////////////////////////////////////////////////////////////////////////

FDataflowGetTextureAssetFromPathNode::FDataflowGetTextureAssetFromPathNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TexturePath);
	RegisterOutputConnection(&TextureAsset);
}

void FDataflowGetTextureAssetFromPathNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TextureAsset))
	{
		const FString& InPath = GetValue(Context, &TexturePath);
		TObjectPtr<UTexture2D> OutTextureAsset = LoadObject<UTexture2D>(nullptr, InPath);
		if (OutTextureAsset == nullptr)
		{
			Context.Error(FText::Format(LOCTEXT("TextureFromPath_NullAsset", "Failed to load asset {0}, returning null object"), FText::FromString(InPath)), this, Out);
		}

		SetValue(Context, OutTextureAsset, &TextureAsset);
	}
}

bool FDataflowGetTextureAssetFromPathNode::SupportsAssetProperty(UObject* Asset) const
{
	return (Cast<UTexture2D>(Asset) != nullptr);
}

void FDataflowGetTextureAssetFromPathNode::SetAssetProperty(UObject* Asset)
{
	if (UTexture2D* InMaterialAsset = Cast<UTexture2D>(Asset))
	{
		TexturePath = InMaterialAsset->GetPathName();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FDataflowTextureToImageNode::FDataflowTextureToImageNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TextureAsset);
	RegisterOutputConnection(&Image);
}

void FDataflowTextureToImageNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Image))
	{
		if (const TObjectPtr<UTexture2D> InTextureAsset = GetValue(Context, &TextureAsset))
		{
			FImage TempImage;
			if (const FSharedImageConstRef CpuCopy = InTextureAsset->GetCPUCopy())
			{
				CpuCopy->CopyTo(TempImage, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
			}
#if WITH_EDITORONLY_DATA
			else if (InTextureAsset->Source.IsValid())
			{
				FImage MipImage;
				InTextureAsset->Source.GetMipImage(MipImage, 0);
				MipImage.CopyTo(TempImage, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
			}
#endif
			else
			{
				// TODO: Handle more ways to get image data from textures
				Context.Warning(TEXT("Unable to read image from texture"));

				FDataflowImage EmptyImage;
				SetValue(Context, EmptyImage, &Image);
				return;
			}

			const TArrayView64<FVector4f> Buffer = { (FVector4f*)TempImage.AsRGBA32F().GetData(), TempImage.AsRGBA32F().Num() };
			FDataflowImage OutImage;
			OutImage.CreateRGBA32F(TempImage.GetWidth(), TempImage.GetHeight());
			OutImage.CopyRGBAPixels(Buffer);

			SetValue(Context, OutImage, &Image);
			return;
		}

		FDataflowImage EmptyImage;
		SetValue(Context, EmptyImage, &Image);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FDataflowImageToTextureNode::FDataflowImageToTextureNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Image);
	RegisterInputConnection(&TextureName);
	RegisterOutputConnection(&TransientTexture);
}


void FDataflowImageToTextureNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TransientTexture))
	{
		const FName InName = GetValue(Context, &TextureName);
		UObject* Package = GetTransientPackage();
		const FName UniqueName = MakeUniqueObjectName(Package, UTexture2D::StaticClass(), InName);
		TObjectPtr<UTexture2D> OutTexture = NewObject<UTexture2D>(Package, UniqueName, RF_Transient);

		const FDataflowImage InImage = GetValue(Context, &Image);

		if (InImage.GetWidth() > 0 && InImage.GetHeight() > 0)
		{
			DataflowTextureAssetNodes::Private::UpdateTexture2DFromImage(*OutTexture, InImage.GetImage());
		}
		else
		{
			Context.Warning(TEXT("Input image is empty"));
		}

		SetValue(Context, OutTexture, &TransientTexture);
		return;
	}
}

#undef LOCTEXT_NAMESPACE
