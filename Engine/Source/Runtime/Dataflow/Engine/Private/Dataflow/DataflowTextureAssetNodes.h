// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowTerminalNode.h"
#include "Dataflow/DataflowImage.h"

#include "DataflowTextureAssetNodes.generated.h"

class UTexture2D;

/*
* terminal node to a save a dependent 2D texture
*/
USTRUCT(meta = (DataflowTerminal))
struct FDataflowTextureTerminalNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowTextureTerminalNode, "TextureTerminal", "Terminal", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Image"))
	FDataflowImage Image;

	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "TextureAsset"))
	TObjectPtr<UTexture2D> TextureAsset = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;

public:
	FDataflowTextureTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* Terminal node to a save a dependent 2D texture
* The terminal node will create an asset if it does not exists yet 
*/
USTRUCT(meta = (DataflowTerminal))
struct FDataflowTexture2DTerminalNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowTexture2DTerminalNode, "Texture2DTerminal", "Terminal", "Path Asset Image")

private:
	/** Image data to store in the texture */
	UPROPERTY(meta = (DataflowInput))
	FDataflowImage Image;

	/** Path to the texture asset to create or update */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString AssetPath;

	/** Texture asset matching AssetPath and updated with the image data */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UTexture2D> TextureAsset = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;

public:
	FDataflowTexture2DTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* Get a texture asset
*/
USTRUCT(meta = (DataflowTerminal))
struct FDataflowGetTextureAssetNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowGetTextureAssetNode, "GetTextureAsset", "Image", "")

private:
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowOutput))
	TObjectPtr<UTexture2D> TextureAsset = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FDataflowGetTextureAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual bool SupportsAssetProperty(UObject* Asset) const override;
	virtual void SetAssetProperty(UObject* Asset) override;
};

/*
* Get a texture asset from a path
*/
USTRUCT(meta = (DataflowTerminal))
struct FDataflowGetTextureAssetFromPathNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowGetTextureAssetFromPathNode, "GetTextureAssetFromPath", "Image", "Path String")

private:
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString TexturePath;

	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UTexture2D> TextureAsset = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FDataflowGetTextureAssetFromPathNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual bool SupportsAssetProperty(UObject* Asset) const override;
	virtual void SetAssetProperty(UObject* Asset) override;
};

/**
* Import a texture asset as an image. Texture must have CPU availability.
*/
USTRUCT(Meta = (Experimental))
struct FDataflowTextureToImageNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowTextureToImageNode, "TextureToImage", "Image", "Texture Image")

public:

	FDataflowTextureToImageNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(EditAnywhere, Category = "Texture", meta = (DataflowInput))
	TObjectPtr<UTexture2D> TextureAsset = nullptr;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowImage Image;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};


/**
* Create a transient texture asset from an image
*/
USTRUCT(meta = (DataflowTerminal))
struct FDataflowImageToTextureNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowImageToTextureNode, "ImageToTexture", "Image", "Image Texture")

private:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowImage Image;

	UPROPERTY(EditAnywhere, Category = "Texture", meta = (DataflowInput))
	FName TextureName;

	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UTexture2D> TransientTexture = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FDataflowImageToTextureNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};



namespace UE::Dataflow
{
	void RegisterTextureAssetNodes();
}

