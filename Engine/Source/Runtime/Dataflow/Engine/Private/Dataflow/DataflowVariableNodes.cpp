// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowVariableNodes.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowInstance.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowObjectInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowVariableNodes)

namespace UE::Dataflow
{
	void RegisterVariableNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetDataflowVariableNode);
	}
}

namespace GetDataflowVariableNode::Private
{
	static const FName EnableCategoriesName = TEXT("EnableCategories");
	static const FName CategoryMetaDataName = TEXT("Category");
	static const FName TooltipMetaDataName = TEXT("ToolTip");
}

FGetDataflowVariableNode::FGetDataflowVariableNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Value);

	UDataflow* DataflowAsset = Cast<UDataflow>(InParam.OwningObject);
	WeakDataflowPtr = DataflowAsset;
}

FGetDataflowVariableNode::~FGetDataflowVariableNode()
{
	UnregisterHandlers();
}

void FGetDataflowVariableNode::RegisterHandlers()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FGetDataflowVariableNode::OnObjectPropertyChanged);
#endif
}

void FGetDataflowVariableNode::UnregisterHandlers()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
#endif
}

void FGetDataflowVariableNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		// make sure the variable is up to date
		if (!VariableName.IsNone())
		{
			if (UDataflow* DataflowAsset = WeakDataflowPtr.Get())
			{
				SetVariable(DataflowAsset, VariableName);
			}
		}
	}
}

void FGetDataflowVariableNode::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (UDataflow* DataflowAsset = WeakDataflowPtr.Get())
	{
		if (InObject == DataflowAsset)
		{
			const FName VariablePropertyBagMemberName = GET_MEMBER_NAME_CHECKED(UDataflow, Variables);
			if (InPropertyChangedEvent.GetMemberPropertyName() == VariablePropertyBagMemberName)
			{
				const FName PropertyName = InPropertyChangedEvent.GetPropertyName();
				if (PropertyName == VariableName || PropertyName.IsNone())
				{
					// reset the variable - how do we handle renames ? 
					SetVariable(DataflowAsset, VariableName);
				}
			}
		}
	}
}

bool FGetDataflowVariableNode::TryAddVariableToDataflowAsset(UDataflow& DataflowAsset)
{
	const FPropertyBagPropertyDesc* VariableDesc = DataflowAsset.Variables.FindPropertyDescByName(VariableName);
	if (VariableDesc == nullptr)
	{
		if (const FPropertyBagPropertyDesc* Desc = VariablePropertyBag.FindPropertyDescByName(VariableName))
		{
			DataflowAsset.Variables.AddProperties({ *Desc });
			DataflowAsset.Variables.SetValue(VariableName, Desc->CachedProperty, VariablePropertyBag.GetValue().GetMemory());
			return true;
		}
	}
	return false;
}

void FGetDataflowVariableNode::SetVariable(UDataflow* Dataflow, FName InVariableName)
{
	UnregisterHandlers();

	WeakDataflowPtr = Dataflow;
	EvaluateFunction = nullptr;

	VariablePropertyBag.Reset();

	if (Dataflow)
	{
		if (const FPropertyBagPropertyDesc* Desc = Dataflow->Variables.FindPropertyDescByName(InVariableName))
		{
			VariableName = InVariableName;
#if WITH_EDITOR
			Category = Desc->GetMetaData(GetDataflowVariableNode::Private::CategoryMetaDataName);
			Description = Desc->GetMetaData(GetDataflowVariableNode::Private::TooltipMetaDataName);
#endif

			VariablePropertyBag.AddProperties({ *Desc });
			VariablePropertyBag.SetValue(VariableName, Desc->CachedProperty, Dataflow->Variables.GetValue().GetMemory());

			UpdateOutputTypes(*Desc);
			Invalidate();
		}

		RegisterHandlers();
	}
}

void FGetDataflowVariableNode::ChangeOutputType(FDataflowOutput* Output, FName NewType)
{
	if (Output && Output->GetType() != NewType)
	{
		SetConnectionConcreteType(Output, NewType);
		Output->LockType();
	}
}

void FGetDataflowVariableNode::UpdateOutputTypes(const FPropertyBagPropertyDesc& Desc)
{
	using namespace UE::Dataflow;
	// compute type
	if (!Desc.CachedProperty)
	{
		return;
	}

	FDataflowOutput* Output = FindOutput(&Value);
	check(Output);

	ContainerType = Desc.ContainerTypes.GetFirstContainerType();

	const bool bIsArrayType = (ContainerType == EPropertyBagContainerType::Array);
	const FName ConcreteType = FDataflowConnection::GetTypeNameFromProperty(Desc.CachedProperty);

	switch (Desc.ValueType)
	{
	case EPropertyBagPropertyType::Bool:
		ChangeOutputType(Output, GetTypeName<bool>(bIsArrayType));
		EvaluateFunction = &EvaluateBool;
		break;
	case EPropertyBagPropertyType::Byte:
		ChangeOutputType(Output, GetTypeName<uint8>(bIsArrayType));
		EvaluateFunction = &EvaluateByte;
		break;
	case EPropertyBagPropertyType::Int32:
		ChangeOutputType(Output, GetTypeName<int32>(bIsArrayType));
		EvaluateFunction = &EvaluateInt32;
		break;
	case EPropertyBagPropertyType::Int64:
		ChangeOutputType(Output, GetTypeName<int64>(bIsArrayType));
		EvaluateFunction = &EvaluateInt64;
		break;
	case EPropertyBagPropertyType::Float:
		ChangeOutputType(Output, GetTypeName<float>(bIsArrayType));
		EvaluateFunction = &EvaluateFloat;
		break;
	case EPropertyBagPropertyType::Double:
		// the UI shows float but behind the scene set a double property let only deal with float 
		ChangeOutputType(Output, GetTypeName<float>(bIsArrayType));
		EvaluateFunction = &EvaluateDouble;
		break;
	case EPropertyBagPropertyType::Name:
		ChangeOutputType(Output, GetTypeName<FName>(bIsArrayType));
		EvaluateFunction = &EvaluateName;
		break;
	case EPropertyBagPropertyType::String:
		ChangeOutputType(Output, GetTypeName<FString>(bIsArrayType));
		EvaluateFunction = &EvaluateString;
		break;
	case EPropertyBagPropertyType::Text:
		ChangeOutputType(Output, GetTypeName<FText>(bIsArrayType));
		EvaluateFunction = &EvaluateText;
		break;
	case EPropertyBagPropertyType::Object:
		ChangeOutputType(Output, ConcreteType);
		EvaluateFunction = &EvaluateObject;
		break;
	case EPropertyBagPropertyType::Struct:
		ChangeOutputType(Output, ConcreteType);
		EvaluateFunction = &EvaluateStruct;
		break;
		// unsupported type for now 
	case EPropertyBagPropertyType::Enum:
	case EPropertyBagPropertyType::SoftObject:
	case EPropertyBagPropertyType::Class:
	case EPropertyBagPropertyType::SoftClass:
	case EPropertyBagPropertyType::UInt32:	// Type not fully supported at UI, will work with restrictions to type editing
	case EPropertyBagPropertyType::UInt64: // Type not fully supported at UI, will work with restrictions to type editing
	default:
		EvaluateFunction = nullptr;
		break;
	}

	if (TStrongObjectPtr<UDataflow> DataflowAsset = WeakDataflowPtr.Pin())
	{
		DataflowAsset->RefreshEdNodeByGuid(GetGuid());
	}
}

void FGetDataflowVariableNode::NotifyAssetVariablesChanged(UDataflow* DataflowAsset)
{
	if (DataflowAsset)
	{
		DataflowAsset->Modify();
		FDataflowAssetDelegates::OnVariablesChanged.Broadcast(DataflowAsset, VariableName);

		FPropertyChangedEvent PropertyChangedEvent(nullptr);
		if (UClass* DataflowClass = DataflowAsset->GetClass())
		{
			FProperty* MemberProperty = DataflowClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDataflow, Variables));
			PropertyChangedEvent.SetActiveMemberProperty(MemberProperty);
		}
#if WITH_EDITOR
		DataflowAsset->PostEditChangeProperty(PropertyChangedEvent);
#endif
	}
}

void FGetDataflowVariableNode::SetVariableMetaData(FName MetaDataName, const FString& InValue)
{
#if WITH_EDITOR
	if (const TStrongObjectPtr<UDataflow> DataflowAsset = WeakDataflowPtr.Pin())
	{
		if (const UPropertyBag* PropertyBag = DataflowAsset->Variables.GetPropertyBagStruct())
		{
			TArray<FPropertyBagPropertyDesc> Descs(PropertyBag->GetPropertyDescs());
			if (FPropertyBagPropertyDesc* PropertyDesc = Descs.FindByPredicate([this](const FPropertyBagPropertyDesc& Desc) { return Desc.Name == VariableName; }))
			{
				if (MetaDataName == GetDataflowVariableNode::Private::CategoryMetaDataName)
				{
					PropertyDesc->SetMetaData(GetDataflowVariableNode::Private::EnableCategoriesName, FString());
				}
				PropertyDesc->SetMetaData(MetaDataName, InValue);

				// Create new bag struct
				const UPropertyBag* NewBagStruct = UPropertyBag::GetOrCreateFromDescs(Descs);
				DataflowAsset->Variables.MigrateToNewBagStruct(NewBagStruct);

				NotifyAssetVariablesChanged(DataflowAsset.Get());
			}
		}
	}
#endif
}

void FGetDataflowVariableNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
#if WITH_EDITOR
	const FName PropertyName = InPropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();
	const FName VariablePropertyBagMemberName = GET_MEMBER_NAME_CHECKED(FGetDataflowVariableNode, VariablePropertyBag);
	const FName CategoryMemberName = GET_MEMBER_NAME_CHECKED(FGetDataflowVariableNode, Category);
	const FName DescriptionMemberName = GET_MEMBER_NAME_CHECKED(FGetDataflowVariableNode, Description);

	if (PropertyName == VariablePropertyBagMemberName || MemberPropertyName == VariablePropertyBagMemberName || PropertyName == VariableName || MemberPropertyName == VariableName)
	{
		// update the variable value in the dataflow asset
		if (const TStrongObjectPtr<UDataflow> DataflowAsset = WeakDataflowPtr.Pin())
		{
			if (const FPropertyBagPropertyDesc* SourceDesc = VariablePropertyBag.FindPropertyDescByName(VariableName))
			{
				if (SourceDesc->CachedProperty)
				{
					EPropertyBagResult Result = DataflowAsset->Variables.SetValue(VariableName, SourceDesc->CachedProperty, VariablePropertyBag.GetValue().GetMemory());
					if (Result == EPropertyBagResult::Success)
					{
						NotifyAssetVariablesChanged(DataflowAsset.Get());
						Invalidate();
					}
				}
			}
		}
	}
	else if (PropertyName == CategoryMemberName || MemberPropertyName == CategoryMemberName)
	{
		SetVariableMetaData(GetDataflowVariableNode::Private::CategoryMetaDataName, Category);
	}
	else if (PropertyName == DescriptionMemberName || MemberPropertyName == DescriptionMemberName)
	{
		SetVariableMetaData(GetDataflowVariableNode::Private::TooltipMetaDataName, Description);
	}
#endif
}

void FGetDataflowVariableNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Value))
	{
		if (EvaluateFunction)
		{
			if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
			{
				if (IDataflowInstanceInterface* Interface = UE::Dataflow::InstanceUtils::GetDataflowInterfaceFromObject(EngineContext->Owner))
				{
					const FDataflowInstance& DataflowInstance = Interface->GetDataflowInstance();
					if (DataflowInstance.GetVariableOverrides().IsVariableOverridden(VariableName))
					{
						EvaluateFunction(*this, DataflowInstance.GetVariableOverrides().GetVariables(), Context, *Out);
						return;
					}
				}
			}
			// no override or engine context , use the default defined in the dataflow asset
			if (TStrongObjectPtr<UDataflow> DataflowAsset = WeakDataflowPtr.Pin())
			{
				if (DataflowAsset->Variables.FindPropertyDescByName(VariableName) != nullptr)
				{
					EvaluateFunction(*this, DataflowAsset->Variables, Context, *Out);
					return;
				}
			}
		}
		
		const FString Message = FString::Printf(TEXT("Dataflow Variable : Failed to evaluate variable [%s], returning defaut value"), *VariableName.ToString());
		Context.Warning(Message, this, Out);

		// nothing worked, just write a null value that will read as default value
		Out->SetNullValue(Context);
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateBool(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		bool OutBool = false;
		TValueOrError<bool, EPropertyBagResult> Result = Variables.GetValueBool(Node.VariableName);
		if (Result.HasValue())
		{
			OutBool = Result.GetValue();
		}
		Out.SetValue<bool>(MoveTemp(OutBool), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<bool> OutBoolArray;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutBoolArray.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutBoolArray[Index] = ArrayRef.GetValueBool(Index).GetValue();
				}
			}
		}
		Out.SetValue(MoveTemp(OutBoolArray), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateByte(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		uint8 OutUInt8 = 0;
		TValueOrError<uint8, EPropertyBagResult> Result = Variables.GetValueByte(Node.VariableName);
		if (Result.HasValue())
		{
			OutUInt8 = Result.GetValue();
		}
		Out.SetValue<uint8>(MoveTemp(OutUInt8), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<uint8> OutInt8Array;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutInt8Array.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutInt8Array[Index] = ArrayRef.GetValueByte(Index).GetValue();
				}
			}
		}
		Out.SetValue(MoveTemp(OutInt8Array), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateInt32(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		int32 OutInt32 = 0;
		TValueOrError<int32, EPropertyBagResult> Result = Variables.GetValueInt32(Node.VariableName);
		if (Result.HasValue())
		{
			OutInt32 = Result.GetValue();
		}
		Out.SetValue<int32>(MoveTemp(OutInt32), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<int32> OutInt32Array;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutInt32Array.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutInt32Array[Index] = ArrayRef.GetValueInt32(Index).GetValue();
				}
			}
		}
		Out.SetValue(MoveTemp(OutInt32Array), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateInt64(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		int64 OutInt64 = 0;
		TValueOrError<int64, EPropertyBagResult> Result = Variables.GetValueInt64(Node.VariableName);
		if (Result.HasValue())
		{
			OutInt64 = Result.GetValue();
		}
		Out.SetValue<int64>(MoveTemp(OutInt64), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<int64> OutInt64Array;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutInt64Array.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutInt64Array[Index] = ArrayRef.GetValueInt64(Index).GetValue();
				}
			}
		}
		Out.SetValue(MoveTemp(OutInt64Array), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateFloat(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		float OutFloat = 0.0f;
		TValueOrError<float, EPropertyBagResult> Result = Variables.GetValueFloat(Node.VariableName);
		if (Result.HasValue())
		{
			OutFloat = Result.GetValue();
		}
		Out.SetValue<float>(MoveTemp(OutFloat), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<float> OutFloatArray;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutFloatArray.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutFloatArray[Index] = ArrayRef.GetValueFloat(Index).GetValue();
				}
			}
		}
		Out.SetValue(MoveTemp(OutFloatArray), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateDouble(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	// UI shows type as float but internally always uses a double and never float 
	// so let's output as float 
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		float OutFloat = 0.0;
		TValueOrError<double, EPropertyBagResult> Result = Variables.GetValueDouble(Node.VariableName);
		if (Result.HasValue())
		{
			OutFloat = static_cast<float>(Result.GetValue());
		}
		Out.SetValue<float>(MoveTemp(OutFloat), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<float> OutFloatArray;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutFloatArray.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutFloatArray[Index] = static_cast<float>(ArrayRef.GetValueDouble(Index).GetValue());
				}
			}
		}
		Out.SetValue(MoveTemp(OutFloatArray), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateName(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		FName OutName;
		TValueOrError<FName, EPropertyBagResult> Result = Variables.GetValueName(Node.VariableName);
		if (Result.HasValue())
		{
			OutName = Result.GetValue();
		}
		Out.SetValue<FName>(MoveTemp(OutName), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<FName> OutNameArray;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutNameArray.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutNameArray[Index] = ArrayRef.GetValueName(Index).GetValue();
				}
			}
		}
		Out.SetValue(MoveTemp(OutNameArray), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateString(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		FString OutString;
		TValueOrError<FString, EPropertyBagResult> Result = Variables.GetValueString(Node.VariableName);
		if (Result.HasValue())
		{
			OutString = Result.GetValue();
		}
		Out.SetValue<FString>(MoveTemp(OutString), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<FString> OutStringArray;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutStringArray.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutStringArray[Index] = ArrayRef.GetValueString(Index).GetValue();
				}
			}
		}
		Out.SetValue(MoveTemp(OutStringArray), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateText(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		FString OutString;
		TValueOrError<FText, EPropertyBagResult> Result = Variables.GetValueText(Node.VariableName);
		if (Result.HasValue())
		{
			OutString = Result.GetValue().ToString();
		}
		Out.SetValue<FString>(MoveTemp(OutString), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<FString> OutStringArray;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutStringArray.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutStringArray[Index] = ArrayRef.GetValueText(Index).GetValue().ToString();
				}
			}
		}
		Out.SetValue(MoveTemp(OutStringArray), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateObject(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch(Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		TObjectPtr<UObject> OutObject = nullptr;
		TValueOrError<UObject*, EPropertyBagResult> Result = Variables.GetValueObject(Node.VariableName);
		if (Result.HasValue())
		{
			OutObject = Result.GetValue();
		}
		Out.SetValue<TObjectPtr<UObject>>(MoveTemp(OutObject), Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		TArray<TObjectPtr<UObject>> OutObjectArray;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				OutObjectArray.SetNum(Count);
				for (int32 Index = 0; Index < Count; Index++)
				{
					OutObjectArray[Index] = ArrayRef.GetValueObject(Index).GetValue();
				}
			}
		}
		Out.SetValue(MoveTemp(OutObjectArray), Context);
		break;
	}
	}
}

/*static*/ void FGetDataflowVariableNode::EvaluateStruct(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out)
{
	switch (Node.ContainerType)
	{
	case EPropertyBagContainerType::None:
	{
		FConstStructView OutStruct;
		TValueOrError<FStructView, EPropertyBagResult> Result = Variables.GetValueStruct(Node.VariableName);
		if (Result.HasValue())
		{
			OutStruct = Result.GetValue();
		}
		Out.SetValueFromStructView(OutStruct, Context);
		break;
	}
	case EPropertyBagContainerType::Array:
	{
		FConstStructArrayView OutStructArrayView;
		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Result = Variables.GetArrayRef(Node.VariableName);
		if (Result.HasValue())
		{
			const FPropertyBagArrayRef ArrayRef = Result.GetValue();
			if (const int32 Count = ArrayRef.Num())
			{
				TValueOrError<FStructView, EPropertyBagResult> ResultElt = ArrayRef.GetValueStruct(0);
				if (const FStructView* const Value = ResultElt.TryGetValue())
				{
					if (const UScriptStruct* const ScriptStruct = Value->GetScriptStruct())
					{
						OutStructArrayView = FConstStructArrayView(
							*ScriptStruct,
							(const void*)Value->GetMemory(),
							Count);
					}
				}
			}
		}
		Out.SetValueFromStructArrayView(OutStructArrayView, Context);
		break;
	}
	}
}

