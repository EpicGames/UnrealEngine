// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Engine/AssetUserData.h"
#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowInstance.h"
#include "UObject/ObjectPtr.h"

#include "DataflowAttachment.generated.h"

#define UE_API DATAFLOWENGINE_API

/**
 * Generic dataflow instance attachment implemented as a user asset data object so it can be embedded inside
 * any asset that is a user-data container. This allows assets in modules that do not, or cannot know about
 * dataflow (e.g. Engine module assets) to contain a dataflow attachment.
 * Using a modular feature implementation this data can be grabbed as an asset opens and a plugin specific
 * implementation can be used for the asset editor (see FDataflowPhysicsAssetEditorOverride)
 */
UCLASS(MinimalApi, Abstract)
class UDataflowAttachment : public UAssetUserData, public IDataflowContentOwner, public IDataflowInstanceInterface
{
	GENERATED_BODY()

public:

	UE_API UDataflowAttachment();

	// IDataflowContentOwner
	void WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const override PURE_VIRTUAL( UDataflowAttachment::WriteDataflowContent, );
	void ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) override PURE_VIRTUAL( UDataflowAttachment::ReadDataflowContent, );
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	
	// IDataflowInstanceInterface
	UE_API const FDataflowInstance& GetDataflowInstance() const override;
	UE_API FDataflowInstance& GetDataflowInstance() override;
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	// UObject
	UE_API virtual void PostLoad() override;
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	/** Provides trhe path to the preview actor to be used in the corresponding Dataflow editor */
	UE_API virtual FString GetPreviewActorPath() const;

private:

	// Dataflow instance for the attachment when editing.
	UPROPERTY()
	FDataflowInstance Instance;

protected:
	
	// IDataflowContentOwner
	TObjectPtr<UDataflowBaseContent> CreateDataflowContent() override PURE_VIRTUAL( UDataflowAttachment::CreateDataflowContent, return nullptr; );
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////
};

class UClass;

struct FDataflowAttachmentFactory
{
public:
	UE_API static FDataflowAttachmentFactory& Get();

	using FCreateAttachmentFunction = TFunction<UDataflowAttachment*(UObject*)>;

	UE_API void Register(FName ClassName, FCreateAttachmentFunction CreateFunction);

	UE_API UDataflowAttachment* Create(UObject* Owner);

private:
	TMap<FName, FCreateAttachmentFunction> CreateFunctionsByName;
};

#undef UE_API
