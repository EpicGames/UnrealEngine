// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/EngineTypes.h"
#include "Dataflow/DataflowObject.h"

#include "DataflowBlueprintLibrary.generated.h"



USTRUCT(BlueprintType)
struct FDataflowVariable
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dataflow")
	FName VariableName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dataflow")
	EPropertyBagPropertyType Type = EPropertyBagPropertyType::None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dataflow")
	EPropertyBagContainerType ContainerType = EPropertyBagContainerType::None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dataflow")
	FString Value;

};


UCLASS(MinimalAPI)
class UDataflowBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	* Find a specific terminal node by name evaluate it using a specific UObject
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API void EvaluateTerminalNodeByName(UDataflow* Dataflow, FName TerminalNodeName, UObject* ResultAsset);

	/**s
	* Evaluate a dataflow for a sepcific asset
	* This will evaluate all the terminal nodes and generate any dependent asset if necessary 
	* This is synchronous operation, so large dataflow may block the game thread for a while when calling it 
	* @param Dataflow Dataflow asset to evaluate
	* @param AssetToUpdate Asset to update from the Dataflow ( if a terminal exists for this type of asset ) 
	* @return true if the evaluate executed with no errors
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool EvaluateDataflow(UDataflow* Dataflow, UObject* AssetToUpdate);

	/**
	* Regenerate an asset using its corresponding dataflow
	* @return true if the asset was regenerated
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool RegenerateAssetFromDataflow(UObject* AssetToRegenerate, bool bRegenerateDependentAssets = false);

	/**
	* Override a Boolean dataflow variable for a specific asset
	* @return true if the override was successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool OverrideDataflowVariableBool(UObject* Asset, FName VariableName, bool VariableValue);

	/**
	* Override an Boolean Array dataflow variable for a specific asset
	* @return true if the override was successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool OverrideDataflowVariableBoolArray(UObject* Asset, FName VariableName, const TArray<bool>& VariableArrayValue);

	/**
	* Override an Integer dataflow variable for a specific asset
	* @return true if the override was successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool OverrideDataflowVariableInt(UObject* Asset, FName VariableName, int64 VariableValue);

	/**
	* Override an Integer Array dataflow variable for a specific asset
	* @return true if the override was successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool OverrideDataflowVariableIntArray(UObject* Asset, FName VariableName, const TArray<int32>& VariableArrayValue);

	/**
	* Override a Float dataflow variable for a specific asset
	* @return true if the override was successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool OverrideDataflowVariableFloat(UObject* Asset, FName VariableName, float VariableValue);

	/**
	* Override a Float Array dataflow variable for a specific asset
	* @return true if the override was successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool OverrideDataflowVariableFloatArray(UObject* Asset, FName VariableName, const TArray<float>& VariableArrayValue);

	/**
	* Override an Unreal Object dataflow variable for a specific asset
	* @return true if the override was successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool OverrideDataflowVariableObject(UObject* Asset, FName VariableName, UObject* VariableValue);

	/**
	* Override an Unreal Object Array dataflow variable for a specific asset
	* @return true if the override was successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API bool OverrideDataflowVariableObjectArray(UObject* Asset, FName VariableName, const TArray<UObject*>& VariableArrayValue);

	UFUNCTION(BlueprintCallable, Category = "Dataflow", meta = (Keywords = "Dataflow graph"))
	static DATAFLOWENGINE_API TArray<FDataflowVariable> GetDataflowVariableList(UObject* Asset);
};
