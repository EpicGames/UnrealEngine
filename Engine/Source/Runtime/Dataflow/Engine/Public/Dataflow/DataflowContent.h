// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowContextObject.h"
#include "UObject/Interface.h"
#include "Templates/SharedPointer.h"
#include "GameFramework/Actor.h"
#include "DataflowContent.generated.h"

#define UE_API DATAFLOWENGINE_API


class FDataflowEditorToolkit;
class USkeletalMesh;
class USkeleton;
class USkeletalMeshComponent;
class UAnimationAsset;
class UDataflowBaseContent;
class FPreviewScene;
class UAnimSingleNodeInstance;

namespace UE::DataflowContextHelpers
{
	// Return a new(or saved) content that can store the execution state of the graph. 
	template<class T>
	DATAFLOWENGINE_API TObjectPtr<T> CreateNewDataflowContent(const TObjectPtr<UObject>& ContentOwner);
}

UINTERFACE(MinimalAPI)
class UDataflowContentOwner : public UInterface
{
	GENERATED_BODY()
};

/** 
 * Dataflow interface for any content owner
 */
class IDataflowContentOwner
{
public:

	GENERATED_BODY()

	/** Function to build the dataflow content */
	UE_API TObjectPtr<UDataflowBaseContent> BuildDataflowContent();

	/** Notification when owner changed */
	DECLARE_MULTICAST_DELEGATE(FOnContentOwnerChanged);

	/** Delegate member to be called in the invalidate */
	FOnContentOwnerChanged OnContentOwnerChanged;

	/** Invalidate all the dataflow contents */
	void InvalidateDataflowContents() const
	{
		OnContentOwnerChanged.Broadcast();
	}
	
	/** Interface to update a dataflow content instance from that owner */
	virtual void WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const = 0;

	/** Interface to update a dataflow content instance from that owner */
	virtual void ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) = 0;
	
protected :

	/** Interface to create a dataflow content instance from that owner */
	virtual TObjectPtr<UDataflowBaseContent> CreateDataflowContent() = 0;
};

/** 
 * Dataflow content owning dataflow asset that that will be used to evaluate the graph
 */
UCLASS(MinimalAPI)
class UDataflowBaseContent : public UDataflowContextObject
{
	GENERATED_BODY()

public:
	UE_API UDataflowBaseContent();
	UE_API ~UDataflowBaseContent();

	/** Notification when content data changed */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnContentDataChanged, TObjectPtr<AActor>& SimulationActor);

	/** Delegate member to be called when data is changed */
	FOnContentDataChanged OnContentDataChanged;

	/** 
	*	Dirty - State Invalidation
	*   Check if non-graph specific data has been changed, this usually requires a re-render 
	*/
	bool IsConstructionDirty() const { return bIsConstructionDirty; }
	UE_API void SetConstructionDirty(bool InDirty);
	
	bool IsSimulationDirty() const { return bIsSimulationDirty; }
	UE_API void SetSimulationDirty(bool InDirty);

	/** 
	*	LastModifiedTimestamp - State Invalidation 
	*   Dataflow timestamp accessors can be used to see if the EvaluationContext has been invalidated. 
	*/
	UE_API void SetLastModifiedTimestamp(UE::Dataflow::FTimestamp InTimestamp, bool bMakeDirty =true);
	const UE::Dataflow::FTimestamp& GetLastModifiedTimestamp() const { return LastModifiedTimestamp; }

	/**  
	*	Context - Dataflow Evaluation State
	*   Dataflow context stores the evaluated state of the graph. 
	*/
	UE_API virtual void SetDataflowContext(const TSharedPtr<UE::Dataflow::FEngineContext>& InContext) override;
 
	/** Rebuild the owner dependent datas  */
	UE_API void UpdateContentDatas();

	/** Collect reference objects for GC */
	virtual void AddContentObjects(FReferenceCollector& Collector) {}

	/** Set all the preview actor exposed properties */
	UE_API virtual void SetActorProperties(TObjectPtr<AActor>& PreviewActor) const;
	
	/** Data flow owner accessors (through the context) */
	UE_API void SetDataflowOwner(const TObjectPtr<UObject>& InOwner);
	UE_API TObjectPtr<UObject> GetDataflowOwner() const;
	
	/** Data flow asset accessors (through the context) */
	UE_API virtual void SetDataflowAsset(const TObjectPtr<UDataflow>& InAsset) override;

	/** Data flow terminal accessors */
	void SetDataflowTerminal(const FString& InPath) { DataflowTerminal = InPath;  SetConstructionDirty(true); SetSimulationDirty(true);}
	const FString& GetDataflowTerminal() const { return DataflowTerminal; }

	/** Terminal asset accessors */
	void SetTerminalAsset(const TObjectPtr<UObject>& InAsset) { TerminalAsset = InAsset;  SetConstructionDirty(true); SetSimulationDirty(true);}
	const TObjectPtr<UObject>& GetTerminalAsset() const { return TerminalAsset;}

	/** Preview class accessors */
	void SetPreviewClass(const TSubclassOf<AActor>& InPreviewClass) { PreviewClass = InPreviewClass;  SetConstructionDirty(true); SetSimulationDirty(true);}
	const TSubclassOf<AActor>& GetPreviewClass() const { return PreviewClass;}

	/** Content Serialization */
	UE_API virtual void Serialize(FArchive& Ar);

	/* Context cache saving */
	bool IsSaved() const { return bIsSaved; }
	void SetIsSaved(bool bInSaved) { bIsSaved = bInSaved; }

	//~ UObject interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //if WITH_EDITOR

protected:
	
	/** Data flow terminal path for evaluation */
	UPROPERTY(Transient, SkipSerialization)
	FString DataflowTerminal;
	
	/** Data flow terminal path for evaluation */
	UPROPERTY(Transient, SkipSerialization)
	TObjectPtr<UObject> TerminalAsset = nullptr;

    /** Last data flow evaluated node time stamp */
	UE::Dataflow::FTimestamp LastModifiedTimestamp = UE::Dataflow::FTimestamp::Invalid;

    /** Dirty flag to trigger rendering. Do we need that? since when accessing the member by non const ref we will not dirty it */
	UPROPERTY()
	bool bIsConstructionDirty = true;

	/** Dirty flag to reset the simulation if necessary */
	UPROPERTY()
	bool bIsSimulationDirty = true;

	/** Saved as a cached context. Will be automatically saved to a cache directory if true. Use the pvar p.Dataflow.Editor.ContextCaching to enable. [def:false] */
	bool bIsSaved = false;

	/** Preview actor class that could be used to visualize the result */
	TSubclassOf<AActor> PreviewClass;

	/** override actor properties from BP */
	static UE_API void OverrideActorProperty(const TObjectPtr<AActor>& PreviewActor, TObjectPtr<UObject> PropertyValue, const FName& PropertyName);

	/** override struct properties from BP */
	template<typename StructType>
	static void OverrideStructProperty(const TObjectPtr<AActor>& PreviewActor, const StructType& PropertyValue, const FName& PropertyName);
};

template<typename StructType>
void UDataflowBaseContent::OverrideStructProperty(const TObjectPtr<AActor>& PreviewActor, const StructType& PropertyValue, const FName& PropertyName)
{
	if(PreviewActor)
	{
		if(const FProperty* DataflowProperty = PreviewActor->GetClass()->FindPropertyByName(PropertyName))
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(DataflowProperty))
			{
				if(StructProperty->Struct == StructType::StaticStruct())
				{
					if(StructType* PropertyStruct = DataflowProperty->ContainerPtrToValuePtr<StructType>(PreviewActor))
					{
						(*PropertyStruct) = PropertyValue;
					}
				}
			}
		}
	}
}

/** 
 * Dataflow content owning dataflow and skelmesh assets that that will be used to evaluate the graph
 */
UCLASS(MinimalAPI)
class  UDataflowSkeletalContent : public UDataflowBaseContent
{
	GENERATED_BODY()

public:
	UE_API UDataflowSkeletalContent();
	virtual ~UDataflowSkeletalContent() override= default;
	
#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //if WITH_EDITOR

	/** Collect reference objects for GC */
	UE_API virtual void AddContentObjects(FReferenceCollector& Collector) override;

	/** Data flow skeletal mesh accessors */
	UE_API void SetSkeletalMesh(const TObjectPtr<USkeletalMesh>& InMesh, const bool bHideAsset = false);
	const TObjectPtr<USkeletalMesh>& GetSkeletalMesh() const { return SkeletalMesh; }

	/** Data flow animation asset accessors */
	UE_API void SetAnimationAsset(const TObjectPtr<UAnimationAsset>& InAnimation, const bool bHideAsset = false);
	const TObjectPtr<UAnimationAsset>& GetAnimationAsset() const { return AnimationAsset; }

	//~ UObject interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/** Set all the preview actor exposed properties */
	UE_API virtual void SetActorProperties(TObjectPtr<AActor>& PreviewActor) const override;

protected:
#if WITH_EDITOR
	UFUNCTION()
	bool ShouldFilterAnimation(const FAssetData& AssetData) const;
#endif

	/** Data flow skeletal mesh*/
	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	/** Enable animation filtering */
	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	bool bListCompatibleAnimationsOnly = true;

	/** Animation asset to be used to preview simulation */
	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization, meta = (GetAssetFilter = "ShouldFilterAnimation"))
	TObjectPtr<UAnimationAsset> AnimationAsset = nullptr;

	/** Boolean to control if the skeletal mesh could be edited or not */
	bool bHideSkeletalMesh = false;

	/** Boolean to control if the animation asset could be edited or not */
	bool bHideAnimationAsset = false;
};

#undef UE_API
