// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "DataflowContextObject.generated.h"

#define UE_API DATAFLOWENGINE_API

struct FManagedArrayCollection;
class UDataflow;
namespace UE::Dataflow
{
	class IDataflowConstructionViewMode;
	template<class Base> class TEngineContext;
	typedef TEngineContext<FContextSingle> FEngineContext;
}

/** 
 * Context object used for selection/rendering 
 */

UCLASS(MinimalAPI)
class UDataflowContextObject : public UObject
{
	GENERATED_BODY()
public:

	/** Selection Collection Access */
	void SetSelectedNode(TObjectPtr<UDataflowEdNode> InSelectedNode) { SelectedNode = InSelectedNode; }
	TObjectPtr<UDataflowEdNode> GetSelectedNode() const { return SelectedNode; }

	/** Get a single selected node of the specified type. Return nullptr if the specified node is not selected, or if multiple nodes are selected*/
	template<typename NodeType>
	NodeType* GetSelectedNodeOfType() const
	{
		if (SelectedNode && SelectedNode->GetDataflowNode())
		{
			return SelectedNode->GetDataflowNode()->AsType<NodeType>();
		}
		return nullptr;
	}

	/** Render Collection used to generate the DynamicMesh3D from the selected node */
	void SetRenderCollection(const TSharedPtr<FManagedArrayCollection>& InCollection) { RenderCollection = InCollection; }
	TSharedPtr<const FManagedArrayCollection> GetRenderCollection() const { return RenderCollection; }

	/** ViewMode Access */
	void SetConstructionViewMode(const UE::Dataflow::IDataflowConstructionViewMode* InMode) { ConstructionViewMode = InMode; }
	const UE::Dataflow::IDataflowConstructionViewMode* GetConstructionViewMode() const { return ConstructionViewMode; }
	
	/**
	*	Context - Dataflow Evaluation State
	*   Dataflow context stores the evaluated state of the graph.
	*/
	virtual void SetDataflowContext(const TSharedPtr<UE::Dataflow::FEngineContext>& InDataflowContext) { DataflowContext = InDataflowContext; }
	const TSharedPtr<UE::Dataflow::FEngineContext>& GetDataflowContext() const { return DataflowContext; }
	TSharedPtr<UE::Dataflow::FEngineContext>& GetDataflowContext() { return DataflowContext; }

	/** Dataflow Asset */
	virtual void SetDataflowAsset(const TObjectPtr<UDataflow>& InAsset) { DataflowGraph = InAsset; }
	TObjectPtr<UDataflow> GetDataflowAsset() const { return DataflowGraph;  }

	TSharedPtr<UE::Dataflow::IDataflowEvaluator> GetDataflowEvalutor() const { return DataflowEvaluator; }

	/** Collection passing through the currently selected node and a flag indicating whether it's on a node input or output */
	void SetSelectedCollection(TSharedPtr<const FManagedArrayCollection> InSelectedCollection, bool bInUsingInputCollection) 
	{ 
		SelectedCollection = InSelectedCollection; 
		bUsingInputCollection = bInUsingInputCollection;
	}
	const TSharedPtr<const FManagedArrayCollection> GetSelectedCollection() const { return SelectedCollection; }
	bool IsUsingInputCollection() const { return bUsingInputCollection; }

	UE_API void EvaluateGraph(UE::Dataflow::FOnPostEvaluationFunction OnPostEvaluation);

	//~ UObject interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

protected:

	/** Render collection to be used */
	TSharedPtr<FManagedArrayCollection> RenderCollection = nullptr;

	/** Node that is selected in the graph */
	UPROPERTY(Transient, SkipSerialization)
	TObjectPtr<UDataflowEdNode> SelectedNode = nullptr;

	/** Construction view mode for the context object @todo(michael) : is it only for construction or for simulation as well*/
	const UE::Dataflow::IDataflowConstructionViewMode* ConstructionViewMode = nullptr;

	/** Engine context (data flow owner/asset) to be used for dataflow evaluation */
	TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = nullptr;

	/** EValuator being used to evaluate the graph*/
	TSharedPtr<UE::Dataflow::IDataflowEvaluator> DataflowEvaluator = nullptr;

	/** Dataflow graph for evaluation */
	UPROPERTY(Transient, SkipSerialization)
	TObjectPtr<UDataflow> DataflowGraph = nullptr;

	/** Managed Array Collection passing through the currently selected node */
	TSharedPtr<const FManagedArrayCollection> SelectedCollection = nullptr;

	/** Whether the Collection is taken from a node Input (i.e. it's in the the state before node execution) */
	bool bUsingInputCollection = false;

};

#undef UE_API
