// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowDebugDrawInterface.h"
#include "DebugRenderSceneProxy.h"
#include "Engine/EngineTypes.h"
#include "Math/Color.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Dataflow/DataflowNode.h"
#include "UObject/EnumProperty.h"

#include "DataflowDebugDraw.generated.h"

struct FDataflowBaseElement;

class FDataflowDebugDraw : public IDataflowDebugDrawInterface
{
public:

	DATAFLOWENGINE_API FDataflowDebugDraw(class FDataflowDebugRenderSceneProxy* DebugRenderSceneProxy, IDataflowDebugDrawInterface::FDataflowElementsType& DataflowElements);

	virtual ~FDataflowDebugDraw() = default;

private:

	virtual void SetColor(const FLinearColor& InColor) override;
	virtual void SetPointSize(float Size) override;
	virtual void SetLineWidth(double Width) override;
	virtual void SetWireframe(bool bInWireframe) override;
	virtual void SetShaded(bool bInShaded) override;
	virtual void SetTranslucent(bool bInShadedTranslucent) override;
	virtual void SetForegroundPriority() override;
	virtual void SetWorldPriority() override;
	virtual void ResetAllState() override;

	virtual void ReservePoints(int32 NumAdditionalPoints) override;
	virtual void DrawObject(const TRefCountPtr<IDataflowDebugDrawObject>& Object) override;
	virtual void DrawPoint(const FVector& Position) override;
	virtual void DrawLine(const FVector& Start, const FVector& End) const override;
	virtual void DrawMesh(const IDebugDrawMesh& Mesh) const override;

	virtual void DrawBox(const FVector& Extents, const FQuat& Rotation, const FVector& Center, double UniformScale) const override;
	virtual void DrawSphere(const FVector& Center, double Radius) const override;
	virtual void DrawCapsule(const FVector& Center, const double& Radius, const double& HalfHeight, const FVector& XAxis, const FVector& YAxis, const FVector &ZAxis) const override;
	virtual void DrawText3d(const FString& String, const FVector& Location) const override;

	virtual void DrawOverlayText(const FString& InString) override;
	virtual FString GetOverlayText() const override;

	/** Dataflow elements non const accessor */
	virtual IDataflowDebugDrawInterface::FDataflowElementsType& ModifyDataflowElements() { return DataflowElements; }

	/** Dataflow elements const accessor */
	virtual const IDataflowDebugDrawInterface::FDataflowElementsType& GetDataflowElements() const { return DataflowElements; }

	/** Scene proxy coming from the dataflow debug draw component */
	class FDataflowDebugRenderSceneProxy* DebugRenderSceneProxy = nullptr;

	/** List of dataflow elements to filled by the debug draw */ 
	IDataflowDebugDrawInterface::FDataflowElementsType& DataflowElements;

	FLinearColor Color = FLinearColor::White;
	double LineWidth = 1.0;
	float PointSize = 5.0;
	bool bWireframe = true;
	bool bShaded = false;
	bool bTranslucent = false;
	ESceneDepthPriorityGroup PriorityGroup = SDPG_World;

	FLinearColor ColorWithTranslucency = FLinearColor::White;
	FDebugRenderSceneProxy::EDrawType DrawType = FDebugRenderSceneProxy::EDrawType::Invalid;

	TArray<FString> OverlayStrings;
};

UENUM(BlueprintType)
enum class EDataflowDebugDrawRenderType : uint8
{
	Wireframe UMETA(DisplayName = "Wireframe"),
	Shaded UMETA(DisplayName = "Shaded"),
};

/**
 * DebugDraw basic common settings
 */
USTRUCT()
struct FDataflowNodeDebugDrawSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	EDataflowDebugDrawRenderType RenderType = EDataflowDebugDrawRenderType::Wireframe;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (EditCondition = "RenderType==EDataflowDebugDrawRenderType::Shaded", EditConditionHides));
	bool bTranslucent = true;

	UPROPERTY(EditAnywhere, Category = "Debug Draw");
	FLinearColor Color = FLinearColor::Gray;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ClampMin = "0.1", ClampMax = "10.0"));
	float LineWidthMultiplier = 1.f;

	DATAFLOWENGINE_API void SetDebugDrawSettings(IDataflowDebugDrawInterface& DataflowRenderingInterface) const;
};

/**
 * SphereCovering DebugDraw basic common settings
 */
UENUM(BlueprintType)
enum class EDataflowSphereCoveringColorMethod : uint8
{
	Single UMETA(DisplayName = "Single Color"),
	ColorByRadius UMETA(DisplayName = "Color by Radius"),
	Random UMETA(DisplayName = "Random Colors"),
};

USTRUCT()
struct FDataflowNodeSphereCoveringDebugDrawSettings
{
	GENERATED_USTRUCT_BODY()

	/** Display sphere covering */
	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering")
	bool bDisplaySphereCovering = false;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering")
	EDataflowDebugDrawRenderType RenderType = EDataflowDebugDrawRenderType::Wireframe;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (EditCondition = "RenderType==EDataflowDebugDrawRenderType::Shaded", EditConditionHides));
	bool bTranslucent = true;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (ClampMin = "0.1", ClampMax = "10.0"));
	float LineWidthMultiplier = .25f;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering");
	EDataflowSphereCoveringColorMethod ColorMethod = EDataflowSphereCoveringColorMethod::Single;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (EditCondition = "ColorMethod == EDataflowSphereCoveringColorMethod::Single", EditConditionHides));
	FLinearColor Color = FLinearColor::Red;

	/** Random seed */
	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (ClampMin = "0", EditCondition = "ColorMethod == EDataflowSphereCoveringColorMethod::Random", EditConditionHides))
	int32 ColorRandomSeed = 0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (EditCondition = "ColorMethod == EDataflowSphereCoveringColorMethod::ColorByRadius", EditConditionHides));
	FLinearColor ColorA = FLinearColor::Red;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (EditCondition = "ColorMethod == EDataflowSphereCoveringColorMethod::ColorByRadius", EditConditionHides));
	FLinearColor ColorB = FLinearColor::Blue;
};

static void SetWireframeRender(IDataflowDebugDrawInterface& DataflowRenderingInterface)
{
	DataflowRenderingInterface.SetShaded(false);
	DataflowRenderingInterface.SetWireframe(true);
}

static void SetShadedRender(IDataflowDebugDrawInterface& DataflowRenderingInterface)
{
	DataflowRenderingInterface.SetShaded(true);
	DataflowRenderingInterface.SetWireframe(true);
}


