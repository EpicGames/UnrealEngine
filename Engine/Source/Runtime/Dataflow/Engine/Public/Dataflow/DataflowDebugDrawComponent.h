// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "DataflowContent.h"
#include "Debug/DebugDrawComponent.h"
#include "DataflowDebugDrawComponent.generated.h"

struct IDataflowDebugDrawObject;

UCLASS(MinimalAPI)
class UDataflowDebugDrawComponent : public UDebugDrawComponent
{
	GENERATED_BODY()

private:
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
};

class FDataflowDebugRenderSceneProxy final : public FDebugRenderSceneProxy
{
public:

	FDataflowDebugRenderSceneProxy(const UPrimitiveComponent* InComponent);

	/** Remove all primitives stored on this proxy */
	void ClearAll();

	struct FDebugPoint
	{
		FVector Position;
		float	Size = 0;
		FLinearColor Color = FLinearColor::White;
		ESceneDepthPriorityGroup Priority = ESceneDepthPriorityGroup::SDPG_World;
	};
	
	/** Add point to the scene proxy */
	void AddPoint(const FDebugPoint& Point);

	/** Add dataflow object to the scene proxy */
	void AddObject(const TRefCountPtr<IDataflowDebugDrawObject>& Object);

	/** Reserve the number of points to add */
	void ReservePoints(int32 NumAdditionalPoints);

private:
		
	friend class UDataflowDebugDrawComponent;
	TArray<FDebugPoint> Points;

	/** List of user provided dataflow objects to draw */
	TArray<TRefCountPtr<IDataflowDebugDrawObject>> Objects;

private:
	virtual void GetDynamicMeshElementsForView(const FSceneView* View, const int32 ViewIndex, const FSceneViewFamily& ViewFamily, const uint32 VisibilityMap, FMeshElementCollector& Collector, FMaterialCache& DefaultMaterialCache, FMaterialCache& SolidMeshMaterialCache) const override;
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
};
