// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "HitProxies.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowElement.h"
#include "GenericPlatform/ICursor.h"

#include "DataflowDebugDrawObject.generated.h"

#define UE_API DATAFLOWENGINE_API

class FPrimitiveDrawInterface;

/** Dataflow object debug draw parent class */
struct FDataflowDebugDrawBaseObject : public IDataflowDebugDrawObject
{
	using Super = IDataflowDebugDrawObject;
	
	static FName StaticType() { return FName("FDataflowDebugDrawBaseObject"); }
	
	FDataflowDebugDrawBaseObject(IDataflowDebugDrawInterface::FDataflowElementsType& InDataflowElements) :
		IDataflowDebugDrawObject(), DataflowElements(InDataflowElements)
	{}
	
	/** Populate dataflow elements */
	virtual void PopulateDataflowElements() = 0;

	/** Debug draw dataflow element */
	virtual void DrawDataflowElements(FPrimitiveDrawInterface* PDI) = 0;

	/** Compute teh dataflow elements bounding box */
	virtual FBox ComputeBoundingBox() const = 0;

	/** Check of the object type */
	virtual bool IsA(FName InType) const override
	{
		return InType.ToString().Equals(StaticType().ToString()) 
			|| Super::IsA(InType); 
	}
protected :

	/** List of dataflow elements the debug draw object is going to populate and render */
	IDataflowDebugDrawInterface::FDataflowElementsType& DataflowElements;

	/** Elements offset in the global array */
	int32 ElementsOffset = 0;

	/** Elements size in the global array */
	int32 ElementsSize = 0;
};

template<typename ObjectType, typename... Args>
TRefCountPtr<ObjectType> MakeDebugDrawObject(Args&& ... args)
{
	TRefCountPtr<ObjectType> DataflowObject = MakeRefCount<ObjectType>(std::forward<Args>(args)...);
	DataflowObject->PopulateDataflowElements();

	return DataflowObject;
}

/** Dataflow hit proxy for viewport selection */
struct HDataflowElementHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( UE_API )

	HDataflowElementHitProxy(int32 InElementIndex, FName InElementName)
		: HHitProxy(HPP_Foreground)
		, ElementIndex(InElementIndex)
		, ElementName(InElementName)
	{}

	//~ Begin HHitProxy interface
	virtual EMouseCursor::Type GetMouseCursor() override { return EMouseCursor::Crosshairs; }
	//~ End HHitProxy interface

	/** Element index to retrieve the matching dataflow element */
	int32 ElementIndex;

	/** Element name to retrieve the matching dataflow element */
	FName ElementName;
};

/** Proxy dataflow scene element that contains a ref to HDataflowElementHitProxy */
USTRUCT()
struct FDataflowProxyElement : public FDataflowBaseElement
{
	GENERATED_BODY()

	FDataflowProxyElement() {}

	FDataflowProxyElement(const FString& InElementName, FDataflowBaseElement* InParentElement, const FBox& InBoundingBox, const bool bInIsConstruction) :
		FDataflowBaseElement(InElementName, InParentElement, InBoundingBox, bInIsConstruction ), ElementProxy(nullptr) {}

	/** Element proxy used for selection */
	TRefCountPtr<HHitProxy> ElementProxy = nullptr;

	static FName StaticType() { return FName("FDatafloProxyElement"); }

	/** Check of the element type */
	virtual bool IsA(FName InType) const override
	{
		return InType.ToString().Equals(StaticType().ToString())
			|| Super::IsA(InType);
	}
};

#undef UE_API
