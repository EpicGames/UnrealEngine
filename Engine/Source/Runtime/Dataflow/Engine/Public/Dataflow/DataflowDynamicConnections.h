// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowConnection.h"

#include "StructUtils/PropertyBag.h"

#include "DataflowDynamicConnections.generated.h"

struct FDataflowNode;

/*
* Dynamic connection object allow a node to add dynamic inputs or outputs backed by a strongly typed properties via a property bag 
*/
USTRUCT()
struct FDataflowDynamicConnections
{
	GENERATED_USTRUCT_BODY()

	struct IOwnerInterface
	{
		virtual FDataflowNode* GetOwner(const FDataflowDynamicConnections* Caller) = 0;
		virtual const FInstancedPropertyBag& GetPropertyBag(const FDataflowDynamicConnections* Caller) = 0;
	};

public:
	DATAFLOWENGINE_API FDataflowDynamicConnections();
	DATAFLOWENGINE_API FDataflowDynamicConnections(UE::Dataflow::FPin::EDirection PInDirection, IOwnerInterface* OwnerInterface, UDataflow* DataflowAsset);

	DATAFLOWENGINE_API void Refresh();

private:
	using FConnectionReference = UE::Dataflow::TConnectionReference<FDataflowAllTypes>;
	FConnectionReference GetConnectionReference(int32 Index) const;

	bool IsSupportedType(const FPropertyBagPropertyDesc& PropertyDesc);
	FDataflowConnection* CreateConnection(FConnectionReference Connectionreference);
	TArray<FDataflowConnection*> GetDynamicConnections() const;
	void ClearDynamicConnections();

	bool SetConnectionTypeFromPropertyDesc(FDataflowConnection& Connection, const FPropertyBagPropertyDesc& PropertyDesc);
	FName GetCppTypeFromPropertyDesc(const FPropertyBagPropertyDesc& PropertyDesc);
	FDataflowConnection* AddNewConnectionFromPropertyDesc(const FPropertyBagPropertyDesc& PropertyDesc);

private:
	UPROPERTY()
	TArray<FDataflowAllTypes> DynamicProperties;

	TMap<FName, FGuid> ConnectionNameToPropertyId;

	UE::Dataflow::FPin::EDirection PinDirection = UE::Dataflow::FPin::EDirection::NONE;
	IOwnerInterface* OwnerInterface;
	TWeakObjectPtr<UDataflow> DataflowAssetWeakPtr;

	// used to prevent re-entrance of the refresh function
	// this happens if subgraph  includes a node to itself and am output pin is connected to another node
	bool bIsRefreshing = false;
};