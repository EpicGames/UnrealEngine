// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Internationalization/Text.h"
#include "Misc/Build.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "DataflowEdNode.generated.h"

class FArchive;
class UEdGraphPin;
class UObject;
namespace UE::Dataflow { class FGraph; class FRenderingParameters; class IDataflowConstructionViewMode;  }
namespace GeometryCollection::Facades { class FRenderingFacade; }

UCLASS(MinimalAPI)
class UDataflowEdNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	FGuid DataflowNodeGuid;
	TSharedPtr<UE::Dataflow::FGraph> DataflowGraph;

public:
	virtual ~UDataflowEdNode() { UnRegisterDelegateHandle(); };

	// UEdGraphNode interface
#if WITH_EDITOR
	DATAFLOWENGINE_API virtual void AllocateDefaultPins() override;
	DATAFLOWENGINE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	DATAFLOWENGINE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
#endif // WITH_EDITOR && !UE_BUILD_SHIPPING
	DATAFLOWENGINE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	DATAFLOWENGINE_API virtual bool ShowPaletteIconOnNode() const override;
	DATAFLOWENGINE_API virtual FLinearColor GetNodeTitleColor() const override;
	DATAFLOWENGINE_API virtual FLinearColor GetNodeBodyTintColor() const override;
	DATAFLOWENGINE_API virtual FText GetTooltipText() const override;
	DATAFLOWENGINE_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	DATAFLOWENGINE_API virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	DATAFLOWENGINE_API virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	DATAFLOWENGINE_API virtual void OnPinRemoved(UEdGraphPin* InRemovedPin) override;
	DATAFLOWENGINE_API virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override;
	DATAFLOWENGINE_API virtual void PostEditUndo() override;
	DATAFLOWENGINE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
#endif // WITH_EDITOR
	// End of UEdGraphNode interface

	// UObject interface
	DATAFLOWENGINE_API void Serialize(FArchive& Ar) override;
	// End UObject interface

	bool IsBound() const { return DataflowGraph && DataflowNodeGuid.IsValid(); }

	TSharedPtr<UE::Dataflow::FGraph> GetDataflowGraph() { return DataflowGraph; }
	TSharedPtr<const UE::Dataflow::FGraph> GetDataflowGraph() const { return DataflowGraph; }
	void SetDataflowGraph(TSharedPtr<UE::Dataflow::FGraph> InDataflowGraph) { DataflowGraph = InDataflowGraph; }

	DATAFLOWENGINE_API void UpdatePinsFromDataflowNode();
	DATAFLOWENGINE_API void UpdatePinsConnectionsFromDataflowNode();
	DATAFLOWENGINE_API void UpdatePinsDefaultValuesFromNode();

	FGuid GetDataflowNodeGuid() const { return DataflowNodeGuid; }
	DATAFLOWENGINE_API void SetDataflowNodeGuid(FGuid InGuid);

	DATAFLOWENGINE_API TSharedPtr<FDataflowNode> GetDataflowNode();
	DATAFLOWENGINE_API TSharedPtr<const FDataflowNode> GetDataflowNode() const;

	/** Add a new option pin if the underlying Dataflow node AddPin member is overriden. */
	DATAFLOWENGINE_API void AddOptionPin();
	/** Remove an option pin if the underlying Dataflow node RemovePin member is overriden. */
	DATAFLOWENGINE_API void RemoveOptionPin();

	DATAFLOWENGINE_API bool PinIsCompatibleWithType(const UEdGraphPin& Pin, const FEdGraphPinType& PinType) const;

#if WITH_EDITOR
	// Pin hiding
	DATAFLOWENGINE_API void HideAllInputPins();
	DATAFLOWENGINE_API void ShowAllInputPins();
	DATAFLOWENGINE_API void ToggleHideInputPin(FName PinName);
	DATAFLOWENGINE_API bool CanToggleHideInputPin(FName PinName) const;
	DATAFLOWENGINE_API bool IsInputPinShown(FName PinName) const;
#endif

	//
	// Node Rendering
	//
	DATAFLOWENGINE_API void SetShouldRenderNode(bool bInRender);
	bool ShouldRenderNode() const { return bRenderInAssetEditor; }

	DATAFLOWENGINE_API void SetShouldWireframeRenderNode(bool bInRender);
	bool ShouldWireframeRenderNode() const { return bRenderWireframeInAssetEditor; }

	DATAFLOWENGINE_API void SetCanEnableWireframeRenderNode(bool bInCanEnable);
	DATAFLOWENGINE_API bool CanEnableWireframeRenderNode() const;

	DATAFLOWENGINE_API TArray<UE::Dataflow::FRenderingParameter> GetRenderParameters() const;

	DATAFLOWENGINE_API void RegisterDelegateHandle();
	DATAFLOWENGINE_API void UnRegisterDelegateHandle();

	DATAFLOWENGINE_API bool HasAnyWatchedConnection() const;
	DATAFLOWENGINE_API bool IsConnectionWatched(const FDataflowConnection& Connection) const;
	DATAFLOWENGINE_API void WatchConnection(const FDataflowConnection& Connection, bool Value);

	DATAFLOWENGINE_API bool IsPinWatched(const UEdGraphPin* Pin) const;
	DATAFLOWENGINE_API void WatchPin(const UEdGraphPin* Pin, bool bWatch);

	DATAFLOWENGINE_API static FDataflowConnection* GetConnectionFromPin(const UEdGraphPin* Pin);
	DATAFLOWENGINE_API static TSharedPtr<FDataflowNode> GetDataflowNodeFromEdNode(UEdGraphNode* EdNode);
	DATAFLOWENGINE_API static TSharedPtr<const FDataflowNode> GetDataflowNodeFromEdNode(const UEdGraphNode* EdNode);

#if WITH_EDITOR
	DATAFLOWENGINE_API static bool SupportsEditablePinType(const UEdGraphPin& Pin);

	DATAFLOWENGINE_API static TSharedPtr<UE::Dataflow::FGraph> GetDataflowGraphFromPin(const UEdGraphPin& Pin);
	DATAFLOWENGINE_API static TSharedPtr<const FDataflowNode> GetDataflowNodeFromPin(const UEdGraphPin& Pin);
	DATAFLOWENGINE_API static const FDataflowInput* GetDataflowInputFromPin(const UEdGraphPin& Pin);
	DATAFLOWENGINE_API static const FDataflowOutput* GetDataflowOutputFromPin(const UEdGraphPin& Pin);
	DATAFLOWENGINE_API static bool CanConnectPins(const UEdGraphPin& OutputPin, const UEdGraphPin& InputPin);
	DATAFLOWENGINE_API static bool CanConvertPins(const UEdGraphPin& OutputPin, const UEdGraphPin& InputPin);

	DATAFLOWENGINE_API static FSlateIcon GetIconFromNode(const FDataflowNode& Node);
#endif

private:

#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UEdGraphPin* CreateEdPin(const UE::Dataflow::FPin& Pin);
#endif

	void ScheduleNodeInvalidation();

	/** used to avoid update loops setting and editing the editable pins */
	bool bEditablePinReentranceGuard = false;

	UPROPERTY(Transient)
	bool bRenderInAssetEditor = false;

	UPROPERTY(Transient)
	bool bRenderWireframeInAssetEditor = false;

	UPROPERTY(Transient)
	bool bCanEnableRenderWireframe = true;

	// Store Guids from connection being watched (display values of corresponding output) 
	TArray<FGuid> WatchedConnections;

	void RemoveSpacesInAllPinTypes();

	FDelegateHandle OnNodeInvalidatedDelegateHandle;
};

