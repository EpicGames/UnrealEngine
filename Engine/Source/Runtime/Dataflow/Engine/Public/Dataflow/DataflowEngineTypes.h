// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineBaseTypes.h"

#include "DataflowEngineTypes.generated.h"

USTRUCT(BlueprintType)
struct FStringValuePair
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
		FString Key;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
		FString Value;

	bool operator ==(const FStringValuePair& Other) const {
		return Key.Equals(Other.Key) && Value.Equals(Other.Value);
	}
};
