// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Dataflow/DataflowEngineTypes.h"
#include "Math/Transform.h"
#include "ReferenceSkeleton.h"
#include "StructUtils/PropertyBag.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"


#define UE_API DATAFLOWENGINE_API

struct FDataflowConnection;
struct FInstancedPropertyBag;

namespace UE::Geometry
{
	class FDynamicMesh3;
}

namespace UE::Dataflow
{
	namespace Reflection
	{
		template<class T>
		const T* FindObjectPtrProperty(const UObject* Owner, FName Name)
		{
			if (Owner && Owner->GetClass())
			{
				if (const ::FProperty* UEProperty = Owner->GetClass()->FindPropertyByName(Name))
				{
					if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(UEProperty))
					{
						if (const void* ObjectContainer = ObjectProperty->ContainerPtrToValuePtr<void>(Owner))
						{
							if (const UObject* Value = ObjectProperty->GetObjectPropertyValue(ObjectContainer))
							{
								return Cast<T>(Value);
							}
						}
					}
				}
			}
			return nullptr;
		}

		template<class T>
		const T FindOverrideProperty(const UObject* Owner, FName PropertyName, FName ArrayKey, const T& Default = T())
		{
			if (Owner && Owner->GetClass())
			{
				if (FArrayProperty* ArrayProperty = FindFProperty<FArrayProperty>(Owner->GetClass(), PropertyName))
				{
					if (FStructProperty* ArrayInnerProperty = CastFieldChecked<FStructProperty>(ArrayProperty->Inner))
					{
						if (ArrayInnerProperty->Struct == FStringValuePair::StaticStruct())
						{
							if (const TArray<FStringValuePair>* ContainerData = ArrayProperty->ContainerPtrToValuePtr< TArray<FStringValuePair> >(Owner))
							{
								for (const FStringValuePair& Pair : *ContainerData)
								{
									if (Pair.Key.Equals(ArrayKey.ToString()))
									{
										return Pair.Value;
									}
								}
							}
						}
					}
				}
			}
			return Default;
		}
	}

	namespace Animation
	{
		UE_API void GlobalTransforms(const FReferenceSkeleton&, TArray<FTransform>& GlobalTransforms);
	}

	namespace Color
	{
		UE_API FLinearColor GetRandomColor(const int32 RandomSeed, int32 Idx);
	}

	namespace Type
	{
		UE_API FPropertyBagPropertyDesc GetPropertyBagPropertyDescFromDataflowType(const FName Name, const FName Type);
		UE_API FPropertyBagPropertyDesc GetPropertyBagPropertyDescFromDataflowConnection(const FDataflowConnection& Connection);
		UE_API FName MakeUniqueNameForPropertyBag(FName OriginalName, const FInstancedPropertyBag& PropertyBag);
	}

	namespace RenderGeometry
	{
		UE_API UStaticMesh* GetDataflowBox();
		UE_API UStaticMesh* GetDataflowSphere();
		UE_API UStaticMesh* GetDataflowPlane();
	}

	namespace RenderMaterial
	{
		UE_API UMaterialInterface* GetDataflowColorMaterial(const bool bTwoSided = false);
		UE_API UMaterialInterface* GetDataflowColorWireframeMaterial(const bool bTwoSided = false);
		UE_API UMaterialInterface* GetDataflowCheckerBoardMaterial(const bool bUseVertexColor = false);
		UE_API UMaterialInterface* GetDataflowVertexColorMaterial();
		UE_API UMaterialInterface* GetDataflowPointsMaterial();
		UE_API UMaterialInterface* GetDataflowLinesMaterial();
	}

	namespace Mesh
	{
		// Barycentric interpolation of a vertex attribute
		template <typename T>
		T InterpolateAttribute(const FVector& InVertexA,
			const FVector& InVertexB,
			const FVector& InVertexC,
			const FVector& InPointInTriangle,
			const T& InAttributeAtVertexA,
			const T& InAttributeAtVertexB,
			const T& InAttributeAtVertexC,
			const T& InDefaultValue)
		{
			double AreaTriangleABC = 0.5 * FVector::CrossProduct(InVertexB - InVertexA, InVertexC - InVertexA).Length();
			double AreaTriangleABP = 0.5 * FVector::CrossProduct(InVertexA - InPointInTriangle, InVertexB - InPointInTriangle).Length();
			double AreaTriangleACP = 0.5 * FVector::CrossProduct(InVertexA - InPointInTriangle, InVertexC - InPointInTriangle).Length();
			double AreaTriangleBCP = 0.5 * FVector::CrossProduct(InVertexB - InPointInTriangle, InVertexC - InPointInTriangle).Length();

			if (AreaTriangleABC > SMALL_NUMBER)
			{
				const double U = AreaTriangleBCP / AreaTriangleABC;
				const double V = AreaTriangleACP / AreaTriangleABC;
				const double W = AreaTriangleABP / AreaTriangleABC;

				if (FMath::Abs(U + V + W - 1.0) < UE_SMALL_NUMBER)
				{
					return U * InAttributeAtVertexA + V * InAttributeAtVertexB + W * InAttributeAtVertexC;
				}
			}

			return InDefaultValue;
		}

		UE_API bool GetMeshVertices(const UE::Geometry::FDynamicMesh3* InMeshPtr, TArray<FVector3f>& OutVertices);

		UE_API bool GetMeshVertexNormals(const UE::Geometry::FDynamicMesh3* InMeshPtr, TArray<FVector3f>& OutVertexNormals);

		UE_API bool GetMeshUVs(const UE::Geometry::FDynamicMesh3* InDynMeshPtr, TArray<FVector2f>& OutUVs, const int32 InUVLayer);
	}
}

#undef UE_API
