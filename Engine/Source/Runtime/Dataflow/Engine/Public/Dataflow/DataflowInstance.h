// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowContent.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/ObjectPtr.h"

#include "DataflowInstance.generated.h"

class FAssetRegistryTagsContext;
class UDataflowAttachment;

namespace UE::Dataflow::Private
{
	class FVariablesOverridesDetails;
}

namespace UE::Dataflow::InstanceUtils::Private
{
	// need to be in the header to be used by the templated structure override methods
	template<typename T, typename TWriteElementFunction>
	bool OverrideVariableArray(FInstancedPropertyBag& Variables, FName VariableName, const TArray<T>& Values, const TWriteElementFunction& WriteElementFunction, FGuid& OutGuid)
	{
		if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByName(VariableName))
		{
			TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> MutableArrayRef = Variables.GetMutableArrayRef(*Desc);
			if (FPropertyBagArrayRef* ArrayRef = MutableArrayRef.TryGetValue())
			{
				bool bSuccess = true;
				ArrayRef->EmptyValues();
				ArrayRef->AddValues(Values.Num());
				for (int32 Idx = 0; Idx < Values.Num(); ++Idx)
				{
					bSuccess &= (WriteElementFunction(*ArrayRef, Idx, Values[Idx]) == EPropertyBagResult::Success);
				}
				if (bSuccess)
				{
					OutGuid = Desc->ID;
				}
				return bSuccess;
			}
		}
		OutGuid.Invalidate();
		return false;
	}
}

class UDataflow;
struct FDataflowInstance;

/**
* This wraps the variable overrides
* This is also separate from FDataflowInstance to allow for customization to display override checkboxes in front of each  variable property
* ( see DataflowDetails.h )
*/
USTRUCT()
struct FDataflowVariableOverrides
{
	GENERATED_USTRUCT_BODY();

	DATAFLOWENGINE_API FDataflowVariableOverrides(FDataflowInstance* InOwner = nullptr);

	// prevent the copy constructor to avoid having a copy with the same owner
	// use the assignment operator instead on a newly created FDataflowVariableOverrides with the right owner to copy its content over 
	FDataflowVariableOverrides(const FDataflowVariableOverrides& Other) = delete;

	DATAFLOWENGINE_API FDataflowVariableOverrides& operator=(const FDataflowVariableOverrides& Other);

	/** Remove all variables */
	DATAFLOWENGINE_API void RemoveAllVariables();

	/** Sync variables with from the original dataflow asset */
	DATAFLOWENGINE_API void SyncVariables();

	/** check if a specific variable exists */
	DATAFLOWENGINE_API bool HasVariable(FName VariableName) const;

	DATAFLOWENGINE_API bool IsVariableOverridden(FName VariableName) const;

	DATAFLOWENGINE_API const FInstancedPropertyBag& GetVariables() const;

	DATAFLOWENGINE_API FInstancedPropertyBag& GetVariables();

	DATAFLOWENGINE_API static FName GetVariablePropertyName();

	/** override a dataflow boolean variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableBool(FName VariableName, bool bValue);

	/** override a dataflow boolean array variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableBoolArray(FName VariableName, const TArray<bool>& Values);

	/** override a dataflow integer variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableInt(FName VariableName, int64 Value);

	/** override a dataflow integer array variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableInt32Array(FName VariableName, const TArray<int32>& Values);

	/** override a dataflow integer array variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableInt64Array(FName VariableName, const TArray<int64>& Values);

	/** override a dataflow float variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableFloat(FName VariableName, float Value);

	/** override a dataflow float array variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableFloatArray(FName VariableName, const TArray<float>& Values);

	/** override a dataflow UObject variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableObject(FName VariableName, const UObject* Value);

	/** override a dataflow UObject array variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableObjectArray(FName VariableName, const TArray<TObjectPtr<UObject>>& Values);

	/** override a dataflow UObject array variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableObjectArray(FName VariableName, const TArray<UObject*>& Values);

	/** override a dataflow FName variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableName(FName VariableName, FName Value);

	/** override a dataflow FName variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableName(FName VariableName, const TArray<FName>& Values);

	/** override a dataflow String variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableString(FName VariableName, FString Value);

	/** override a dataflow String variable for this asset */
	DATAFLOWENGINE_API bool OverrideVariableString(FName VariableName, const TArray<FString>& Values);

	template <typename T>
	bool OverrideVariableStruct(FName VariableName, const T& Value)
	{
		if (const FPropertyBagPropertyDesc* Desc = Variables.FindPropertyDescByName(VariableName))
		{
			if (Variables.SetValueStruct(VariableName, Value) == EPropertyBagResult::Success)
			{
				OverriddenVariableGuids.AddUnique(Desc->ID);
				return true;
			}
		}
		return false;
	}

	template <typename T>
	bool OverrideVariableStructArray(FName VariableName, const TArray<T>& Values)
	{
		using namespace UE::Dataflow::InstanceUtils::Private;

		auto WriteStructValue = [](FPropertyBagArrayRef& ArrayRef, int32 Idx, const T& Value)
			{
				return ArrayRef.SetValueStruct(Idx, Value);
			};

		return OverrideVariableArrayAndNotify(Variables, VariableName, Values, WriteStructValue);
	}

#if WITH_EDITOR

	void OnOwnerPostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
	void OnDataflowVariablesChanged(const UDataflow* DataflowAsset, FName VariableName);
#endif

private:
	friend class UE::Dataflow::Private::FVariablesOverridesDetails;

	DATAFLOWENGINE_API const FInstancedPropertyBag* GetDefaultVariablesFromAsset() const;

	DATAFLOWENGINE_API FInstancedPropertyBag& GetOverridenVariables();

	/** @return true if the property of specified ID is overridden. */
	DATAFLOWENGINE_API bool IsVariableOverridden(const FGuid PropertyID) const;

	/** Sets the override status of specified property by ID. */
	DATAFLOWENGINE_API void SetVariableOverridden(const FGuid PropertyID, const bool bIsOverridden);

	void RemoveOverridenVariablesNotInDataflowAsset();

private:
	DATAFLOWENGINE_API void SetVariableOverrideAndNotify(const FGuid& PropertyID, bool bOverrideState);

	template<typename T, typename TWriteElementFunction>
	bool OverrideVariableArrayAndNotify(FInstancedPropertyBag& InVariables, FName InVariableName, const TArray<T>& InValues, const TWriteElementFunction& InWriteElementFunction)
	{
		FGuid PropertyGuid;
		const bool bSuccess = UE::Dataflow::InstanceUtils::Private::OverrideVariableArray(InVariables, InVariableName, InValues, InWriteElementFunction, PropertyGuid);
		if (bSuccess)
		{
			SetVariableOverrideAndNotify(PropertyGuid, true);
		}
		return bSuccess;
	}

private:

	/** Variables to override */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (FixedLayout, ShowOnlyInnerProperties))
	FInstancedPropertyBag Variables;

	/** Array of overridden variable GUIDs. Non-overridden properties will inherit the values from the DataflowAsset default parameters. */
	UPROPERTY()
	TArray<FGuid> OverriddenVariableGuids;

	FDataflowInstance* Owner = nullptr;
};

/**
* This structure is to be embedded in any asset that need generation from a dataflow
*/
USTRUCT()
struct FDataflowInstance
{
	GENERATED_USTRUCT_BODY();

	DATAFLOWENGINE_API explicit FDataflowInstance(UObject* InOwner = nullptr, UDataflow* InDataflowAsset = nullptr, FName InTerminalNodeName = NAME_None);
	DATAFLOWENGINE_API FDataflowInstance& operator=(const FDataflowInstance& Other);
	
	// Delete these to avoid complications with Bind/UnbindEditorDelegates.
	//
	// They can be implemented if needed.
	FDataflowInstance(const FDataflowInstance& Other) = delete;
	FDataflowInstance(FDataflowInstance&& Other) = delete;
	
	DATAFLOWENGINE_API ~FDataflowInstance();

public:
	DATAFLOWENGINE_API void SetDataflowAsset(UDataflow* DataflowAsset);

	DATAFLOWENGINE_API UDataflow* GetDataflowAsset() const;

	DATAFLOWENGINE_API void SetDataflowTerminal(FName TerminalNodeName);

	DATAFLOWENGINE_API FName GetDataflowTerminal() const;

	DATAFLOWENGINE_API const FInstancedPropertyBag& GetVariables() const;

	DATAFLOWENGINE_API FInstancedPropertyBag& GetVariables();

	/** Get the dataflow terminal member property name */
	DATAFLOWENGINE_API static FName GetDataflowTerminalPropertyName();

	/** Get the dataflow asset member property name */
	DATAFLOWENGINE_API static FName GetDataflowAssetPropertyName();

	/** Get the variable overrides member property name */
	DATAFLOWENGINE_API static FName GetVariableOverridesPropertyName();

	DATAFLOWENGINE_API const FDataflowVariableOverrides& GetVariableOverrides() const;

	DATAFLOWENGINE_API FDataflowVariableOverrides& GetVariableOverrides();

	DATAFLOWENGINE_API void SyncVariables();

#if WITH_EDITORONLY_DATA
	/** Get the preview blueprint variable values for this asset. */
	FInstancedPropertyBag& GetPreviewBlueprintVariableOverrides()
	{
		return PreviewBlueprintVariableOverrides;
	}
	/** Get the preview blueprint variable values for this asset. Const version. */
	const FInstancedPropertyBag& GetPreviewBlueprintVariableOverrides() const
	{
		return PreviewBlueprintVariableOverrides;
	}
#endif

	/** 
	* Update the asset by re-evaluating the dataflow
	* return true if the dataflow has been properly evaluated and the asset updated
	*/
	DATAFLOWENGINE_API bool UpdateOwnerAsset(bool bUpdateDependentAssets = false) const;

	/** Call as part of the PostLoad of any UObject containing a FDataflowInstance member */
	DATAFLOWENGINE_API void PostLoad();

#if WITH_EDITOR
	DATAFLOWENGINE_API TSharedPtr<FStructOnScope> MakeStructOnScope() const;
#endif 

private:
	// IMPORTANT: Any properties added here must be handled in the assignment operator

	/** Dataflow asset to use */
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	TObjectPtr<UDataflow> DataflowAsset;

	/** name of the terminal node to use when generating the asset */
	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FName DataflowTerminal;

	/** Dataflow variables to override */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta=(ShowOnlyInnerProperties))
	FDataflowVariableOverrides VariableOverrides;

#if WITH_EDITORONLY_DATA
	/** Preview blueprint variables to override */
	UPROPERTY()
	FInstancedPropertyBag PreviewBlueprintVariableOverrides;
#endif

private:
	friend class UE::Dataflow::Private::FVariablesOverridesDetails;

#if WITH_EDITOR
	void BindEditorDelegates();
	void UnbindEditorDelegates();
	void OnOwnerPostEditChangeProperty(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent);

	FDelegateHandle OnOwnerPropertyChangedHandle;
#endif

	// Owning Object
	UPROPERTY()
	TObjectPtr<UObject> Owner = nullptr;
};

UINTERFACE(MinimalAPI)
class UDataflowInstanceInterface : public UInterface
{
	GENERATED_BODY()
};

/**
* Interface to use on asset class that are generated using a data flow
*/
class IDataflowInstanceInterface
{
	GENERATED_BODY()

public:
	virtual const FDataflowInstance& GetDataflowInstance() const = 0;
	virtual FDataflowInstance& GetDataflowInstance() = 0;
};

namespace UE::Dataflow::InstanceUtils
{
	/** Weather or not a dataflow asset is assigned to an UObject */
	DATAFLOWENGINE_API bool HasValidDataflowAsset(UObject* Obj);

	/** Get the Dataflow asset from a UObject if available */
	DATAFLOWENGINE_API UDataflow* GetDataflowAssetFromObject(UObject* Obj);

	/** Get the Dataflow asset from a UObject if available (const version) */
	DATAFLOWENGINE_API const UDataflow* GetDataflowAssetFromObject(const UObject* Obj);

	/** Get the Dataflow instance interface from a UObject */
	DATAFLOWENGINE_API IDataflowInstanceInterface* GetDataflowInterfaceFromObject(UObject* Obj);

	/** Get the Dataflow attachment from a UObject */
	DATAFLOWENGINE_API UDataflowAttachment* GetDataflowAttachmentFromObject(const UObject* Obj);

	/** Get the Dataflow instance interface from a const UObject */
	DATAFLOWENGINE_API const IDataflowInstanceInterface* GetDataflowInterfaceFromObject(const UObject* Obj);

	/** 
	* Set a Dataflow asset/object on a specific object
	* The object type must be compatible with dataflow
	* Returns true if the dataflow object was set successfuly
	*/
	DATAFLOWENGINE_API bool SetDataflowAssetOnObject(UDataflow* DataflowAsset, UObject* Obj);

	/** Get the terminal node name from a UObject if available */
	DATAFLOWENGINE_API FName GetTerminalNodeNameFromObject(UObject* Obj);

	/** get the list of terminal node names for a specific data flow asset */
	DATAFLOWENGINE_API TArray<FName> GetTerminalNodeNames(const UDataflow* DataflowAsset);

	// Register tags for an asset using Dataflow
	DATAFLOWENGINE_API void GetAssetRegistryTags(const UObject* BoundAsset, FAssetRegistryTagsContext& Context);


	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
}
