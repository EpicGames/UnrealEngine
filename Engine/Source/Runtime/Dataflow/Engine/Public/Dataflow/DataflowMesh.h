// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DataflowMesh.generated.h"

#define UE_API DATAFLOWENGINE_API

class UMaterialInterface;

/** 
* Containter for storing mesh and material information in Dataflow graph
*/
UCLASS(MinimalAPI)
class UDataflowMesh : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	UE_API const UE::Geometry::FDynamicMesh3* GetDynamicMesh() const;
	UE_API const UE::Geometry::FDynamicMesh3& GetDynamicMeshRef() const;

	UE_API const TArray<TObjectPtr<UMaterialInterface>>& GetMaterials() const;

	template<typename MeshType>
	void SetDynamicMesh(MeshType&& InMesh)
	{
		if (!DynamicMesh)
		{
			DynamicMesh = MakeUnique<UE::Geometry::FDynamicMesh3>(Forward<MeshType>(InMesh));
		}
		else
		{
			*DynamicMesh = Forward<MeshType>(InMesh);
		}
	}

	template<typename MaterialArrayType>
	void SetMaterials(MaterialArrayType&& InMaterials)
	{
		Materials = Forward<MaterialArrayType>(InMaterials);
	}

	template<typename MaterialArrayType>
	void AddMaterials(MaterialArrayType&& InMaterials)
	{
		Materials.Append(Forward<MaterialArrayType>(InMaterials));
	}

protected:

	TUniquePtr<UE::Geometry::FDynamicMesh3> DynamicMesh;

	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	UE_API virtual void Serialize(FArchive& Archive) override;
};

#undef UE_API
