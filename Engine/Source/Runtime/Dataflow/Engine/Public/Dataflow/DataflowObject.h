// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Animation/MeshDeformerInstance.h"
#include "Animation/MeshDeformerProducer.h"
#include "Dataflow/DataflowPreview.h"
#include "Dataflow/DataflowCore.h"
#include "EdGraph/EdGraph.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "StructUtils/PropertyBag.h"
#include "Delegates/DelegateCombinations.h"

#include "DataflowObject.generated.h"

class FArchive;
struct FDataflowNode;
class UDataflow;
class UObject;
class UDataflowEdNode;
class UDataflowSubGraph;
class UMaterial;
namespace UE::Dataflow
{ 
	class FGraph;
	struct FCompiledGraph;

	enum class ESubGraphChangedReason: uint8
	{
		Created,
		Pasted,			// Like Created, but emitted from the paste-from-clipboard path. Listeners should refresh views but should NOT auto-open the subgraph in a tab (the user pasted into a different graph).
		Renamed,
		Deleting,
		Deleted,
		ChangedType,
	};
}

struct FDataflowAssetDelegates
{
	/** Called when variables are edited ( add, remove, changetype , setvalue ) */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnVariablesChanged, const UDataflow*, FName);
	static DATAFLOWENGINE_API FOnVariablesChanged OnVariablesChanged;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnVariablesOverrideStateChanged, const UDataflow*, FName, bool);
	static DATAFLOWENGINE_API FOnVariablesOverrideStateChanged OnVariablesOverrideStateChanged;

	/** Called when sub-graphs are edited ( add, remove ) */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSubGraphsChanged, const UDataflow*, const FGuid&, UE::Dataflow::ESubGraphChangedReason);
	static DATAFLOWENGINE_API FOnSubGraphsChanged OnSubGraphsChanged;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNodeInvalidated, UDataflow&, FDataflowNode&);
	static DATAFLOWENGINE_API FOnNodeInvalidated OnNodeInvalidated;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDataflowPostEditUndo, UDataflow&);
	static DATAFLOWENGINE_API FOnDataflowPostEditUndo OnDataflowPostEditUndo;
#endif
};

/**
*	FDataflowAssetEdit
*     Structured RestCollection access where the scope
*     of the object controls serialization back into the
*     dynamic collection
*
*/
class FDataflowAssetEdit
{
public:
	typedef TFunctionRef<void()> FPostEditFunctionCallback;
	friend UDataflow;

	/**
	 * @param UDataflow				The "FAsset" to edit
	 */
	DATAFLOWENGINE_API FDataflowAssetEdit(UDataflow *InAsset, FPostEditFunctionCallback InCallable);
	DATAFLOWENGINE_API ~FDataflowAssetEdit();

	DATAFLOWENGINE_API UE::Dataflow::FGraph* GetGraph();

private:
	FPostEditFunctionCallback PostEditCallback;
	UDataflow* Asset;
};

/** Data flow types */
UENUM()
enum class EDataflowType : uint8
{
	/** the dataflow will be used to build assets */
	Construction,

	/** The dataflow will be used to define the simulation evolution */
	Simulation
};

/**
* UDataflow (UObject)
*
* UObject wrapper for the UE::Dataflow::FGraph
*
*/
UCLASS(BlueprintType, customconstructor, MinimalAPI)
class UDataflow : public UEdGraph, public IMeshDeformerProducer, public IDataflowGraphInterface
{
	GENERATED_UCLASS_BODY()

	UE::Dataflow::FTimestamp LastModifiedRenderTarget = UE::Dataflow::FTimestamp::Invalid; 
	TArray< TObjectPtr<const UDataflowEdNode> > RenderTargets; // Not Serialized
	TArray< TObjectPtr<const UDataflowEdNode> > WireframeRenderTargets; // Not Serialized
	TSharedPtr<UE::Dataflow::FGraph, ESPMode::ThreadSafe> Dataflow;
	DATAFLOWENGINE_API void PostEditCallback();

public:
	DATAFLOWENGINE_API UDataflow(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** 
	* Find all the node of a speific type and evaluate them using a specific UObject
	*/
	UE_DEPRECATED(5.1, "Use Blueprint library version of the function")
	DATAFLOWENGINE_API void EvaluateTerminalNodeByName(FName NodeName, UObject* Asset);

	virtual bool IsEditorOnly() const { return true; }

	/** Simulation tag to use in the node registry */
	static DATAFLOWENGINE_API const FString SimulationTag;

	//~ Begin UObject interface
	virtual void BeginDestroy() override;
	//~ End UObject interface

	//~ Begin IMeshDeformerProducer interface
	virtual FMeshDeformerBeginDestroyEvent& OnBeginDestroy() override {return BeginDestroyEvent;};
	//~ End IMeshDeformerProducer interface

	//~ Begin IDataflowGraphInterface interface
	virtual TSharedPtr<UE::Dataflow::FGraph> GetDataflowGraph() const override { return Dataflow; }
	//~ End IDataflowGraphInterface interface

public:
	UE_DEPRECATED(5.8, "bActive property is deprecated as it is no longer used")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bActive property is deprecated as it is no longer used"))
	bool bActive = true;

	UE_DEPRECATED(5.8, "Targets property is deprecated as it is no longer used")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Targets property is deprecated as it is no longer used"))
	TArray<TObjectPtr<UObject>> Targets;

	UPROPERTY(EditAnywhere, Category = "Render")
	TObjectPtr<UMaterial> Material = nullptr;

	UPROPERTY(EditAnywhere, Category = "Evaluation",meta=(EditConditionHides))
	EDataflowType Type = EDataflowType::Construction;

	/**
	* Dataflow variables
	* Variables are used by the graph to parametrized the graph
	* They can can default values and can be overriden on a per asset basis ( see Dataflow Instance )
	*/
	UPROPERTY()
	FInstancedPropertyBag Variables;

	/** 
	* Reference asset to use for this Dataflow 
	* The reference asset is used when opening the dataflow asset editor 
	* If the dataflow is shared among multiple assets, and the reference asset is set, only this asset will be able to change the dataflow
	* (for the other the dataflow graph will be read-only and only evaluation and overrides will be available)
	*/
	UPROPERTY(EditAnywhere, Category = Reference)
	TObjectPtr<UObject> ReferenceAsset = nullptr;

public:
	/** UObject Interface */
	static DATAFLOWENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

#if WITH_EDITOR
	DATAFLOWENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	DATAFLOWENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
#endif
	DATAFLOWENGINE_API virtual void PostLoad() override;
	/** End UObject Interface */

	DATAFLOWENGINE_API void Serialize(FArchive& Ar);

	/** Accessors for internal geometry collection */
	TSharedPtr<const UE::Dataflow::FGraph, ESPMode::ThreadSafe> GetDataflow() const { return Dataflow; }
	TSharedPtr<UE::Dataflow::FGraph, ESPMode::ThreadSafe> GetDataflow() { return Dataflow; }

	/**Editing the collection should only be through the edit object.*/
	FDataflowAssetEdit EditDataflow() const {
		//ThisNC is a by-product of editing through the component.
		UDataflow* ThisNC = const_cast<UDataflow*>(this);
		return FDataflowAssetEdit(ThisNC, [ThisNC]() {ThisNC->PostEditCallback(); });
	}

	DATAFLOWENGINE_API TObjectPtr<const UDataflowEdNode> FindEdNodeByDataflowNodeGuid(const FGuid& Guid) const;
	DATAFLOWENGINE_API TObjectPtr<UDataflowEdNode> FindEdNodeByDataflowNodeGuid(const FGuid& Guid);

	//
	// Render Targets
	//
	DATAFLOWENGINE_API void AddRenderTarget(TObjectPtr<const UDataflowEdNode>);
	DATAFLOWENGINE_API void RemoveRenderTarget(TObjectPtr<const UDataflowEdNode>);
	const TArray< TObjectPtr<const UDataflowEdNode> >& GetRenderTargets() const { return RenderTargets; }

	DATAFLOWENGINE_API void AddWireframeRenderTarget(TObjectPtr<const UDataflowEdNode>);
	DATAFLOWENGINE_API void RemoveWireframeRenderTarget(TObjectPtr<const UDataflowEdNode>);
	const TArray< TObjectPtr<const UDataflowEdNode> >& GetWireframeRenderTargets() const { return WireframeRenderTargets; }

	const UE::Dataflow::FTimestamp& GetRenderingTimestamp() const { return LastModifiedRenderTarget; }

	/**
	* Find the Dataflow asset from a sepcific graph/subgraph
	*/
	DATAFLOWENGINE_API static UDataflow* GetDataflowAssetFromEdGraph(UEdGraph* EdGraph);
	DATAFLOWENGINE_API static const UDataflow* GetDataflowAssetFromEdGraph(const UEdGraph* EdGraph);

	/** Find a SubGraph by name */
	DATAFLOWENGINE_API const UDataflowSubGraph* FindSubGraphByName(FName Name) const;
	DATAFLOWENGINE_API UDataflowSubGraph* FindSubGraphByName(FName Name);

	/** Find a SubGraph by its guid */
	DATAFLOWENGINE_API const UDataflowSubGraph* FindSubGraphByGuid(const FGuid& SubGraphGuid) const;
	DATAFLOWENGINE_API UDataflowSubGraph* FindSubGraphByGuid(const FGuid& SubGraphGuid);

	/** Add a SubGraph to the asset */
	DATAFLOWENGINE_API void AddSubGraph(UDataflowSubGraph* SubGraph);
	DATAFLOWENGINE_API void RemoveSubGraph(UDataflowSubGraph* SubGraph);

	DATAFLOWENGINE_API const TArray<TObjectPtr<UDataflowSubGraph>>& GetSubGraphs() const;

	/** 
	* Make sure EdNode gets refreshed from its dataflow node 
	* This is useful when connections are changed
	*/
	DATAFLOWENGINE_API void RefreshEdNode(TObjectPtr<UDataflowEdNode> EdNode);
	DATAFLOWENGINE_API void RefreshEdNodeByGuid(const FGuid NodeGuid);

	/** 
	* Compile the dataflow graph if it has not been compiled since last change to the graph 
	* returns a shared pointer to the compiled graph
	*/
	DATAFLOWENGINE_API 	TSharedPtr<const UE::Dataflow::FCompiledGraph> CompileGraphIfNeeded();

#if WITH_EDITORONLY_DATA

	/*
	* The following PreviewScene properties are modeled after PreviewSkeletalMesh in USkeleton
	*	- they are inside WITH_EDITORONLY_DATA because they are not used at game runtime
	*	- TSoftObjectPtrs since that will make it possible to avoid loading these assets until the PreviewScene asks for them
	*	- DuplicateTransient so that if you copy a ClothAsset it won't copy these preview properties
	*	- AssetRegistrySearchable makes it so that if the user searches the name of a PreviewScene asset in the Asset Browser
	*/

	/** Cachie params used in this asset */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	FDataflowPreviewCacheParams PreviewCacheParams;

	/** Cache asset used in this asset */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	TSoftObjectPtr<UObject> PreviewCacheAsset = nullptr;

	/** Caching blueprint actor class to spawn */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	TSubclassOf<AActor> PreviewBlueprintClass;
	
	/** Caching blueprint actor transform to spawn */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	FTransform PreviewBlueprintTransform = FTransform::Identity;

	/** Geometry cache asset used to extract skeletal mesh results from simulation */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	TSoftObjectPtr<UObject> PreviewGeometryCacheAsset = nullptr;

	/** SkeletalMesh interpolated from simulation */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	TSoftObjectPtr<UObject> PreviewEmbeddedSkeletalMesh = nullptr;

	/** Static Mesh interpolated from simulation */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable)
	TSoftObjectPtr<UObject> PreviewEmbeddedStaticMesh = nullptr;
#endif

#if WITH_EDITOR
	/** Used to disable per-node serialization when serializing a transaction */
	bool IsPerNodeTransactionSerializationEnabled() const { return bEnablePerNodeTransactionSerialization; }
private:

	/** Used to disable per-node serialization when serializing a transaction */
	bool bEnablePerNodeTransactionSerialization = true;
#endif

private:
	/** 
	* List of Dataflow SubGraphs 
	* In editor they also exists in the SubGraphs parent class property
	*/
	UPROPERTY()
	TArray<TObjectPtr<UDataflowSubGraph>> DataflowSubGraphs; 

	/** Stores the topology GUID used fro the most recent compilation (see CompiledGraphs) */
	FGuid LastCompiledTopologyGUID;

	/** Contained the most recent compiled dataflow graph (see GetGraphTopologyGuid )*/
	TMap<const UObject*, TSharedRef<UE::Dataflow::FCompiledGraph>> CompiledGraphs;


private:
	/** Broadcasts a notification just before the dataflow is destroyed. */
	FMeshDeformerBeginDestroyEvent BeginDestroyEvent;
};


