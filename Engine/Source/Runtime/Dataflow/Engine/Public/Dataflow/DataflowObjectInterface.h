// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNodeParameters.h"
#include "GeometryCollection/ManagedArrayCollection.h"

class UObject;
class FArchive;

namespace UE::Dataflow
{
	namespace Private
	{
		DATAFLOWENGINE_API void UpdateAssetFromTerminalNodes(FContext& Context, UObject* Asset, bool bUseAllTerminals);
		DATAFLOWENGINE_API bool EvaluateGraphInternal(TSharedRef<IDataflowEvaluator> Evaluator, UObject* Owner, FOnPostEvaluationFunction OnPostEvaluation);
		DATAFLOWENGINE_API bool EvaluateGraphInternal(TSharedRef<IDataflowEvaluator> Evaluator, UDataflow* DataflowObject, UObject* ObjectToUpdate, FOnPostEvaluationFunction OnPostEvaluation);
		DATAFLOWENGINE_API UObject* FindBoundAsset(UObject* Object);
	}

	template<class Base = FContextSingle>
	class TEngineContext : public Base
	{
	public:
		DATAFLOW_CONTEXT_INTERNAL(Base, TEngineContext);

		explicit TEngineContext(const TObjectPtr<UObject>& InOwner)
			: Base()
			, Owner(InOwner)
		{}

		TObjectPtr<UObject> Owner = nullptr;

		/** 
		* return the asset bound to this context 
		* This may be the owner but in the case of the use of Dataflow attachement this may be the Outer of the Owner
		*/
		UObject* GetBoundAsset() const
		{
			return Private::FindBoundAsset(Owner);
		}

		virtual ~TEngineContext() = default;

		int32 GetKeys(TSet<FContextCacheKey>& InKeys) const { return Base::GetKeys(InKeys); }

		const TUniquePtr<FContextCacheElementBase>* GetBaseData(FContextCacheKey Key) const { return Base::GetDataImpl(Key); }

		virtual void Serialize(FArchive& Ar) { Base::Serialize(Ar); }

		/**
		* Evaluate the dataflow graph attached to the owner of this context and update the owner accordingly
		* Returns an evaluator that can be used to query the progress of the evaluation 
		* Note that the evaluator is not kept by this context and needs to be kept by the caller
		*/
		template <typename T UE_REQUIRES(std::is_base_of_v<IDataflowEvaluator, T>)>
		TSharedPtr<IDataflowEvaluator> EvaluateGraph(FOnPostEvaluationFunction OnPostEvaluation)
		{
			TSharedRef<IDataflowEvaluator> Evaluator = MakeShared<T>(*this);
			if (Private::EvaluateGraphInternal(Evaluator, Owner, OnPostEvaluation))
			{
				return Evaluator.ToSharedPtr();
			}
			return {};
			
		}

		/**
		* Evaluate an external dataflow graph and update the owner of this context with it 
		* Returns an evaluator that can be used to query the progress of the evaluation 
		* Note that the evaluator is not kept by this context and needs to be kept by the caller
		*/
		template <typename T UE_REQUIRES(std::is_base_of_v<IDataflowEvaluator, T>)>
		TSharedPtr<IDataflowEvaluator> EvaluateGraph(UDataflow* DataflowObject, FOnPostEvaluationFunction OnPostEvaluation)
		{
			TSharedRef<IDataflowEvaluator> Evaluator = MakeShared<T>(*this);
			if (Private::EvaluateGraphInternal(Evaluator, DataflowObject, Owner, OnPostEvaluation))
			{
				return Evaluator.ToSharedPtr();
			}
			return {};
		}
	};

	typedef TEngineContext<FContextSingle> FEngineContext;
	typedef TEngineContext<FContextThreaded> FEngineContextThreaded;
}

