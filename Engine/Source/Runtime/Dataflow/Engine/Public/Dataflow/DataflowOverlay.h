// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/UnrealString.h"

struct FDataflowNode;
struct FManagedArrayCollection;
struct FDataflowSelection;

namespace UE::Dataflow::Overlay
{
	DATAFLOWENGINE_API FString BuildOverlayNodeInfoString(const FDataflowNode* InNode);
	DATAFLOWENGINE_API FString BuildOverlaySelectionEvaluateResultString(FDataflowSelection& InSelection);
	DATAFLOWENGINE_API FString BuildOverlayCollectionInfoString(const FManagedArrayCollection& InCollection);
	DATAFLOWENGINE_API FString BuildOverlayBoundsInfoString(const FManagedArrayCollection& InCollection);
	DATAFLOWENGINE_API FString BuildOverlayMemInfoString(const FDataflowNode* InNode);
	DATAFLOWENGINE_API FString BuildOverlayFinalString(const TArray<FString>& InStringArr);
}

