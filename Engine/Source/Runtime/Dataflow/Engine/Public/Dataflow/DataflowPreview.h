// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "DataflowPreview.generated.h"

class UAnimationAsset;
class USkeletalMesh;

USTRUCT()
struct FDataflowPreviewCacheParams
{
	GENERATED_BODY()
public :

	/** Number of sampling frames per second for caching*/
	UPROPERTY(EditAnywhere, Category="Caching")
	int32 FrameRate = 30;

	/** Number of subframes per frame for timeline clamping*/
	UPROPERTY(EditAnywhere, meta = (EditCondition = "bCanEditSubframeRate", ClampMin = "1"), Category = "Caching")
	int32 SubframeRate = 1;
	UPROPERTY(EditAnywhere, meta = (InlineEditConditionToggle), Category = "Caching")
	bool bCanEditSubframeRate = false;

	/** Time range of the simulation*/
	UPROPERTY(EditAnywhere, Category="Caching")
	FVector2f TimeRange = FVector2f(0.0f, 5.0f);

	/** If enabled, the simulation will restart on Time Range without modifying anything outside of Time Range */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Caching")
	bool bRestartSimulation = false;

	/** Time range of the restarted simulation*/
	UPROPERTY(EditAnywhere, Category = "Caching", meta = (EditCondition = "bRestartSimulation"))
	FVector2f RestartTimeRange = FVector2f(0.0f, 0.0f);

	/** Control whether the async caching feature is available or not */
	UPROPERTY()
	bool bEnableAsyncCaching = false;

	/** 
	* When enabled runs the caching proces on an async thread (allowing the UI to remain responsive)
	* This option will be disabled if the asset associated with the graph does not support asynchronous caching 
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Caching", meta = (EditCondition = "bEnableAsyncCaching", HideEditConditionToggle))
	bool bAsyncCaching = false;

	
};
