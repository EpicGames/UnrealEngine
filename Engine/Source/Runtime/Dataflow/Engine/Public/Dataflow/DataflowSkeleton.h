// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ReferenceSkeleton.h"

#include "DataflowSkeleton.generated.h"

namespace UE::Geometry
{
	class FDynamicMesh3;
}

/** 
* Represents skeletons for dataflow
*/
USTRUCT()
struct FDataflowSkeleton
{
	GENERATED_USTRUCT_BODY()

	// read-only access to the underlying skeleton object
	DATAFLOWENGINE_API const FReferenceSkeleton& GetRefSkeleton() const;

	DATAFLOWENGINE_API FReferenceSkeletonModifier ModifySkeleton();

	DATAFLOWENGINE_API void Empty();
	
	DATAFLOWENGINE_API void Copy(const FReferenceSkeleton& ToCopy);

	DATAFLOWENGINE_API bool Serialize(FArchive& Ar);

private:
	FReferenceSkeleton RefSkeleton;
};

template<> struct TStructOpsTypeTraits<FDataflowSkeleton> : public TStructOpsTypeTraitsBase2<FDataflowSkeleton>
{
	enum
	{
		WithSerializer = true,
	};
};