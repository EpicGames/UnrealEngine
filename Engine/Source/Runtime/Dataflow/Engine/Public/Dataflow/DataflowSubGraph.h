// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"

#include "DataflowSubGraph.generated.h"

struct FDataflowSubGraphInputNode;
struct FDataflowSubGraphOutputNode;

namespace UE::Dataflow
{
	struct FEditAssetUtils;
}

class UDataflowSubGraph;

struct FDataflowSubGraphDelegates
{
	/** Called when variables are edited ( add, remove, changetype , setvalue ) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDataflowSubGraphLoaded, const UDataflowSubGraph&);
	static DATAFLOWENGINE_API FOnDataflowSubGraphLoaded OnSubGraphLoaded;
};

UCLASS(MinimalAPI)
class UDataflowSubGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

public:
	FDataflowSubGraphInputNode* GetInputNode() const;
	FDataflowSubGraphOutputNode* GetOutputNode() const;

	DATAFLOWENGINE_API bool IsForEachSubGraph() const;
	DATAFLOWENGINE_API void SetForEachSubGraph(bool bValue);

	bool IsLoaded() const { return bIsLoaded; }
	const FGuid& GetSubGraphGuid() const { return SubGraphGuid; }

private:
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif

	/**
	* Editor-only override of the SubGraph GUID.
	* Restricted to FEditAssetUtils via friend access so the paste-from-clipboard path
	* can preserve the original GUID when re-creating a copied subgraph.
	*/
	friend struct UE::Dataflow::FEditAssetUtils;
	void SetSubGraphGuid(const FGuid& InGuid) { SubGraphGuid = InGuid; }

	/**
	* Guid that uniquely identify this SubGraph
	* this allow to identify the SubGraph across renames for example
	*/
	UPROPERTY()
	FGuid SubGraphGuid;

	UPROPERTY()
	bool bIsForEach = false;

	bool bIsLoaded : 1 = false;
};