// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowConnection.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowDynamicConnections.h"
#include "StructUtils/PropertyBag.h"

#include "DataflowSubGraphNodes.generated.h"

struct FDataflowOutput;
struct FPropertyBagPropertyDesc;
class UDataflow;
class UDataflowSubGraph;

namespace UE::Dataflow
{
	enum class ESubGraphChangedReason: uint8;

	struct ISubGraphContextCallback
	{
		virtual void EvaluateParentContext(UE::Dataflow::FContext& Context, UE::Dataflow::FContext& SubGraphContext, const FDataflowOutput& SubGraphOutput) const = 0;
	};
}



/**
* This node represent the inputs of a dataflow subgraph 
*/
USTRUCT(meta=(Icon="GraphEditor.Function_16x"))
struct FDataflowSubGraphInputNode
	: public FDataflowNode
#if CPP
	, public FDataflowDynamicConnections::IOwnerInterface
#endif
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSubGraphInputNode, "SubGraphInput", "SubGraph", "")

public:
	FDataflowSubGraphInputNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());

	/* Begin IOwnerInterface */
	virtual FDataflowNode* GetOwner(const FDataflowDynamicConnections* Caller) override { return this; }
	virtual const FInstancedPropertyBag& GetPropertyBag(const FDataflowDynamicConnections* Caller) override { return PropertyBag; }
	/* End IOwnerInterface */

	DATAFLOWENGINE_API void AddConnectionsTo(TArrayView<FDataflowConnection*> Connections);

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void PostSerialize(const FArchive& Ar) override;
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;

	virtual bool SupportsDropConnectionOnNode(FName TypeName, UE::Dataflow::FPin::EDirection Direction) const override;
	virtual const FDataflowConnection* OnDropConnectionOnNode(const FDataflowConnection& DroppedConnection) override;

	UPROPERTY()
	FDataflowDynamicConnections DynamicConnections;

	UPROPERTY(EditAnyWhere, Category = Inputs, Meta = (DisplayName = "Inputs"))
	FInstancedPropertyBag PropertyBag;
};


/**
* This node represent the Outputs of a dataflow subgraph
*/
USTRUCT(meta = (Icon = "GraphEditor.Function_16x"))
struct FDataflowSubGraphOutputNode 
	: public FDataflowNode
#if CPP
	, public FDataflowDynamicConnections::IOwnerInterface
#endif
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSubGraphOutputNode, "SubGraphOutput", "SubGraph", "")

public:
	FDataflowSubGraphOutputNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());

public:
	/* Begin IOwnerInterface */
	virtual FDataflowNode* GetOwner(const FDataflowDynamicConnections* Caller) override { return this; }
	virtual const FInstancedPropertyBag& GetPropertyBag(const FDataflowDynamicConnections* Caller) override { return PropertyBag; }
	/* End IOwnerInterface */

	DATAFLOWENGINE_API void AddConnectionsTo(TArrayView<FDataflowConnection*> Connections);

private:
	// Subgraph output nodes are considered terminal
	virtual bool IsTerminal() const override
	{
		return true;
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void PostSerialize(const FArchive& Ar) override;
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;

	virtual bool SupportsDropConnectionOnNode(FName TypeName, UE::Dataflow::FPin::EDirection Direction) const override;
	virtual const FDataflowConnection* OnDropConnectionOnNode(const FDataflowConnection& DroppedConnection) override;

	// we want the user to be able to change the type of the properties in the property bag or rename them 
	virtual bool MakeConnectedPropertiesReadOnly() const override { return false; }

	UPROPERTY()
	FDataflowDynamicConnections DynamicConnections;

	UPROPERTY(EditAnyWhere, Category = Inputs, Meta = (DisplayName = "Inputs"))
	FInstancedPropertyBag PropertyBag;
};


/**
* Get the current index in a subgraph
* This is to be used in subgraph when iterating over an array
*/
USTRUCT(meta = (Icon = "GraphEditor.Function_16x"))
struct FDataflowSubGraphGetCurrentIndexNode
	: public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowSubGraphGetCurrentIndexNode, "GetCurrentIndex", "SubGraph", "")

public:
	FDataflowSubGraphGetCurrentIndexNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	UPROPERTY(Meta = (DataflowOutput))
	int32 Index = 0;
};

/**
* Call a subgraph
*/
USTRUCT(meta = (Icon = "GraphEditor.Function_16x"))
struct FDataflowCallSubGraphNode
	: public FDataflowNode
#if CPP
	, public FDataflowDynamicConnections::IOwnerInterface
	, public UE::Dataflow::ISubGraphContextCallback
#endif
{
	GENERATED_USTRUCT_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowCallSubGraphNode, "SubGraphCall", "SubGraph", "")

public:
	DATAFLOWENGINE_API FDataflowCallSubGraphNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid());
	DATAFLOWENGINE_API virtual ~FDataflowCallSubGraphNode() override;

	DATAFLOWENGINE_API void SetSubGraphGuid(const FGuid& Guid);
	const FGuid& GetSubGraphGuid() const { return SubGraphGuid; }

	DATAFLOWENGINE_API void RefreshSubGraphName();

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void PostSerialize(const FArchive& Ar) override;

	/* Begin ISubGraphContextCallback */
	virtual void EvaluateParentContext(UE::Dataflow::FContext& Context, UE::Dataflow::FContext& SubGraphContext, const FDataflowOutput& SubGraphOutput) const override;
	/* End ISubGraphContextCallback */

	/* Begin IOwnerInterface */
	virtual FDataflowNode* GetOwner(const FDataflowDynamicConnections* Caller) override { return this; }
	virtual const FInstancedPropertyBag& GetPropertyBag(const FDataflowDynamicConnections* Caller) override;
	/* End IOwnerInterface */


	void UnregisterHandlers(UDataflowSubGraph* SubGraph);
	void RegisterHandlers(UDataflowSubGraph* SubGraph);
	void OnSubGraphChanged(const struct FEdGraphEditAction& InAction);
	void OnSomeSubGraphsChanged(const UDataflow* InDataflowAsset, const FGuid& InSubGraphGuid, UE::Dataflow::ESubGraphChangedReason InReason);
	void OnSubGraphLoaded(const UDataflowSubGraph& InSubGraph);
	
	void SyncInputsPropertyBagWithSubGraph();
	void SyncOutputsPropertyBagWithSubGraph();

	bool IsValid() const;

	UPROPERTY()
	FGuid SubGraphGuid;

	UPROPERTY(VisibleAnyWhere, Category = Subgraph)
	FName SubGraphName;

	UPROPERTY()
	FDataflowDynamicConnections DynamicInputs;

	UPROPERTY()
	FInstancedPropertyBag InputsPropertyBag;

	UPROPERTY()
	FDataflowDynamicConnections DynamicOutputs;

	UPROPERTY()
	FInstancedPropertyBag OutputsPropertyBag;

	TWeakObjectPtr<UDataflow> DataflowAssetWeakPtr;
	FDelegateHandle OnGraphChangedHandle;
};

namespace UE::Dataflow
{
	void RegisterSubGraphNodes();
}
