// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"

#include "DataflowVariableNodes.generated.h"

class UDataflow;
class UObject;
struct FInstancedPropertyBag;
struct FPropertyChangedEvent;

USTRUCT()
struct FGetDataflowVariableNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetDataflowVariableNode, "GetVariable", "Dataflow", "")

public:
	DATAFLOWENGINE_API FGetDataflowVariableNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	DATAFLOWENGINE_API ~FGetDataflowVariableNode();

	DATAFLOWENGINE_API void SetVariable(UDataflow* DataflowAsset, FName VariableName);
	DATAFLOWENGINE_API bool TryAddVariableToDataflowAsset(UDataflow& DataflowAsset);

	FName GetVariableName() const { return VariableName; }

private:
	UPROPERTY(meta=(DataflowOutput))
	FDataflowAnyType Value;

	// Called when UPROPERTY members of the dataflow node have been changed in the editor
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;

	/** Override this method to provide custom post-serialization for this node. This method will be called after Serialize. It is also called after copy-paste with ArchiveState IsLoading. */
	virtual void PostSerialize(const FArchive& Ar);

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;

	static void EvaluateBool(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateByte(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateInt32(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateInt64(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateFloat(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateDouble(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateName(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateString(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateText(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateObject(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);
	static void EvaluateStruct(const FGetDataflowVariableNode& Node, const FInstancedPropertyBag& Variables, UE::Dataflow::FContext& Context, const FDataflowOutput& Out);

	void UpdateOutputTypes(const FPropertyBagPropertyDesc& Desc);
	void ChangeOutputType(FDataflowOutput* Output, FName NewType);

private:
	UPROPERTY(EditAnywhere, Category = "Default Value", meta = (FixedLayout, ShowOnlyInnerProperties))
	FInstancedPropertyBag VariablePropertyBag;

	UPROPERTY(VisibleAnywhere, Category = Variable )
	FName VariableName;

	/** Category the variable should be displayed under */
	UPROPERTY(EditAnywhere, Category = Variable)
	FString Category;

	/** Decsription of this variable, the description will show in the tooltip of overrides */
	UPROPERTY(EditAnywhere, Category = Variable)
	FString Description;

	void RegisterHandlers();
	void UnregisterHandlers();

	using FStaticEvaluationFunctionPtr = void(*)(const FGetDataflowVariableNode&, const FInstancedPropertyBag&, UE::Dataflow::FContext&, const FDataflowOutput&);
	FStaticEvaluationFunctionPtr EvaluateFunction = nullptr;

	EPropertyBagContainerType ContainerType = EPropertyBagContainerType::None;

	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);
	void NotifyAssetVariablesChanged(UDataflow* DataflowAsset);
	void SetVariableMetaData(FName MetaDataName, const FString& Value);

	TWeakObjectPtr<UDataflow> WeakDataflowPtr;

};

namespace UE::Dataflow
{
	void RegisterVariableNodes();
}

