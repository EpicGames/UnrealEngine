// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataflowSimulation : ModuleRules
	{
		public DataflowSimulation(ReadOnlyTargetRules Target) : base(Target)
		{
			bTreatAsEngineModule = true;
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DataflowCore",
					"DataflowEngine",
					"RigidPhysics",
				}
			);
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// NOTE: UVEditorTools is a separate module so that it doesn't rely on the editor.
					// So, do not add editor dependencies here.
	
					"Engine",
					"RenderCore",
					"RHI",
					"DataflowCore",
					"DataflowEngine",
					"Chaos",
				}
			);
			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}