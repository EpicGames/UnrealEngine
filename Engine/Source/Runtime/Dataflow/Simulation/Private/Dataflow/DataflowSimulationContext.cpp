// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationContext.h"
#include "Dataflow/DataflowSimulationProxy.h"
#include "Dataflow/RigidPhysicsSceneState.h"

namespace UE::Dataflow
{
	
template<typename Base>
void TSimulationContext<Base>::GetSimulationProxies(const FString& ProxyType, const TArray<FString>& SimulationGroups, TArray<FDataflowSimulationProxy*>& FilteredProxies) const
{
	TBitArray<> ProxyGroups(false, GroupIndices.Num());
	BuildGroupBits(SimulationGroups, ProxyGroups);
	
	if(const TSet<FDataflowSimulationProxy*>* TypedProxies = SimulationProxies.Find(ProxyType))
	{
		for(FDataflowSimulationProxy* SimulationProxy : *TypedProxies)
		{
			if(SimulationProxy && SimulationProxy->HasGroupBit(ProxyGroups))
			{
				FilteredProxies.Add(SimulationProxy);
			}
		}
	}
}

template<typename Base>
void TSimulationContext<Base>::BuildGroupBits(const TArray<FString>& SimulationGroups, TBitArray<>& GroupBits) const
{
	GroupBits.Init(false, GroupIndices.Num());
    for(const FString& SimulationGroup : SimulationGroups)
    {
    	if(const uint32* GroupIndex = GroupIndices.Find(SimulationGroup))
    	{
    		GroupBits[*GroupIndex] = true;
    	}
    }
}
	
template<typename Base>
void TSimulationContext<Base>::RegisterProxyGroups()
{
	GroupIndices.Reset();
	TArray<uint32> IndexArray;
	
	for(const TPair<FString, TSet<FDataflowSimulationProxy*>>& TypedProxies : SimulationProxies)
	{
		for(FDataflowSimulationProxy* SimulationProxy : TypedProxies.Value)
		{
			if(SimulationProxy)
			{
				IndexArray.Reset(SimulationProxy->GetSimulationGroups().Num());
				for(const FString& SimulationGroup : SimulationProxy->GetSimulationGroups())
				{
					const uint32* IndexPtr = GroupIndices.Find(SimulationGroup);
					IndexArray.Add( (IndexPtr == nullptr) ?
							GroupIndices.Add(SimulationGroup, GroupIndices.Num()) : *IndexPtr);
				}
				SimulationProxy->GroupBits.Init(false, GroupIndices.Num());
				for(const uint32& GroupIndex : IndexArray)
				{
					SimulationProxy->GroupBits[GroupIndex] = true;
				}
			}
		}
	}
}

template<typename Base>
void TSimulationContext<Base>::AddSimulationProxy(const FString& ProxyType, FDataflowSimulationProxy* SimulationProxy)
{
	SimulationProxies.FindOrAdd(ProxyType).Add(SimulationProxy);
}
	
template<typename Base>
void TSimulationContext<Base>::RemoveSimulationProxy(const FString& ProxyType, const FDataflowSimulationProxy* SimulationProxy)
{
	SimulationProxies.FindOrAdd(ProxyType).Remove(SimulationProxy);
}

template<typename Base>
void TSimulationContext<Base>::ResetSimulationProxies()
{
	SimulationProxies.Reset();
}
	
template<typename Base>
int32 TSimulationContext<Base>::NumSimulationProxies(const FString& ProxyType) const
{
	const TSet<FDataflowSimulationProxy*>* TypedProxies = SimulationProxies.Find(ProxyType);
	return TypedProxies ? TypedProxies->Num() : 0;
}

#if UE_RIGIDPHYSICS_API_ENABLED 

template<typename Base>
FRigidPhysicsSceneDataflowState* TSimulationContext<Base>::GetRigidState()
{
	if (!RigidState)
	{
		RigidState = MakePimpl<FRigidPhysicsSceneDataflowState>();
	}

	return RigidState.Get();
}

template<typename Base>
void TSimulationContext<Base>::SetRigidSceneName(const FString& InSceneName)
{
	SceneName = InSceneName;
}

template<typename Base>
const FString& TSimulationContext<Base>::GetRigidSceneName()
{
	return SceneName;
}

#endif

template class TSimulationContext<UE::Dataflow::FContextSingle>;
template class TSimulationContext<UE::Dataflow::FContextThreaded>;
	
}