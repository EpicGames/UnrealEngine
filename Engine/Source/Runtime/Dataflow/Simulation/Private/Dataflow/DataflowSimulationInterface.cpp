// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationInterface.h"
#include "Dataflow/DataflowSimulationManager.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSimulationInterface)
	
void IDataflowSimulationInterface::RegisterManagerInterface(const TObjectPtr<UWorld>& SimulationWorld)
{
	if(SimulationWorld)
	{
		if (UDataflowSimulationManager* SimulationManager = SimulationWorld->GetSubsystem<UDataflowSimulationManager>())
		{
			SimulationManager->AddSimulationInterface(this);
		}
	}
}

bool IDataflowSimulationInterface::IsInterfaceRegistered(const TObjectPtr<UWorld>& SimulationWorld) const
{
	if(SimulationWorld)
	{
		if (const UDataflowSimulationManager* SimulationManager = SimulationWorld->GetSubsystem<UDataflowSimulationManager>())
		{
			return SimulationManager->HasSimulationInterface(this);
		}
	}
	return false;
}

void IDataflowSimulationInterface::UnregisterManagerInterface(const TObjectPtr<UWorld>& SimulationWorld) const
{
	if(SimulationWorld)
	{
		if (UDataflowSimulationManager* SimulationManager = SimulationWorld->GetSubsystem<UDataflowSimulationManager>())
		{
			SimulationManager->RemoveSimulationInterface(this); 
		}
	}
}
