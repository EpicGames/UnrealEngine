// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationManager.h"
#include "Dataflow/DataflowSimulationUtils.h"
#include "Dataflow/DataflowSimulationInterface.h"
#include "Dataflow/DataflowSimulationProxy.h"
#include "Components/ActorComponent.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Map.h"
#include "Dataflow/Interfaces/DataflowPhysicsSolver.h"
#include "ChaosDebugDraw/ChaosDDContext.h"
#include "Engine/World.h"
#include "Kismet/GameplayStatics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSimulationManager)

namespace UE::Dataflow
{
	namespace Private
	{
		enum class ESimulationThreadingMode : uint8
		{
			GameThread,
			BlockingThread,
			AsyncThread,
		};

		/** Simulation task priority */
		static FAutoConsoleTaskPriority CVarDataflowSimulationTaskPriority(
			TEXT("TaskGraph.TaskPriorities.DataflowSimulationTask"),
			TEXT("Task and thread priority for the dataflow simulation."),
			ENamedThreads::HighThreadPriority, // If we have high priority task threads, then use them...
			ENamedThreads::NormalTaskPriority, // .. at normal task priority
			ENamedThreads::HighTaskPriority);  // If we don't have high priority threads, then use normal priority threads at high task priority instead
		
		/** Simulation threading mode */
		int32  DataflowSimulationThreadingMode = (uint8)ESimulationThreadingMode::AsyncThread;
		FAutoConsoleVariableRef CVarDataflowSimulationThreadingMode(TEXT("p.Dataflow.Simulation.ThreadingMode"), DataflowSimulationThreadingMode,
			TEXT("0 : run simulation on GT | 1 : run simulation on PT (GT is blocked in manager Tick) | 2 : run simulation on PT (GT will be blocked at the end of the world tick)"));

		/** Simulation task use to run async the dataflow evaluation */
		class FDataflowSimulationTask
		{
		public:

#if CHAOS_DEBUG_DRAW
			FDataflowSimulationTask(const TObjectPtr<UDataflow>& InDataflowAsset, const TSharedPtr<FDataflowSimulationContext>& InSimulationContext,
				const float InDeltaTime, const float InSimulationTime, const ChaosDD::Private::FChaosDDTaskParentContext& InParentDDContext)
				: DataflowAsset(InDataflowAsset), SimulationContext(InSimulationContext), DeltaTime(InDeltaTime), SimulationTime(InSimulationTime), ParentDDContext(InParentDDContext)
			{
			}
#else
			FDataflowSimulationTask(const TObjectPtr<UDataflow>& InDataflowAsset, const TSharedPtr<FDataflowSimulationContext>& InSimulationContext,
							const float InDeltaTime, const float InSimulationTime)
							: DataflowAsset(InDataflowAsset), SimulationContext(InSimulationContext), DeltaTime(InDeltaTime), SimulationTime(InSimulationTime)
			{
			}
#endif

			TStatId GetStatId() const
			{
				RETURN_QUICK_DECLARE_CYCLE_STAT(FDataflowSimulationProxyParallelTask, STATGROUP_TaskGraphTasks);
			}

			static ENamedThreads::Type GetDesiredThread()
			{
				if (CVarDataflowSimulationTaskPriority.Get() != 0)
				{
					return CVarDataflowSimulationTaskPriority.Get();
				}
				return ENamedThreads::GameThread;
			}

			static ESubsequentsMode::Type GetSubsequentsMode()
			{
				return ESubsequentsMode::TrackSubsequents;
			}

			void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
			{
#if CHAOS_DEBUG_DRAW
				ChaosDD::Private::FChaosDDScopeTaskContext DDTaskContext(ParentDDContext);
#endif
				UE::Dataflow::EvaluateSimulationGraph(DataflowAsset, SimulationContext, DeltaTime, SimulationTime);
			}

		private:
			/** Dataflow graph asset used to launch the simulation */
			TObjectPtr<UDataflow> DataflowAsset;

			/** Simulation context */
			TSharedPtr<UE::Dataflow::FDataflowSimulationContext> SimulationContext;

			/** Delta time used to advance the simulation */
			float DeltaTime;

			/** World simulation time  */
			float SimulationTime;

#if CHAOS_DEBUG_DRAW
			/** Parent debug draw context */
			const ChaosDD::Private::FChaosDDTaskParentContext ParentDDContext;
#endif
		};

		inline void PreSimulationTick(const TObjectPtr<UObject>& SimulationWorld, const float SimulationTime, const float DeltaTime)
		{
			if(SimulationWorld)
			{
				TArray<AActor*> Actors;
				UGameplayStatics::GetAllActorsWithInterface(SimulationWorld, UDataflowSimulationActor::StaticClass(), Actors);

				for (AActor* CurrentActor : Actors)
				{
					IDataflowSimulationActor::Execute_PreDataflowSimulationTick(CurrentActor, SimulationTime, DeltaTime);
				}
			}
		}

		inline void PostSimulationTick(const TObjectPtr<UObject>& SimulationWorld, const float SimulationTime, const float DeltaTime)
		{
			if(SimulationWorld)
			{
				TArray<AActor*> Actors;
				UGameplayStatics::GetAllActorsWithInterface(SimulationWorld, UDataflowSimulationActor::StaticClass(), Actors);

				for (AActor* CurrentActor : Actors)
				{
					IDataflowSimulationActor::Execute_PostDataflowSimulationTick(CurrentActor, SimulationTime, DeltaTime);
				}
			}
		}

		inline void NotifySimulationEnabledChanged(const TObjectPtr<UObject>& SimulationWorld, const bool bSimulationEnabled)
		{
			if (SimulationWorld)
			{
				TArray<AActor*> Actors;
				UGameplayStatics::GetAllActorsWithInterface(SimulationWorld, UDataflowSimulationActor::StaticClass(), Actors);

				for (AActor* CurrentActor : Actors)
				{
					IDataflowSimulationActor::Execute_OnDataflowSimulationEnabledChanged(CurrentActor, bSimulationEnabled);
				}
			}
		}
	}

	void RegisterSimulationInterface(const TObjectPtr<UObject>& SimulationObject)
	{
		if(IDataflowSimulationInterface* SimulationInterface = Cast<IDataflowSimulationInterface>(SimulationObject))
		{
			if(SimulationInterface->GetSimulationAsset().DataflowAsset)
			{
				FDataflowSimulationProxy* SimulationProxy = SimulationInterface->GetSimulationProxy();
				
				if(!SimulationProxy || (SimulationProxy && !SimulationProxy->IsValid()))
				{
					// Build the simulation proxy
					SimulationInterface->BuildSimulationProxy();
				}

				// Register the simulation interface to the manager
				SimulationInterface->RegisterManagerInterface(SimulationObject->GetWorld());
			}
		}
	}

	void UnregisterSimulationInterface(const TObjectPtr<UObject>& SimulationObject)
	{
		if(IDataflowSimulationInterface* SimulationInterface = Cast<IDataflowSimulationInterface>(SimulationObject))
		{
			if(SimulationInterface->GetSimulationAsset().DataflowAsset)
			{
				FDataflowSimulationProxy* SimulationProxy = SimulationInterface->GetSimulationProxy();
				
				if(SimulationProxy && SimulationProxy->IsValid())
				{
					// Reset the simulation proxy
					SimulationInterface->ResetSimulationProxy();
				}

				// Unregister the simulation interface from the manager
				SimulationInterface->UnregisterManagerInterface(SimulationObject->GetWorld());
			}
		}
	}
}

FDelegateHandle UDataflowSimulationManager::OnObjectPropertyChangedHandle;
FDelegateHandle UDataflowSimulationManager::OnWorldPostActorTick;
FDelegateHandle UDataflowSimulationManager::OnCreatePhysicsStateHandle;
FDelegateHandle UDataflowSimulationManager::OnDestroyPhysicsStateHandle;

UDataflowSimulationManager::UDataflowSimulationManager()
{}

void UDataflowSimulationManager::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
	
	check(IsInGameThread());
	//check(SimulationTasks.IsEmpty());
	
	PreProcessSimulation(DeltaTime);
	
	if(bIsSimulationEnabled)
	{
		// Transfer data from GT -> PT
		ReadSimulationInterfaces(DeltaTime, false);
		
		if(UE::Dataflow::Private::DataflowSimulationThreadingMode == static_cast<uint8>(UE::Dataflow::Private::ESimulationThreadingMode::GameThread))
		{
			// Advance the simulation in time
			AdvanceSimulationProxies(DeltaTime, GetWorld()->GetTimeSeconds());

			// Transfer data from PT -> GT
			WriteSimulationInterfaces(DeltaTime, false);
		}
		else
		{
			// Start all the simulation tasks in parallel
			StartSimulationTasks(DeltaTime, GetWorld()->GetTimeSeconds());

			if(UE::Dataflow::Private::DataflowSimulationThreadingMode == static_cast<uint8>(UE::Dataflow::Private::ESimulationThreadingMode::BlockingThread))
            {
            	// Wait until all tasks are complete
            	CompleteSimulationTasks();
            
            	// Transfer data from PT -> GT
            	WriteSimulationInterfaces(DeltaTime, false);
            }
		}
	}
	PostProcessSimulation(DeltaTime);
}

void UDataflowSimulationManager::SetSimulationEnabled(const bool bSimulationEnabled)
{
	if (bIsSimulationEnabled != bSimulationEnabled)
	{
		bIsSimulationEnabled = bSimulationEnabled;
		UE::Dataflow::Private::NotifySimulationEnabledChanged(GetWorld(), bSimulationEnabled);
	}
}

void UDataflowSimulationManager::NotifyActorOfCurrentSimulationEnabled(AActor* PreviewActor) const
{
	if (PreviewActor && PreviewActor->Implements<UDataflowSimulationActor>())
	{
		IDataflowSimulationActor::Execute_OnDataflowSimulationEnabledChanged(PreviewActor, bIsSimulationEnabled);
	}
}

void UDataflowSimulationManager::OnStartup()
{
	OnWorldPostActorTick = FWorldDelegates::OnWorldPostActorTick.AddLambda(
	[](const UWorld* SimulationWorld, ELevelTick LevelTick, const float DeltaSeconds)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UDataflowSimulationManager::OnWorldPostActorTick)

		if(SimulationWorld)
		{
			if (UDataflowSimulationManager* DataflowManager = SimulationWorld->GetSubsystem<UDataflowSimulationManager>())
			{
				if(DataflowManager->bIsSimulationEnabled)
				{
					if(UE::Dataflow::Private::DataflowSimulationThreadingMode == static_cast<uint8>(UE::Dataflow::Private::ESimulationThreadingMode::AsyncThread))
					{
						// Wait until all tasks are complete
						DataflowManager->CompleteSimulationTasks();
					
						// Transfer data from PT -> GT
						DataflowManager->WriteSimulationInterfaces(DeltaSeconds, false);
					}
				}
			}
		}
		
	});

	OnCreatePhysicsStateHandle = UActorComponent::GlobalCreatePhysicsDelegate.AddLambda([](UActorComponent* ActorComponent)
	{
		UE::Dataflow::RegisterSimulationInterface(ActorComponent);
	});

	OnDestroyPhysicsStateHandle = UActorComponent::GlobalDestroyPhysicsDelegate.AddLambda([](UActorComponent* ActorComponent)
	{
		UE::Dataflow::UnregisterSimulationInterface(ActorComponent);
	});

#if WITH_EDITOR
	OnObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddLambda([](UObject* ModifiedObject, FPropertyChangedEvent& ChangedProperty)
	{
		if(IDataflowSimulationInterface* SimulationInterface = Cast<IDataflowSimulationInterface>(ModifiedObject))
		{
			if(!SimulationInterface->IsInterfaceRegistered(ModifiedObject->GetWorld()))
			{
				// Unregister the simulation interface from the manager
				SimulationInterface->UnregisterManagerInterface(ModifiedObject->GetWorld());
                
                // Register the simulation interface to the manager
                SimulationInterface->RegisterManagerInterface(ModifiedObject->GetWorld());
			}
		}
	});
#endif
}

void UDataflowSimulationManager::OnShutdown()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);
#endif
	FWorldDelegates::OnWorldPostActorTick.Remove(OnWorldPostActorTick);
	UActorComponent::GlobalCreatePhysicsDelegate.Remove(OnCreatePhysicsStateHandle);
	UActorComponent::GlobalDestroyPhysicsDelegate.Remove(OnDestroyPhysicsStateHandle);
}

void UDataflowSimulationManager::PreProcessSimulation(const float DeltaTime)
{
	for(const TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
	{
		for(const TPair<FString,TSet<IDataflowSimulationInterface*>>& SimulationInterfaces : DataflowData.Value.SimulationInterfaces)
		{
			for(IDataflowSimulationInterface* SimulationInterface : SimulationInterfaces.Value)
			{
				if(SimulationInterface)
				{
					SimulationInterface->PreProcessSimulation(DeltaTime);
				}
			}
		}
	}
}

void UDataflowSimulationManager::PostProcessSimulation(const float DeltaTime)
{
	for(const TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
	{
		for(const TPair<FString,TSet<IDataflowSimulationInterface*>>& SimulationInterfaces : DataflowData.Value.SimulationInterfaces)
		{
			for(IDataflowSimulationInterface* SimulationInterface : SimulationInterfaces.Value)
			{
				if(SimulationInterface)
				{
					SimulationInterface->PostProcessSimulation(DeltaTime);
				}
			}
		}
	}
}

void UDataflowSimulationManager::ReadSimulationInterfaces(const float DeltaTime, const bool bAsyncTask)
{
	// Pre-simulation callback that could be used in BP before the simulation
	//Dataflow::PreSimulationTick(GetWorld(), GetWorld()->GetTimeSeconds(), DeltaTime);

	InitSimulationInterfaces();
	for(const TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
	{
		for(const TPair<FString,TSet<IDataflowSimulationInterface*>>& SimulationInterfaces : DataflowData.Value.SimulationInterfaces)
		{
			for(IDataflowSimulationInterface* SimulationInterface : SimulationInterfaces.Value)
			{
				if(SimulationInterface)
				{
					SimulationInterface->WriteToSimulation(DeltaTime, bAsyncTask);
				}
			}
		}
	}
}

void UDataflowSimulationManager::InitSimulationInterfaces()
{
	for(const TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
	{
		if(DataflowData.Value.SimulationContext.IsValid())
		{
			DataflowData.Value.SimulationContext->ResetSimulationProxies();
		}
		for(const TPair<FString,TSet<IDataflowSimulationInterface*>>& SimulationInterfaces : DataflowData.Value.SimulationInterfaces)
		{
			for(IDataflowSimulationInterface* SimulationInterface : SimulationInterfaces.Value)
			{
				if(SimulationInterface)
				{
					if(!SimulationInterface->GetSimulationProxy())
					{
						SimulationInterface->BuildSimulationProxy();
					}
					if(SimulationInterface->GetSimulationProxy())
					{
						SimulationInterface->GetSimulationProxy()->SetSimulationGroups(SimulationInterface->GetSimulationAsset().SimulationGroups);
						if(DataflowData.Value.SimulationContext.IsValid())
						{
							DataflowData.Value.SimulationContext->AddSimulationProxy(SimulationInterfaces.Key, SimulationInterface->GetSimulationProxy());
						}
					}
				}
			}
		}
		if(DataflowData.Value.SimulationContext.IsValid())
		{
			DataflowData.Value.SimulationContext->RegisterProxyGroups();
		}
	}
}

void UDataflowSimulationManager::ResetSimulationInterfaces()
{
	for(const TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
	{
		if(DataflowData.Value.SimulationContext.IsValid())
		{
			DataflowData.Value.SimulationContext->ResetSimulationProxies();
		}
	}
}

void UDataflowSimulationManager::WriteSimulationInterfaces(const float DeltaTime, const bool bAsyncTask)
{
	for(const TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
	{
		for(const TPair<FString,TSet<IDataflowSimulationInterface*>>& SimulationInterfaces : DataflowData.Value.SimulationInterfaces)
		{
			for(IDataflowSimulationInterface* SimulationInterface : SimulationInterfaces.Value)
			{
				if(SimulationInterface)
				{
					SimulationInterface->ReadFromSimulation(DeltaTime, bAsyncTask);
				}
			}
		}
	}
	if(bStepSimulationScene)
	{
		SetSimulationEnabled(false);
		bStepSimulationScene = false;
	}
	ResetSimulationInterfaces();
	// Post-simulation callback that could be used in BP after the simulation
	//Dataflow::PostSimulationTick(GetWorld(), GetWorld()->GetTimeSeconds(), DeltaTime);
}

void UDataflowSimulationManager::ReadRestartData()
{
	for (const TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
	{
		for (const TPair<FString, TSet<IDataflowSimulationInterface*>>& SimulationInterfaces : DataflowData.Value.SimulationInterfaces)
		{
			for (IDataflowSimulationInterface* SimulationInterface : SimulationInterfaces.Value)
			{
				if (SimulationInterface)
				{
					SimulationInterface->ReadRestartData();
				}
			}
		}
	}
}

void UDataflowSimulationManager::AdvanceSimulationProxies(const float DeltaTime, const float SimulationTime)
{
	for(TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
	{
		UE::Dataflow::EvaluateSimulationGraph(DataflowData.Key, DataflowData.Value.SimulationContext, DeltaTime, SimulationTime);
	}
}

void UDataflowSimulationManager::StartSimulationTasks(const float DeltaTime, const float SimulationTime)
{
	check(IsInGameThread());
	
	// Wait until all tasks are complete
	CompleteSimulationTasks();
	
	check(SimulationTasks.IsEmpty());

	// Parent debug draw context
#if CHAOS_DEBUG_DRAW
	const ChaosDD::Private::FChaosDDTaskParentContext ParentDDContext;
#endif

	for(TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
	{
		if(!DataflowData.Value.IsEmpty())
		{
			// Add a simulation task linked to that solver
#if CHAOS_DEBUG_DRAW
			SimulationTasks.Add(TGraphTask<UE::Dataflow::Private::FDataflowSimulationTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(
					DataflowData.Key, DataflowData.Value.SimulationContext, DeltaTime, SimulationTime, ParentDDContext));
#else
			SimulationTasks.Add(TGraphTask<UE::Dataflow::Private::FDataflowSimulationTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(
					DataflowData.Key, DataflowData.Value.SimulationContext, DeltaTime, SimulationTime));
#endif
		}
	}
}

void UDataflowSimulationManager::CompleteSimulationTasks()
{
	check(IsInGameThread());

	for(FGraphEventRef& SimulationTask : SimulationTasks)
	{
		if (IsValidRef(SimulationTask))
		{
			// There's a simulation in flight
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(SimulationTask, ENamedThreads::GameThread);

			// No longer need this task, it has completed
			SimulationTask.SafeRelease();
		}
	}
	SimulationTasks.Reset();
}

TSharedPtr<UE::Dataflow::FDataflowSimulationContext> UDataflowSimulationManager::GetSimulationContext(const TObjectPtr<UDataflow>& DataflowAsset) const
{
	if(DataflowAsset)
	{
		if(const UE::Dataflow::Private::FDataflowSimulationData* DataflowData = SimulationData.Find(DataflowAsset))
		{
			return DataflowData->SimulationContext;
		}
	}
	return nullptr;
}

bool UDataflowSimulationManager::HasSimulationInterface(const IDataflowSimulationInterface* SimulationInterface) const
{
	if(SimulationInterface)
	{
		if(const TObjectPtr<UDataflow> DataflowAsset = SimulationInterface->GetSimulationAsset().DataflowAsset)
		{
			if(const UE::Dataflow::Private::FDataflowSimulationData* DataflowData = SimulationData.Find(DataflowAsset))
			{
				if(const TSet<IDataflowSimulationInterface*>* SimulationInterfaces =
					DataflowData->SimulationInterfaces.Find(SimulationInterface->GetSimulationType()))
				{
					return SimulationInterfaces->Find(SimulationInterface) != nullptr;
				}
			}
		}
	}
	return false;
}

void UDataflowSimulationManager::AddSimulationInterface(IDataflowSimulationInterface* SimulationInterface)
{
	if(SimulationInterface)
	{
		if(TObjectPtr<UDataflow> DataflowAsset = SimulationInterface->GetSimulationAsset().DataflowAsset)
		{
			UE::Dataflow::Private::FDataflowSimulationData& DataflowData = SimulationData.FindOrAdd(DataflowAsset);
			if(!DataflowData.SimulationContext.IsValid())
			{
				DataflowData.SimulationContext = MakeShared<UE::Dataflow::FDataflowSimulationContext>(DataflowAsset);
			}
			DataflowData.SimulationInterfaces.FindOrAdd(SimulationInterface->GetSimulationType()).Add(SimulationInterface);
		}
	}
}

void UDataflowSimulationManager::RemoveSimulationInterface(const IDataflowSimulationInterface* SimulationInterface)
{
	if(SimulationInterface)
	{
		for(TPair<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData>& DataflowData : SimulationData)
		{
			if(TSet<IDataflowSimulationInterface*>* SimulationInterfaces =
					DataflowData.Value.SimulationInterfaces.Find(SimulationInterface->GetSimulationType()))
			{
				SimulationInterfaces->Remove(SimulationInterface);
			}
		}
	}
}

ETickableTickType UDataflowSimulationManager::GetTickableTickType() const
{
	return HasAnyFlags(RF_ClassDefaultObject) || !GetWorld() || GetWorld()->IsNetMode(NM_DedicatedServer) ? ETickableTickType::Never : ETickableTickType::Always;
}

TStatId UDataflowSimulationManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UDataflowSimulationManager, STATGROUP_Tickables);
}

bool UDataflowSimulationManager::DoesSupportWorldType(EWorldType::Type WorldType) const
{
	return Super::DoesSupportWorldType(WorldType) || WorldType == EWorldType::EditorPreview || WorldType == EWorldType::GamePreview || WorldType == EWorldType::GameRPC;
}

void UDataflowSimulationManager::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
}

void UDataflowSimulationManager::Deinitialize()
{
	Super::Deinitialize();

	CompleteSimulationTasks();
}


