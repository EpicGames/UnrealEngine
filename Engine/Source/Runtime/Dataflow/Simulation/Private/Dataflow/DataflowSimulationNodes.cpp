// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationNodes.h"
#include "Dataflow/DataflowSimulationManager.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"
#include "Dataflow/Interfaces/DataflowPhysicsSolver.h"
#include "Dataflow/RigidPhysicsDataflowNodes.h"

#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSimulationNodes)

namespace UE::Dataflow
{
	void RegisterDataflowSimulationNodes()
	{
		// Common nodes
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetSimulationTimeDataflowNode);
		
		// Solvers nodes
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetPhysicsSolversDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAdvancePhysicsSolversDataflowNode);

		// Proxies nodes
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFilterSimulationProxiesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSimulationProxiesTerminalDataflowNode);

#if UE_RIGIDPHYSICS_API_ENABLED
		// Scene Setup
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRigidSceneSetupTerminalDataflowNode);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAdvanceRigidPhysicsSolverDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSceneSimulationTerminalDataflowNode);
		RegisterRigidPhysicsDataflowSimulationNodes();
#endif
		
		static constexpr FLinearColor CDefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.0f, 0.5f);
		
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Terminal", FLinearColor(1.0f, 0.0f, 0.0f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Setup", FLinearColor(1.0f, 1.0f, 0.0f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Physics", FLinearColor(0.577580f, 0.527115f, 0.215861f), CDefaultNodeBodyTintColor);
	}
}

void FGetSimulationTimeDataflowNode::EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const
{
	SetValue(SimulationContext, FDataflowSimulationTime(SimulationContext.GetDeltaTime(), SimulationContext.GetSimulationTime()), &SimulationTime);
}

void FAdvancePhysicsSolversDataflowNode::EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const
{
	const TArray<FDataflowSimulationProperty> SolverProperties = GetValue(SimulationContext, &PhysicsSolvers);
	const float SimulationDeltaTime = GetValue(SimulationContext, &SimulationTime).DeltaTime;
	
	if(!SolverProperties.IsEmpty())
	{
		for(const FDataflowSimulationProperty& SolverProperty : SolverProperties)
		{
			if(SolverProperty.SimulationProxy)
			{
				if(FDataflowPhysicsSolverProxy* SolverProxy = SolverProperty.SimulationProxy->AsType<FDataflowPhysicsSolverProxy>())
				{
					SolverProxy->AdvanceSolverDatas(SimulationDeltaTime);
				}
			}
		}
	}
	SetValue(SimulationContext, SolverProperties, &PhysicsSolvers);
}

void FGetPhysicsSolversDataflowNode::EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const
{
	TArray<FDataflowSimulationProxy*> SimulationProxies;
	SimulationContext.GetSimulationProxies(FDataflowPhysicsSolverProxy::StaticStruct()->GetName(), SimulationGroups, SimulationProxies);

	TArray<FDataflowSimulationProperty> SolverProperties;
	for(FDataflowSimulationProxy* SimulationProxy : SimulationProxies)
	{
		SolverProperties.Add({SimulationProxy});
	}
	
	SetValue(SimulationContext, SolverProperties, &PhysicsSolvers);
}

void FFilterSimulationProxiesDataflowNode::EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const
{
	const TArray<FDataflowSimulationProperty> SimulationProperties = GetValue(SimulationContext, &SimulationProxies);

	TArray<FDataflowSimulationProperty> FilteredProperties;
	if(!SimulationProperties.IsEmpty())
	{
		TBitArray<> GroupBits;
		SimulationContext.BuildGroupBits(SimulationGroups, GroupBits);

		for(const FDataflowSimulationProperty& SimulationProperty : SimulationProperties)
		{
			if(SimulationProperty.SimulationProxy && SimulationProperty.SimulationProxy->HasGroupBit(GroupBits))
			{
				FilteredProperties.Add({SimulationProperty.SimulationProxy});
			}
		}
	}
	
	SetValue(SimulationContext, FilteredProperties, &FilteredProxies);
}

void FSimulationProxiesTerminalDataflowNode::EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const
{
	const TArray<FDataflowSimulationProperty> SolverProperties = GetValue(SimulationContext, &SimulationProxies);
}

void FAdvanceRigidPhysicsSolverDataflowNode::EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const
{
#if UE_RIGIDPHYSICS_API_ENABLED
	const FRigidPhysicsSceneDataflowState InRigidState = GetValue(SimulationContext, &State);

	const float SimulationDeltaTime = GetValue(SimulationContext, &SimulationTime).DeltaTime;
	
	if(FRigidPhysicsSceneDataflowState* RigidState = SimulationContext.GetRigidState())
	{
		if (UE::Physics::FRigidContextGameRW Context = RigidState->Handle.LockRW())
		{
			Context->StartTick(SimulationDeltaTime);
			Context->EndTick();
		}
	}

	SetValue(SimulationContext, InRigidState, &State);
#endif
}

void FSceneSimulationTerminalDataflowNode::EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const
{
#if UE_RIGIDPHYSICS_API_ENABLED
	FRigidPhysicsSceneDataflowState InState = GetValue(SimulationContext, &State);
#endif
}

void FRigidSceneSetupTerminalDataflowNode::EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const
{
#if UE_RIGIDPHYSICS_API_ENABLED
	FRigidPhysicsSceneDataflowState InState = GetValue(SimulationContext, &State);
	*SimulationContext.GetRigidState() = InState;
#endif
}