// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationPlugin.h"
#include "Dataflow/DataflowSimulationNodes.h"
#include "Dataflow/DataflowGraph.h"
#include "Dataflow/DataflowSimulationManager.h"
#include "CoreMinimal.h"

#define LOCTEXT_NAMESPACE "DataflowSimulation"

void IDataflowSimulationPlugin::StartupModule()
{
	UE::Dataflow::RegisterDataflowSimulationNodes();

	UE::Dataflow::RegisterNodeFilter(FDataflowSimulationNode::StaticType());
	UE::Dataflow::RegisterNodeFilter(FDataflowInvalidNode::StaticType());
	UE::Dataflow::RegisterNodeFilter(FDataflowExecutionNode::StaticType());

#if UE_RIGIDPHYSICS_API_ENABLED
	UE::Dataflow::RegisterNodeFilter(FRigidSceneSetupTerminalDataflowNode::StaticType());
	UE::Dataflow::RegisterNodeFilter(FSceneSimulationTerminalDataflowNode::StaticType());
#endif

	UDataflowSimulationManager::OnStartup();
}

void IDataflowSimulationPlugin::ShutdownModule()
{
	UDataflowSimulationManager::OnShutdown();
}

IMPLEMENT_MODULE(IDataflowSimulationPlugin, DataflowSimulation)

#undef LOCTEXT_NAMESPACE
