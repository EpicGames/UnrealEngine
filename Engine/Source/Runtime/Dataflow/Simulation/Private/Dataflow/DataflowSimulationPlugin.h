// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * The public interface to this module
 */
class IDataflowSimulationPlugin : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();
};

