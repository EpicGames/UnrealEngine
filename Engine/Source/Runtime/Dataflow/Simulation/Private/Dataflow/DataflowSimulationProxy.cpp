// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowSimulationProxy)

void FDataflowSimulationProxy::SetSimulationGroups(const TSet<FString>& InSimulationGroups)
{
	SimulationGroups = InSimulationGroups;
}

bool FDataflowSimulationProxy::HasSimulationGroup(const FString& SimulationGroup) const
{
	return (SimulationGroups.Find(SimulationGroup) != nullptr);
}

bool FDataflowSimulationProxy::HasGroupBit(const TBitArray<>& SimulationBits) const
{
	const TBitArray<> CommonGroups = TBitArray<>::BitwiseAND(GroupBits, SimulationBits, EBitwiseOperatorFlags::MinSize);
	return (CommonGroups.CountSetBits() != 0);
}
