// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowSimulationUtils.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSimulationNodes.h"
#include "Dataflow/DataflowObjectInterface.h"

#define LOCTEXT_NAMESPACE "DataflowSimulationGenerator"

namespace UE::Dataflow
{
	void EvaluateSimulationGraph(const TObjectPtr<UDataflow>& SimulationGraph, const TSharedPtr<FDataflowSimulationContext>& SimulationContext,
		const float DeltaTime, const float SimulationTime)
	{
		if(SimulationContext.IsValid())
		{
			SimulationContext->SetTimingInfos(DeltaTime, SimulationTime);
			
			if(SimulationGraph)
			{
				if(const TSharedPtr<UE::Dataflow::FGraph> DataflowGraph = SimulationGraph->GetDataflow())
				{
					// Invalidation of all the simulation nodes that are always dirty
					for(const TSharedPtr<FDataflowNode>& InvalidNode : DataflowGraph->GetFilteredNodes(FDataflowInvalidNode::StaticType()))
					{
						InvalidNode->Invalidate();
					}
					
					// Pull the graph evaluation from the solver nodes
					for(const TSharedPtr<FDataflowNode>& ExecutionNode : DataflowGraph->GetFilteredNodes(FDataflowExecutionNode::StaticType()))
					{
						SimulationContext->Evaluate(ExecutionNode.Get(), nullptr);
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
