// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/Interfaces/DataflowCollisionObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowCollisionObject)

FDataflowCollisionObjectProxy::FDataflowCollisionObjectProxy()
{}

IDataflowCollisionObjectInterface::IDataflowCollisionObjectInterface()
{}
