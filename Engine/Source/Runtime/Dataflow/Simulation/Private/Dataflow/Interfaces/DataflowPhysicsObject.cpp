// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/Interfaces/DataflowPhysicsObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataflowPhysicsObject)

FDataflowPhysicsObjectProxy::FDataflowPhysicsObjectProxy()
{}

IDataflowPhysicsObjectInterface::IDataflowPhysicsObjectInterface()
{}
