// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/RigidPhysicsDataflowNodes.h"
#include "Dataflow/Interfaces/DataflowPhysicsSolver.h"
#include "Dataflow/DataflowSimulationNodes.h"
#include "Dataflow/DataflowSimulationManager.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"
#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidModifier.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidPhysics/RigidPhysicsService.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigidPhysicsDataflowNodes)
void RegisterRigidPhysicsDataflowSimulationNodes()
{
#if UE_RIGIDPHYSICS_API_ENABLED
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRigidSceneSetupDataflowNode);
#endif
}

void FRigidSceneSetupDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const
{
#if UE_RIGIDPHYSICS_API_ENABLED
	using namespace Chaos;
	using namespace Chaos::Rigids::Async;
	using namespace UE::Physics;

	if (Context.IsA(UE::Dataflow::FDataflowSimulationContext::StaticType()))
	{
		UE::Dataflow::FDataflowSimulationContext& SimulationContext = StaticCast<UE::Dataflow::FDataflowSimulationContext&>(Context);
		if (Output->IsA(&State))
		{
			FRigidPhysicsSceneDataflowState InState;
			FRigidPhysicsService& Service = FRigidPhysicsService::GetInstance();
			InState.Handle = Service.CreateScene(FRigidDebugName(SimulationContext.GetRigidSceneName()));
			SetValue(SimulationContext, InState, &State);
		}
	}
#endif
}