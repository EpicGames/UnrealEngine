// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowSimulationProxy.h"

#include "RigidPhysics/RigidFwd.h"

struct FRigidPhysicsSceneDataflowState;

namespace UE::Dataflow
{
	/** Simulation context that will be used by all the simulation nodes*/
	template<typename Base = UE::Dataflow::FContextSingle>
	class TSimulationContext : public UE::Dataflow::TEngineContext<Base>
	{
	public:
		DATAFLOW_CONTEXT_INTERNAL(UE::Dataflow::TEngineContext<Base>, TSimulationContext);

		explicit TSimulationContext(const TObjectPtr<UObject>& InOwner)
				: Super(InOwner)
		{}
		
		virtual ~TSimulationContext() override = default;
		
		/** Set the timing infos */
		void SetTimingInfos(const float DeltaSeconds, const float TimeSeconds) {DeltaTime = DeltaSeconds; SimulationTime = TimeSeconds;}

		/** Get the delta time in seconds */
		float GetDeltaTime() const {return DeltaTime;}
		
		/** Get the simulation time in seconds */
		float GetSimulationTime() const {return SimulationTime;}

		/** Get the typed proxies */
		template<typename ProxyType>
		void GetTypedProxies(TArray<ProxyType*>& FilteredProxies) const;

		/** Filter the physics solvers matching the groups */
		DATAFLOWSIMULATION_API void GetSimulationProxies(const FString& ProxyType, const TArray<FString>& SimulationGroups, TArray<FDataflowSimulationProxy*>& FilteredProxies) const;

		/** Add simulation proxy to the context */
		DATAFLOWSIMULATION_API void AddSimulationProxy(const FString& ProxyType, FDataflowSimulationProxy* SimulationProxy);
		
		/** Remove simulation proxy from the context */
    	DATAFLOWSIMULATION_API void RemoveSimulationProxy(const FString& ProxyType, const FDataflowSimulationProxy* SimulationProxy);

		/** Reset all the simulation proxies */
		DATAFLOWSIMULATION_API void ResetSimulationProxies();
		
		/** Return the number of physics solvers */
		DATAFLOWSIMULATION_API int32 NumSimulationProxies(const FString& ProxyType) const;

		/** Register all the proxy groups used in the proxy */
		DATAFLOWSIMULATION_API void RegisterProxyGroups();

		/** Build group bits based on the string array */
		DATAFLOWSIMULATION_API void BuildGroupBits(const TArray<FString>& SimulationGroups, TBitArray<>& GroupBits) const;

#if UE_RIGIDPHYSICS_API_ENABLED 
		/** Return state associated with the rigids dataflow simulation */
		UE_INTERNAL DATAFLOWSIMULATION_API FRigidPhysicsSceneDataflowState* GetRigidState();

		/** Set the name of scene created through the rigids dataflow */
		UE_INTERNAL DATAFLOWSIMULATION_API void SetRigidSceneName(const FString& InSceneName);

		/** Get the name of scene created through the rigids dataflow */
		UE_INTERNAL DATAFLOWSIMULATION_API const FString& GetRigidSceneName();
#endif

		/** Push another level of iteration indices */
		void PushIterationIndex()
		{
			IterationIndices.Push(0);
		}

		/** Pop the last level of iteration indices */
		void PopIterationIndex()
		{
			IterationIndices.Pop();
		}

		/** Set the last iteration index */
		void SetIterationIndex(const int32 IterationIndex)
		{
			IterationIndices.Last() = IterationIndex;
		}

		/** Get the last iteration index */
		int32 GetIterationIndex()
		{
			return IterationIndices.Last();
		}
		
	private :

		/** Simulation time */
		float SimulationTime = 0.0f;

		/** Delta time */
		float DeltaTime = 0.0f;

		/** List of all the simulation proxies within the context sorted by type */
		TMap<FString, TSet<FDataflowSimulationProxy*>> SimulationProxies;

		/** List of all the simulation groups indices */
		TMap<FString, uint32> GroupIndices;

		/** List of all the nested iteration indices */
		TArray<int32> IterationIndices;

#if UE_RIGIDPHYSICS_API_ENABLED
		UE_INTERNAL TPimplPtr<FRigidPhysicsSceneDataflowState> RigidState;
		UE_INTERNAL FString SceneName = "";
#endif
	};

	template<typename Base>
	template<typename ProxyType>
	void TSimulationContext<Base>::GetTypedProxies(TArray<ProxyType*>& FilteredProxies) const
	{
		if(const TSet<FDataflowSimulationProxy*>* TypedProxies = SimulationProxies.Find(ProxyType::StaticStruct()->GetName()))
		{
			for(FDataflowSimulationProxy* SimulationProxy : *TypedProxies)
			{
				if(SimulationProxy)
				{
					FilteredProxies.Add(SimulationProxy->AsType<ProxyType>());
				}
			}
		}
	}

	using FDataflowSimulationContext = TSimulationContext<UE::Dataflow::FContextSingle>;
	using FDataflowSimulationContextThreaded =  TSimulationContext<UE::Dataflow::FContextThreaded>;
}
