// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowObject.h"
#include "UObject/Interface.h"

#include "DataflowSimulationInterface.generated.h"

#define UE_API DATAFLOWSIMULATION_API

struct FDataflowSimulationProxy;

/**
 * Dataflow simulation asset (should be in the interface children)
 */
USTRUCT(BlueprintType)
struct FDataflowSimulationAsset
{
	GENERATED_BODY()

	/* Simulation dataflow asset used to advance in time on Pt */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics")
	TObjectPtr<UDataflow> DataflowAsset;

	/* Simulation groups used to filter within the simulation nodes*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics")
	TSet<FString> SimulationGroups;
};

UINTERFACE(MinimalAPI)
class UDataflowSimulationInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Dataflow simulation interface to send/receive datas (GT <-> PT)
 */
class IDataflowSimulationInterface
{
	GENERATED_BODY()
	
public:
	IDataflowSimulationInterface() {}

	/** Get the dataflow simulation asset */
	virtual FDataflowSimulationAsset& GetSimulationAsset() = 0;

	/** Get the const dataflow simulation asset */
	virtual const FDataflowSimulationAsset& GetSimulationAsset() const = 0;
	
	/** Build the simulation proxy */
	virtual void BuildSimulationProxy() = 0;
	
	/** Reset the simulation proxy */
	virtual void ResetSimulationProxy() = 0;

	/** Get the const simulation proxy */
	virtual const FDataflowSimulationProxy* GetSimulationProxy() const = 0;

	/** Get the simulation proxy */
	virtual FDataflowSimulationProxy* GetSimulationProxy() = 0;

	/** Get the simulation name */
	virtual FString GetSimulationName() const = 0;

	/** Preprocess data before simulation */
	virtual void PreProcessSimulation(const float DeltaTime) {};

	/** Write data to be sent to the simulation proxy */
	virtual void WriteToSimulation(const float DeltaTime, const bool bAsyncTask) {};

	/** Read data received from the simulation proxy */
	virtual void ReadFromSimulation(const float DeltaTime, const bool bAsyncTask) {};

	/** Read restart data (positions) from simulation proxy */
	virtual void ReadRestartData() {};

	/** Postprocess data after simulation */
	virtual void PostProcessSimulation(const float DeltaTime) {};
	
	/** Get the simulation type */
    virtual FString GetSimulationType() const {return TEXT("");};

	/** Register simulation interface solver to manager */
	UE_API void RegisterManagerInterface(const TObjectPtr<UWorld>& SimulationWorld);

	/** Unregister simulation interface from the manager */
	UE_API void UnregisterManagerInterface(const TObjectPtr<UWorld>& SimulationWorld) const;

	/** Check if the interface has been registered to the manager */
	UE_API bool IsInterfaceRegistered(const TObjectPtr<UWorld>& SimulationWorld) const;
};

#undef UE_API
