// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Subsystems/WorldSubsystem.h"
#include "Templates/SharedPointer.h"
#include "Dataflow/DataflowSimulationContext.h"

#include "DataflowSimulationManager.generated.h"

#define UE_API DATAFLOWSIMULATION_API

struct FDataflowSimulationProxy;
class IDataflowSimulationInterface;

namespace UE::Dataflow
{
	namespace Private
	{
		/** Per dataflow graph simulation data type (data interfaces + simulation context) */
		struct  FDataflowSimulationData
		{
			/** List of all simulation interfaces used in this this dataflow graph */
			TMap<FString, TSet<IDataflowSimulationInterface*>> SimulationInterfaces;

			/** Simulation context used to evaluate the graph on PT */
			TSharedPtr<UE::Dataflow::FDataflowSimulationContext> SimulationContext;

			/** Check is there is any datas to process */
			bool IsEmpty() const
			{
				for(const TPair<FString, TSet<IDataflowSimulationInterface*>>& InterfacesPair : SimulationInterfaces)
				{
					if(!InterfacesPair.Value.IsEmpty())
					{
						return false;
					}
				}
				return true;
			}
		};
	}
	DATAFLOWSIMULATION_API void RegisterSimulationInterface(const TObjectPtr<UObject>& SimulationObject);
	DATAFLOWSIMULATION_API void UnregisterSimulationInterface(const TObjectPtr<UObject>& SimulationObject);
}

UCLASS(MinimalAPI)
class UDataflowSimulationManager : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	UE_API UDataflowSimulationManager();
	virtual ~UDataflowSimulationManager() override  = default;

	/** Static function to add world delegates */
	static UE_API void OnStartup();
	
	/** Static function to remove world delegates */
	static UE_API void OnShutdown();

	// Begin FTickableGameObject overrides
	UE_API virtual void Tick(float DeltaTime) override;
	virtual bool IsTickableInEditor() const override { return true; }
	UE_API virtual ETickableTickType GetTickableTickType() const override;
	UE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;
	UE_API virtual TStatId GetStatId() const override;
	// End FTickableGameObject overrides
	
	// Begin USubsystem overrides
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	// End USubsystem overrides

	/** Advance in time the registered simulation data (PT) */
    UE_API void AdvanceSimulationProxies(const float DeltaTime, const float SimulationTime);

	/** Check if the manager has a simulation interface */
	UE_API bool HasSimulationInterface(const IDataflowSimulationInterface* SimulationInterface) const;

	/** Add a dataflow simulation interface to the manager*/
	UE_API void AddSimulationInterface(IDataflowSimulationInterface* SimulationInterface);

	/** Remove a dataflow simulation interface from the manager */
	UE_API void RemoveSimulationInterface(const IDataflowSimulationInterface* SimulationInterface);

	/** Read the simulation interfaces and Write all the data to the simulation proxies (to be send from GT->PT) */
	UE_API void ReadSimulationInterfaces(const float DeltaTime, const bool bAsyncTask);

	/** Read all the data from the simulation proxies and write the result onto the interfaces (received from PT->GT) */
	UE_API void WriteSimulationInterfaces(const float DeltaTime, const bool bAsyncTask);

	/** Read the restart data and Write it to the simulation proxies (to be send from GT->PT) */
	UE_API void ReadRestartData();

	/** Init all the simulation interfaces*/
	UE_API void InitSimulationInterfaces();

	/** Reset all the simulation interfaces*/
	UE_API void ResetSimulationInterfaces();

	/** Complete all the simulation tasks */
	UE_API void CompleteSimulationTasks();

	/** Start the simulation tasks given a delta time */
	UE_API void StartSimulationTasks(const float DeltaTime, const float SimulationTime);

	/** Gets whether the simulation is enabled or disabled */
	bool GetSimulationEnabled() const { return bIsSimulationEnabled; }

	/** Set the simulation flag to enable/disable the simulation */
	UE_API void SetSimulationEnabled(const bool bSimulationEnabled);

	/** Forward the current simulation enabled state to a freshly spawned preview actor */
	UE_API void NotifyActorOfCurrentSimulationEnabled(class AActor* PreviewActor) const;

	/** Set the simulation scene stepping bool */
	void SetSimulationStepping(const bool bSimulationStepping) {bStepSimulationScene = bSimulationStepping;}

	/** Get the simulation context for a given asset */
	UE_API TSharedPtr<UE::Dataflow::FDataflowSimulationContext> GetSimulationContext(const TObjectPtr<UDataflow>& DataflowAsset) const;
	
private :

	/** Pre process before the simulation step */
	UE_API void PreProcessSimulation(const float DeltaTime);

	/** Post process after the simulation step */
	UE_API void PostProcessSimulation(const float DeltaTime);

	/** static delegate for object property changed */
	static UE_API FDelegateHandle OnObjectPropertyChangedHandle;

	/** static delegate for post actor tick */
	static UE_API FDelegateHandle OnWorldPostActorTick;

	/** static delegate for physics state creation */
	static UE_API FDelegateHandle OnCreatePhysicsStateHandle;

	/** static delegate for physics state destruction */
	static UE_API FDelegateHandle OnDestroyPhysicsStateHandle;

	/** Dataflow simulation data registered to the manager */
	TMap<TObjectPtr<UDataflow>, UE::Dataflow::Private::FDataflowSimulationData> SimulationData;
	
	/** Simulation tasks in which the graph will be evaluated */
	TArray<FGraphEventRef> SimulationTasks;
 
	/** Boolean to control if the simulation should be disabled or not */
	bool bIsSimulationEnabled = true;
	
	/** Boolean to check if we are stepping the simulation scene */
	bool bStepSimulationScene = false;
};

/** Dataflow simulation actor interface to be able to call BP events before/after the manager ticking in case we need it */
UINTERFACE(MinimalAPI, Blueprintable)
class UDataflowSimulationActor : public UInterface
{
	GENERATED_BODY()
};

class IDataflowSimulationActor
{
	GENERATED_BODY()
 
public:
	/** Pre simulation callback function that can be implemented in C++ or Blueprint. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Dataflow")
	void PreDataflowSimulationTick(const float SimulationTime, const float DeltaTime);

	/** Post simulation callback function that can be implemented in C++ or Blueprint. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Dataflow")
	void PostDataflowSimulationTick(const float SimulationTime, const float DeltaTime);

	/** Called on the preview actor when the Dataflow Editor's simulation enabled state changes. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Dataflow")
	void OnDataflowSimulationEnabledChanged(const bool bSimulationEnabled);
};

#undef UE_API
