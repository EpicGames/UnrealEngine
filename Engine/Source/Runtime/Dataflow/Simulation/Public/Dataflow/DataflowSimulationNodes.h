// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSimulationProxy.h"
#include "Dataflow/DataflowSimulationContext.h"
#include "Dataflow/RigidPhysicsSceneState.h"

#include "DataflowSimulationNodes.generated.h"

#define UE_API DATAFLOWSIMULATION_API

/**
 * Dataflow simulation property 
 */
USTRUCT(BlueprintType)
struct FDataflowSimulationProperty
{
	GENERATED_BODY()

	/* Simulation proxy used to pass information in between nodes */
	FDataflowSimulationProxy* SimulationProxy;
};

/**
* FDataflowSimulationNode
*		Base class for simulation nodes within the Dataflow graph. 
* 
*		Simulation nodes are used to simulate data from the calling client. 
*/
USTRUCT()
struct FDataflowSimulationNode : public FDataflowNode
{
	GENERATED_BODY()

	FDataflowSimulationNode()
		: Super() { }

	FDataflowSimulationNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid())
		: Super(Param,InGuid) {
	}

	/** FDataflowNode interface */
	virtual ~FDataflowSimulationNode() = default;
	
	static FName StaticType() { return FName("FDataflowSimulationNode"); }

	virtual bool IsA(FName InType) const override 
	{ 
		return InType.ToString().Equals(StaticType().ToString()) 
			|| Super::IsA(InType); 
	}

	/** Evaluate simulation dispatch */
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const override
	{
		if(Context.IsA(UE::Dataflow::FDataflowSimulationContext::StaticType()))
		{
			UE::Dataflow::FDataflowSimulationContext& SimulationContext = StaticCast<UE::Dataflow::FDataflowSimulationContext&>(Context);
			EvaluateSimulation(SimulationContext, Output);
			
		}
	}

protected :

	/** Evaluate simulation node given a simulation context */
	virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const {};
};

/**
* FDataflowInvalidNode
*		Base class for invalid nodes within the Dataflow graph. 
* 
*		Invalid nodes will be always invalidated while simulating
*/
USTRUCT()
struct FDataflowInvalidNode : public FDataflowSimulationNode
{
	GENERATED_BODY()

	FDataflowInvalidNode()
		: Super() { }

	FDataflowInvalidNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid())
		: Super(Param,InGuid) {
	}

	/** FDataflowNode interface */
	virtual ~FDataflowInvalidNode() = default;
	
	static FName StaticType() { return FName("FDataflowInvalidNode"); }

	virtual bool IsA(FName InType) const override 
	{ 
		return InType.ToString().Equals(StaticType().ToString()) 
			|| Super::IsA(InType); 
	}
};

/**
* FDataflowExecutionNode
*		Base class for the execute the dataflow simulation graph. 
* 
*		Execution nodes are used to pull the graph from the calling client. 
*/
USTRUCT()
struct FDataflowExecutionNode : public FDataflowSimulationNode
{
	GENERATED_BODY()

	FDataflowExecutionNode()
		: Super() { }

	FDataflowExecutionNode(const UE::Dataflow::FNodeParameters& Param, FGuid InGuid = FGuid::NewGuid())
		: Super(Param,InGuid) {
	}

	/** FDataflowNode interface */
	virtual ~FDataflowExecutionNode() = default;
	
	static FName StaticType() { return FName("FDataflowExecutionNode"); }

	virtual bool IsA(FName InType) const override 
	{ 
		return InType.ToString().Equals(StaticType().ToString()) 
			|| Super::IsA(InType); 
	}

	virtual bool IsTerminal() const override
	{
		return true;
	}
};

/** Get the dataflow simulation time */
USTRUCT()
struct FDataflowSimulationTime 
{
	GENERATED_BODY()

public:
	FDataflowSimulationTime(): DeltaTime(0.0f), CurrentTime(0.0f)
	{}
	
	FDataflowSimulationTime(const float InDeltaTime, const float InCurrentTime) : DeltaTime(InDeltaTime), CurrentTime(InCurrentTime)
	{}
	
	/** Delta time in seconds coming from the context */
	UPROPERTY(Transient, SkipSerialization)
	float DeltaTime = 0.0f;

	/** Current time in seconds coming from the context */
	UPROPERTY(Transient, SkipSerialization)
	float CurrentTime = 0.0f;

	/** Time offset that will need to be added to the current time to compute the sub-stepping */
	UPROPERTY(Transient, SkipSerialization)
	float TimeOffset = 0.0f;
};

/** Get the context simulation time */
USTRUCT(meta = (DataflowSimulation))
struct FGetSimulationTimeDataflowNode : public FDataflowInvalidNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetSimulationTimeDataflowNode, "GetSimulationTime", "Physics|Common", UDataflow::SimulationTag)

public:
	
	FGetSimulationTimeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowInvalidNode(InParam, InGuid)
	{
		RegisterOutputConnection(&SimulationTime);
	}
	
	/** Simulation time property coming from the context */
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowOutput))
	FDataflowSimulationTime SimulationTime = FDataflowSimulationTime(0.0f, 0.0f);

	/** Evaluate simulation node given a simulation context */
	UE_API virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const override;
};

/** Get physics solvers from context */
USTRUCT(meta = (DataflowSimulation))
struct FGetPhysicsSolversDataflowNode : public FDataflowInvalidNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetPhysicsSolversDataflowNode, "GetPhysicsSolvers", "Physics|Solver", UDataflow::SimulationTag)

public:
	
	FGetPhysicsSolversDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowInvalidNode(InParam, InGuid)
	{
		RegisterOutputConnection(&PhysicsSolvers);
	}

	/** Physics solvers coming from the context and filtered with the groups */
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowOutput))
	TArray<FDataflowSimulationProperty> PhysicsSolvers;

	/** Simulation groups to filter the output solvers properties */
	UPROPERTY(EditAnywhere, Category="Simulation")
	TArray<FString> SimulationGroups;

	/** Evaluate simulation node given a simulation context */
	virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const override;
};

/** Advance the simulation physics solver in time */
USTRUCT(meta = (DataflowSimulation))
struct FAdvancePhysicsSolversDataflowNode : public FDataflowSimulationNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAdvancePhysicsSolversDataflowNode, "AdvancePhysicsSolvers", "Physics|Solver", UDataflow::SimulationTag)

public:
	
	FAdvancePhysicsSolversDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowSimulationNode(InParam, InGuid)
	{
		RegisterInputConnection(&SimulationTime);
		RegisterInputConnection(&PhysicsSolvers);
		RegisterOutputConnection(&PhysicsSolvers, &PhysicsSolvers);
	}

	/** Delta time to use to advance the solver*/
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput))
	FDataflowSimulationTime SimulationTime = FDataflowSimulationTime(0.0f,0.0f);

	/** Physics solvers to advance in time */
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "PhysicsSolvers"))
	TArray<FDataflowSimulationProperty> PhysicsSolvers;

	/** Evaluate simulation node given a simulation context */
	virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const override;
};

/** Filter simulation proxies from context */
USTRUCT(meta = (DataflowSimulation))
struct FFilterSimulationProxiesDataflowNode : public FDataflowSimulationNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFilterSimulationProxiesDataflowNode, "FilterSimulationProxies", "Physics|Proxy", UDataflow::SimulationTag)

public:
	
	FFilterSimulationProxiesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowSimulationNode(InParam, InGuid)
	{
		RegisterInputConnection(&SimulationProxies);
		RegisterOutputConnection(&FilteredProxies);
	}

	/** Simulation proxies coming from the context and filtered with the groups */
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput))
	TArray<FDataflowSimulationProperty> SimulationProxies;
 
	/** Simulation proxies coming from the context and filtered with the groups */
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowOutput))
	TArray<FDataflowSimulationProperty> FilteredProxies;

	/** Simulation groups to filter the output solvers properties */
	UPROPERTY(EditAnywhere, Category="Simulation")
	TArray<FString> SimulationGroups;

	/** Evaluate simulation node given a simulation context */
	virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const override;
};

/** Main terminal node for simulation proxies */
USTRUCT(meta = (DataflowSimulation))
struct FSimulationProxiesTerminalDataflowNode : public FDataflowExecutionNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSimulationProxiesTerminalDataflowNode, "SimulationProxiesTerminal", "Terminal|Proxy", UDataflow::SimulationTag)

public:
	
	FSimulationProxiesTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowExecutionNode(InParam, InGuid)
	{
		RegisterInputConnection(&SimulationProxies);
	}

	/** Physics solvers to evaluate  */
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput))
	TArray<FDataflowSimulationProperty> SimulationProxies;
	
	/** Evaluate simulation node given a simulation context */
	virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const override;
};

/** Advance the rigid solver in time */
USTRUCT(meta = (DataflowSimulation))
struct FAdvanceRigidPhysicsSolverDataflowNode : public FDataflowSimulationNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAdvanceRigidPhysicsSolverDataflowNode, "AdvanceRigidSolver", "Physics|Solver", UDataflow::SimulationTag)

public:
	
	FAdvanceRigidPhysicsSolverDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowSimulationNode(InParam, InGuid)
	{
		RegisterInputConnection(&SimulationTime);
		RegisterInputConnection(&State);
		RegisterOutputConnection(&State,&State);
	}

	/** Delta time to use to advance the solver*/
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput))
	FDataflowSimulationTime SimulationTime = FDataflowSimulationTime(0.0f,0.0f);

	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "State"))
	FRigidPhysicsSceneDataflowState State;

	/** Evaluate simulation node given a simulation context */
	virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const override;
};

/** Main terminal node for simulation */
USTRUCT(meta = (DataflowSimulation))
struct FSceneSimulationTerminalDataflowNode : public FDataflowSimulationNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSceneSimulationTerminalDataflowNode, "SceneSimulationTerminal", "Terminal|Scene", UDataflow::SimulationTag)
public:
	
	FSceneSimulationTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowSimulationNode(InParam, InGuid)
	{
		RegisterInputConnection(&State);
	}
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput))
	FRigidPhysicsSceneDataflowState State;
	
	/** Evaluate simulation node given a simulation context */
	virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const override;
};

/** Main terminal node for scene setup */
USTRUCT(meta = (DataflowSimulation))
struct FRigidSceneSetupTerminalDataflowNode : public FDataflowSimulationNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRigidSceneSetupTerminalDataflowNode, "RigidSceneSetupTerminal", "Terminal|Scene", UDataflow::SimulationTag)
public:
	
	FRigidSceneSetupTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowSimulationNode(InParam, InGuid)
	{
		RegisterInputConnection(&State);
	}
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput))
	FRigidPhysicsSceneDataflowState State;
	
	/** Evaluate simulation node given a simulation context */
	virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const override;
};

namespace UE::Dataflow
{
	void RegisterDataflowSimulationNodes();
}


#undef UE_API
