// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataflowSimulationProxy.generated.h"

#define UE_API DATAFLOWSIMULATION_API

/**
 * Dataflow simulation proxy used for simulation (PT)
 */
USTRUCT()
struct FDataflowSimulationProxy
{
	GENERATED_BODY()
	
	FDataflowSimulationProxy() = default;
	virtual ~FDataflowSimulationProxy() = default;
	
	/** Check if the proxy is valid */
	virtual bool IsValid() const { return true;}
	
	/** Get the proxy script struct */
	virtual const UScriptStruct* GetScriptStruct() const
	{
		return StaticStruct();
	}
	
	/** Cast the proxy to child */
	template<class T>
	const T* AsType() const
	{
		const UScriptStruct* ScriptStruct = GetScriptStruct();
		if(ScriptStruct && ScriptStruct->IsChildOf(T::StaticStruct()))
		{
			return static_cast<T*>(this);
		}
		return nullptr;
	}
	
	/** Cast the const proxy to child */
	template<class T>
	T* AsType()
	{
		const UScriptStruct* ScriptStruct = GetScriptStruct();
		if(ScriptStruct && ScriptStruct->IsChildOf(T::StaticStruct()))
		{
			return static_cast<T*>(this);
		}
		return nullptr;
	}

	/** Get the simulation groups from the proxy */
	const TSet<FString>& GetSimulationGroups() const {return SimulationGroups;};

	/** Set the simulation groups onto the proxy */
	UE_API void SetSimulationGroups(const TSet<FString>& InSimulationGroups);

	/** Check if the given group is within the proxy simulation groups*/
	UE_API bool HasSimulationGroup(const FString& SimulationGroup) const;

	/** Check if the proxy has at least one valid bit */
    UE_API bool HasGroupBit(const TBitArray<>& SimulationBits) const;

	/** Bit array matching the simulation groups for fast access */
	TBitArray<> GroupBits;

private:
	/** List of simulation groups this proxy belongs to */
	TSet<FString> SimulationGroups;
};

#undef UE_API
