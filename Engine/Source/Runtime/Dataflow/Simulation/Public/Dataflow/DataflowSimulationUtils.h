// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowObjectInterface.h"
#include "Dataflow/DataflowSimulationContext.h"

class UDataflow;

namespace UE::Dataflow
{
	/** Evaluate the simulation graph given a simulation context and a simulation flag */
	DATAFLOWSIMULATION_API void EvaluateSimulationGraph(const TObjectPtr<UDataflow>& SimulationGraph,
		const TSharedPtr<UE::Dataflow::FDataflowSimulationContext>& SimulationContext, const float DeltaTime, const float SimulationTime);
};


