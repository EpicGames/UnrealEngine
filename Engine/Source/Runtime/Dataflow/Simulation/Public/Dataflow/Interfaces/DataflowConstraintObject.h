// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Dataflow/DataflowSimulationInterface.h"
#include "Dataflow/DataflowSimulationProxy.h"
#include "DataflowConstraintObject.generated.h"

#define UE_API DATAFLOWSIMULATION_API

/**
 * Dataflow collision object proxy (PT)
 */
USTRUCT()
struct FDataflowConstraintObjectProxy : public FDataflowSimulationProxy
{
	GENERATED_BODY()
	
	UE_API FDataflowConstraintObjectProxy();
	virtual ~FDataflowConstraintObjectProxy() override = default;

	/** Get the proxy script struct */
	virtual const UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
};

UINTERFACE(MinimalAPI)
class UDataflowConstraintObjectInterface : public UDataflowSimulationInterface
{
	GENERATED_BODY()
};

/**
 * Dataflow collision object interface to send/receive datas (GT <-> PT)
 */
class IDataflowConstraintObjectInterface : public IDataflowSimulationInterface
{
	GENERATED_BODY()
	
public:
	UE_API IDataflowConstraintObjectInterface();

	/** Get the simulation type */
	virtual FString GetSimulationType() const override
	{
		return FDataflowConstraintObjectProxy::StaticStruct()->GetName();
	}
};

#undef UE_API
