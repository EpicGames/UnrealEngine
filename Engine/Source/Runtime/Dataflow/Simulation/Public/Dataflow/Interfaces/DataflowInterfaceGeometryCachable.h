// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Misc/Optional.h"
#include "Containers/ArrayView.h"
#include "Math/MathFwd.h"
#include "DataflowInterfaceGeometryCachable.generated.h"

class USkeletalMesh;
class USkinnedAsset;
class UStaticMesh;

UINTERFACE(meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class UDataflowGeometryCachable : public UInterface
{
	GENERATED_BODY()
};

/** An asset that can produces a geometry cache */
class IDataflowGeometryCachable
{
	GENERATED_BODY()

public:
	/** Computes skeletal mesh positions for geometry cache */
	virtual TArray<FVector3f> GetGeometryCachePositions(const USkeletalMesh* SkeletalMesh) const = 0;

	/** Computes static mesh positions for geometry cache */
	virtual TArray<FVector3f> GetGeometryCachePositions(const UStaticMesh* StaticMesh) const = 0;

	/** Get mapping from final mesh vertex index to raw import vertex index */
	virtual TOptional<TArray<int32>> GetMeshImportVertexMap(const USkinnedAsset& SkinnedMeshAsset) const = 0;

	/** Returns true if the asset supports animation update from a skeletal mesh  */
	virtual bool IsSkeletalMeshAnimationCompatible(const USkeletalMeshComponent* SkeletalMeshComponent) const { return true; };
};
