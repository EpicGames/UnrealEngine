// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Dataflow/DataflowSimulationInterface.h"
#include "Dataflow/DataflowSimulationProxy.h"
#include "DataflowPhysicsObject.generated.h"

#define UE_API DATAFLOWSIMULATION_API

/**
 * Dataflow physics object proxy (PT)
 */
USTRUCT()
struct FDataflowPhysicsObjectProxy : public FDataflowSimulationProxy
{
	GENERATED_BODY()
	
	UE_API FDataflowPhysicsObjectProxy();
	virtual ~FDataflowPhysicsObjectProxy() override = default;

	/** Get the proxy script struct */
	virtual const UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
};

UINTERFACE(MinimalAPI)
class UDataflowPhysicsObjectInterface : public UDataflowSimulationInterface
{
	GENERATED_BODY()
};

/**
 * Dataflow physics object interface to send/receive datas (GT <-> PT)
 */
class IDataflowPhysicsObjectInterface : public IDataflowSimulationInterface
{
	GENERATED_BODY()
	
public:
	UE_API IDataflowPhysicsObjectInterface();

	/** Get the simulation type */
	virtual FString GetSimulationType() const override
	{
		return FDataflowPhysicsObjectProxy::StaticStruct()->GetName();
	}
};

#undef UE_API
