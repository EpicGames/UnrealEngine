// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Dataflow/DataflowSimulationInterface.h"
#include "Dataflow/DataflowSimulationProxy.h"
#include "DataflowPhysicsSolver.generated.h"

#define UE_API DATAFLOWSIMULATION_API

/**
 * Dataflow simulation physics solver proxy (PT)
 */
USTRUCT()
struct FDataflowPhysicsSolverProxy : public FDataflowSimulationProxy
{
	GENERATED_BODY()

	UE_API FDataflowPhysicsSolverProxy();
	virtual ~FDataflowPhysicsSolverProxy() override = default;

	/** Get the proxy script struct */
	virtual const UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}

	/** Advance the solver datas in time */
	virtual void ReadRestartData() {};

	/** Advance the solver datas in time */
	virtual void AdvanceSolverDatas(const float DeltaTime) {}

	/** Get the solver time step */
	virtual float GetTimeStep() { return 0.033f;}
};

UINTERFACE(MinimalAPI)
class UDataflowPhysicsSolverInterface : public UDataflowSimulationInterface
{
	GENERATED_BODY()
};

/**
 * Dataflow physics solver interface to send/receive (GT)
 */
class IDataflowPhysicsSolverInterface : public IDataflowSimulationInterface
{
	GENERATED_BODY()
	
public:
	UE_API IDataflowPhysicsSolverInterface();
	
	/** Get the simulation type */
	virtual FString GetSimulationType() const override
	{
		return FDataflowPhysicsSolverProxy::StaticStruct()->GetName();
	}
};




#undef UE_API
