// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSimulationProxy.h"
#include "Dataflow/DataflowSimulationContext.h"
#include "Dataflow/DataflowSimulationNodes.h"
#include "Dataflow/RigidPhysicsSceneState.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidPhysicsDataflowNodes.generated.h"

/** Create a rigid scene */
USTRUCT(meta = (DataflowSimulation))
struct FRigidSceneSetupDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRigidSceneSetupDataflowNode, "CreateRigidScene", "Physics|Scene", UDataflow::SimulationTag)
public:

	FRigidSceneSetupDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&State);
	}

	UPROPERTY(meta = (DataflowOutput))
	FRigidPhysicsSceneDataflowState State;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Output) const override;
	//~ End FDataflowNode interface
};

void RegisterRigidPhysicsDataflowSimulationNodes();
