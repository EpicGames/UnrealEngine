// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED
#include "RigidPhysics/RigidFactory.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidPhysics/RigidBody.h"
#endif

#include "GameFramework/Actor.h"
#include "RigidPhysicsSceneState.generated.h"
USTRUCT()
struct FRigidPhysicsSceneDataflowState
{
	GENERATED_BODY()
public:
#if UE_RIGIDPHYSICS_API_ENABLED
	UE::Physics::FRigidSceneHandle Handle;
#endif
};
